[CoreRedirects]

+PackageRedirects=(OldName="/DatasmithContent/Materials/MDL/...", NewName="/MDLImporter/Materials/MDL/", MatchWildcard=true)
