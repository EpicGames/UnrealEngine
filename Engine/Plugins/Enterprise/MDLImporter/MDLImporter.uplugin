{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "MDL Importer",
	"Description": "Importer for MDL material files.",
	"Category": "Importers",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"SupportedTargetPlatforms": [
		"Win64",
		"Mac",
		"Linux"
	],
	"Modules": [
		{
			"Name": "MDLImporter",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Mac",
				"Linux"
			]
		}
	]
}