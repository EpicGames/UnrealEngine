// Copyright Epic Games, Inc. All Rights Reserved.

/**
Flakes texture access based on light angular direction, uses a pseduo(i.e. in atlas format) 2D texture array
to access flakes slices and finally applies bilinear interpolation.

@param ThetaAngles - normalized(by Pi/2) thetaF and thetaI angles
@param FlakesTexArray - texture 2d array in an atlas format
@param FlakesTexCount - number of textures in the array
@param FlakesTexValues - x = texture size of an entry in the 2d array, y = ThetaSliceLUT texel size
@param TexCoord - UV coordinates
@param Tiling - UV tiling factor
@param NumThetaF - number of ThetaF bins
@param NumThetaI- number of thetaI bins
@param MaxThetaI - max thetaI bin that is used
@param ThetaSliceLUT - 1d texture with the index into flake texture array for a given thetaI bin
@return Bilinear interpolated value of the flakes.
*/

#define PI2 1.5707963267948966
#define USE_GRADIENTS 1

// NOTE. had to use macros as function structs aren't portable on certain platforms such as Vulkan

#if USE_GRADIENTS == 1 
#define SAMPLE_TEX_COORD(SampleTexCoord, Result) \
    float2 DdxUV = DDX(SampleTexCoord); \
    float2 DdyUV = DDY(SampleTexCoord); \
    Result = Texture2DSampleGrad(FlakesTexArray, FlakesTexArraySampler, SampleTexCoord, DdxUV, DdyUV); 
#else 
    Result = Texture2DSample(FlakesTexArray, FlakesTexArraySampler, SampleTexCoord); 
#endif 

#define SAMPLE_TEX_ARRAY(UV, Index, Result) \
{ \
    float SideCount = sqrt(FlakesTexCount); \
    float InvSideCount = 1.0 / SideCount;  \
    float TexPadding = 8.0; \
    float TexSize = FlakesTexValues.x; \
    float PaddedTexSize = TexSize + TexPadding * 2.0; \
    float AtlasTexelSize = 1.0 / (SideCount * PaddedTexSize); \
 \
    float2 TexOffset = Index; \
    TexOffset.x = fmod(TexOffset.x, SideCount); \
    TexOffset.y = floor(TexOffset.y * InvSideCount); \
    TexOffset *= InvSideCount; \
    TexOffset += TexPadding * AtlasTexelSize; \
 \
    float2 SampleTexCoord = frac(UV); \
    SampleTexCoord *= InvSideCount * (TexSize / PaddedTexSize); \
    SampleTexCoord += TexOffset; \
    SAMPLE_TEX_COORD(SampleTexCoord, Result); \
}   

#define SAMPLE_LUT(BinThetaI, Result) \
{ \
    float TexelSize = FlakesTexValues.y; \
    float Padding = 4.0; \
    float Index = (BinThetaI) * Padding * TexelSize + TexelSize; \
    float Value = Texture2DSampleLevel(ThetaSliceLUT, ThetaSliceLUTSampler, float2(Index, 0.0), 0.0).x; \
    Result = floor(Value * 255.0 + 0.5); \
}

//ThetaF sampling defines the angular sampling, i.e. angular flake lifetime
float ThetaF = clamp(ThetaAngles.x / PI2, 0.0, 1.0) * NumThetaF + 0.5;
float ThetaI = clamp(ThetaAngles.y / PI2, 0.0, 1.0) * NumThetaF + 0.5;

//bilinear interpolation indices
float LowThetaF = floor(ThetaF);
float HighThetaF = LowThetaF + 1.0;
float LowThetaI = floor(ThetaI);
float HighThetaI = LowThetaI + 1.0;

float2 UV = TexCoord * Tiling;
float2 DdxUV = DDX(UV);
float2 DdyUV = DDY(UV);
if (NumThetaI < NumThetaF)
{
    if ((LowThetaI % 2) == 1 || (HighThetaI % 2) == 1) 
    { 
        //tweak for extra randomness, in the case of almost planar geometries with repeating patches
        UV.xy = UV.yx;
    }

    //map to the original sampling
    float InvNumThetaF = 1.0 / NumThetaF;
    LowThetaI = int(floor(float(LowThetaI) * NumThetaI * InvNumThetaF));
    HighThetaI = int(floor(float(HighThetaI) * NumThetaI * InvNumThetaF));
}

half3 ColorBottomLeft = 0.0;
half3 ColorTopLeft = 0.0;
half3 ColorBottomRight = 0.0;
half3 ColorTopRight  = 0.0;
//access flakes texture and stay in the correct slices(no slip over)
if (LowThetaI < MaxThetaI)
{
    float BaseThetaIndex, UpperIndexLimit;
    SAMPLE_LUT(LowThetaI, BaseThetaIndex);
    SAMPLE_LUT(LowThetaI + 1.0, UpperIndexLimit);

    float LowThetaIndex = BaseThetaIndex + LowThetaF;
    if (LowThetaIndex < UpperIndexLimit)
    {
        SAMPLE_TEX_ARRAY(UV, LowThetaIndex, ColorBottomLeft.xyz);
    }
    float HighThetaIndex = BaseThetaIndex + HighThetaF;
    if (HighThetaIndex < UpperIndexLimit)
    {
        SAMPLE_TEX_ARRAY(UV, HighThetaIndex, ColorBottomRight.xyz);
    }
}

if (HighThetaI < MaxThetaI)
{
    float BaseThetaIndex, UpperIndexLimit;
    SAMPLE_LUT(HighThetaI, BaseThetaIndex);
    SAMPLE_LUT(HighThetaI + 1.0, UpperIndexLimit);

    float LowThetaIndex = BaseThetaIndex + LowThetaF;
    if (LowThetaIndex < UpperIndexLimit)
    {
        SAMPLE_TEX_ARRAY(UV, LowThetaIndex, ColorTopLeft.xyz);
    }
    float HighThetaIndex = BaseThetaIndex + HighThetaF;
    if (HighThetaIndex < UpperIndexLimit)
    {
        SAMPLE_TEX_ARRAY(UV, HighThetaIndex, ColorTopRight.xyz);
    }
}

//bilinear interpolation
float DiffThetaF = ThetaF - float(LowThetaF);
float DiffThetaI = ThetaI - float(LowThetaI);
half3 Bottom = (1.0 - DiffThetaF) * ColorBottomLeft + DiffThetaF * ColorBottomRight;
half3 Top = (1.0 - DiffThetaF) * ColorTopLeft + DiffThetaF * ColorTopRight;    
return (1.0 - DiffThetaI) * Bottom + DiffThetaI * Top;
