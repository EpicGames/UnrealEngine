// Copyright Epic Games, Inc. All Rights Reserved.

/*
Spiral blur function with IOR used to emulate BTDF of the material.

 @param RandomTex - Random texture input for spiral blur
 @param SurfaceScatter - Scattering by depth behind the object (0.0 to 1.0)
 @param VolumeScatter - Scattering by object's thickness (0.0 to 1.0)
 @param IOR - Index of refraction (defaults to 1.0 for no refraction)
 @param Normal - Normal texture input
 @param ObjectThickness - Estimated or calculated object's thickness 
 @param PixelDepth - Pixel depth input (TODO could be sampled in code)

 @param NormalAlignFactor - Factor for Normal to Screen alignment, controls refraction and
                            keeps it within screen space (0.0 to positive infinity with 0.0
                            meaning no correction)
 @param MagicRefractionNumber - Directly controls the amount of refraction offset when sampling
                                the scene (1.0 means no correction)

*/// --------------------------------------------------------------------------

const int RadialSteps = 48;
const int AngularSteps = 1;

const float MagicBlurNumber = 100.0;

// Thickness added to account for normal texture features when
// calculating refraction light bending.
const float RefractionThicknessCorrection = 50.0;

float UVDistance = 0.2;
float AngularOffset = 0.618;

float SizeOnScreenFactor = View.ViewToClip[0][0] / PixelDepth;

float2 ScreenUV = MaterialFloat2(ScreenAlignedPosition(Parameters.ScreenPosition).xy);
float2 BaseUV = ScreenUV;

// Calculate PixelNormal
float3 PixelNormal = mul(mul(Normal, Parameters.TangentToWorld), (MaterialFloat3x3)ResolvedView.TranslatedWorldToView);

// Fetch SceneDepth
float SceneDepth = CalcSceneDepth(BaseUV);

// Straighten PixelNormal to keep refraction closer to screen space
float3 DistortedNormal = PixelNormal;
DistortedNormal.z += sign(DistortedNormal.z) * NormalAlignFactor;
DistortedNormal = normalize(DistortedNormal);

// Adjust BaseUV for refraction (based on IOR, PixelNormal, and ObjectThickness)
float2 NormalUV = DistortedNormal.xy;
NormalUV.y = -NormalUV.y;
float2 RefractionOffset = (ObjectThickness + RefractionThicknessCorrection) * NormalUV * (1.0 / IOR - 1.0);

// Establish max RefractionOffset and scale by SizeOnScreenFactor
RefractionOffset *= MagicRefractionNumber * SizeOnScreenFactor;

BaseUV += RefractionOffset;

float2 NewUV = BaseUV;

float RadialStepSize = UVDistance / RadialSteps;
float2 CurOffset = { 0.0, 0.0 };

// Recalculate SceneDepth with NewUV
SceneDepth = CalcSceneDepth(NewUV);

// Screen aligned TemporalAA jitter
float2 ScenePixels = View.BufferSizeAndInvSize.xy * ScreenUV;
ScenePixels += View.TemporalAAParams.r;
float2 RandomSamp = ((uint)(ScenePixels.x) + 2 * (uint)(ScenePixels.y)) % 5;
RandomSamp += Texture2DSample(RandomTex, RandomTexSampler, View.BufferSizeAndInvSize.xy * ScreenUV / 64).xy;
RandomSamp /= 6;
RandomSamp -= 0.5;

float TempAARotation = RandomSamp.x;
float TempAADistance = RadialStepSize * RandomSamp.x;


BaseUV += View.BufferSizeAndInvSize.zw * RandomSamp;

// Fetch RandomSceneDepth (costly but could improve blur on SceneDepth sharp edges)
// float RandomSceneDepth = CalcSceneDepth(NewUV + RandomSamp * (UVDistance / RadialSteps));

// Calculate BlurAmount
float VolumeBlurAmount = 1.0 - exp(-ObjectThickness * VolumeScatter);
float SurfaceBlurAmount = 1.0 - 1.0 * exp(-(SceneDepth - PixelDepth) * SurfaceScatter); // - 0.5 * exp(-(RandomSceneDepth - PixelDepth) * SurfaceScatter);
float BlurAmount = saturate(VolumeBlurAmount + SurfaceBlurAmount);

float TwoPi = 6.283185;
float3 CurColor = { 0.0, 0.0, 0.0 };
float CurDistance = 0.0;
float Substep = 0.0;

if (BlurAmount == 0.0)
{
    return DecodeSceneColorForMaterialNode(NewUV);
}
else
{
    int i = 0;
	// If the code is applied to scene texture, need to clamp based on viewport bilinear min max
	//float2 BilinearOffset = View.BufferSizeAndInvSize.zw * 0.5f;
	//float2 BilinearMinBufferUV = ViewportUVToBufferUV(float2(0.0f, 0.0f)) + BilinearOffset;
	//float2 BilinearMaxBufferUV = ViewportUVToBufferUV(float2(1.0f, 1.0f)) - BilinearOffset;
    while (i < RadialSteps)
    {
        for (int j = 0; j < AngularSteps; j++)
        {
            float Angle = TwoPi * ((TempAARotation + Substep) / AngularSteps);
            CurOffset.x = cos(Angle);
            CurOffset.y = sin(Angle);
            CurOffset *= BlurAmount * MagicBlurNumber * View.ViewToClip[0][0] * 0.01;
            NewUV = BaseUV + (CurOffset * (CurDistance + TempAADistance));
			//NewUV = clamp(NewUV, BilinearMinBufferUV, BilinearMaxBufferUV);
            CurColor += DecodeSceneColorForMaterialNode(NewUV);
            Substep++;
        }
        CurDistance += RadialStepSize;
        Substep += AngularOffset;
        i++;
    }
    CurColor /= (float) (RadialSteps * AngularSteps);
    return CurColor;
}
