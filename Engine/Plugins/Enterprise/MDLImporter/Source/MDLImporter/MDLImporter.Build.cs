// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System;
using System.IO;
using System.Collections.Generic;
using EpicGames.Core;

namespace UnrealBuildTool.Rules
{
    public class MDLImporter : ModuleRules
    {
        private string ThirdPartyPath
        {
            get { return Path.GetFullPath(Path.Combine(EngineDirectory, "Restricted/NotForLicensees/Source/ThirdParty/Enterprise/")); }
        }

        public MDLImporter(ReadOnlyTargetRules Target) : base(Target)
        {
			bLegalToDistributeObjectCode = true;

            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
					"Analytics",
					"AssetTools",
					"Core",
					"RenderCore",
					"RHI",
                    "ImageCore",
                    "CoreUObject",
                    "Engine",
                    "MessageLog",
					"EditorFramework",
                    "UnrealEd",
                    "Slate",
                    "SlateCore",
                    "MainFrame",
                    "InputCore",
                    "MaterialEditor",
                    "Projects"
                }
            );

            PublicDependencyModuleNames.AddRange(
                new string[]
                {
                }
            );

            List<string> RuntimeModuleNames = new List<string>();
            string BinaryLibraryFolder = Path.Combine(EngineDirectory, "Plugins/Enterprise/MDLImporter/Binaries/ThirdParty/MDL", Target.Platform.ToString());

            if (Target.Platform == UnrealTargetPlatform.Win64)
            {
				RuntimeModuleNames.Add("dds.dll");
				RuntimeModuleNames.Add("libmdl_sdk.dll");
				RuntimeModuleNames.Add("mdl_distiller.dll");
				RuntimeModuleNames.Add("nv_freeimage.dll");

                foreach (string RuntimeModuleName in RuntimeModuleNames)
                {
                    string ModulePath = Path.Combine(BinaryLibraryFolder, RuntimeModuleName);
                    if (!File.Exists(ModulePath))
                    {
                        string Err = string.Format("MDL SDK module '{0}' not found.", ModulePath);
                        System.Console.WriteLine(Err);
                        throw new BuildException(Err);
                    }

                    PublicDelayLoadDLLs.Add(RuntimeModuleName);
                    RuntimeDependencies.Add(ModulePath);
                }

				// MSVC does not allow __LINE__ in template parameters when edit & continue is enabled (see C2975 @ MSDN)
				if (Target.bSupportEditAndContinue && Target.WindowsPlatform.Compiler.IsMSVC())
				{
					PrivateDefinitions.Add("MDL_MSVC_EDITCONTINUE_WORKAROUND=1");
				}
			}
			else if (Target.Platform == UnrealTargetPlatform.Mac || Target.Platform == UnrealTargetPlatform.Linux)
            {
                RuntimeModuleNames.Add("dds.so");
                RuntimeModuleNames.Add("libmdl_sdk.so");
                RuntimeModuleNames.Add("mdl_distiller.so");
                RuntimeModuleNames.Add("nv_freeimage.so");

                foreach (string RuntimeModuleName in RuntimeModuleNames)
                {
                    string ModulePath = Path.Combine(BinaryLibraryFolder, RuntimeModuleName);
                    if (!File.Exists(ModulePath))
                    {
                        string Err = string.Format("MDL SDK module '{0}' not found.", ModulePath);
                        System.Console.WriteLine(Err);
                        throw new BuildException(Err);
                    }

                    PublicDelayLoadDLLs.Add(ModulePath);
                    RuntimeDependencies.Add(ModulePath);
                }
            }

            if (Directory.Exists(ThirdPartyPath))
            {
                //third party libraries
                string[] Libs = { "mdl-sdk-349500.8766a"};
                foreach (string Lib in Libs)
                {
                    string IncludePath = Path.Combine(ThirdPartyPath, Lib, "include");
                    if (Directory.Exists(IncludePath))
                    {
                        PublicSystemIncludePaths.Add(IncludePath);
                    }
                    else
                    {
                        return;
                    }
                }

                PrivateDefinitions.Add("USE_MDLSDK");

                if (Target.Platform == UnrealTargetPlatform.Win64)
                {
                    PublicDefinitions.Add("MI_PLATFORM_WINDOWS");
                }
                else if (Target.Platform == UnrealTargetPlatform.Linux)
                {
                    PublicDefinitions.Add("MI_PLATFORM_LINUX");
                }
                else if (Target.Platform == UnrealTargetPlatform.Mac)
                {
                    PublicDefinitions.Add("MI_PLATFORM_MACOSX");
                }
            }
        }
    }
}
