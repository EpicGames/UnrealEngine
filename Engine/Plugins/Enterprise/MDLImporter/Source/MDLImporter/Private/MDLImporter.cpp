// Copyright Epic Games, Inc. All Rights Reserved.

#include "MDLImporter.h"

#ifdef USE_MDLSDK

#include "IAssetTools.h"
#include "MDLImporterOptions.h"
#include "MDLMapHandler.h"
#include "MDLMaterialFactory.h"
#include "MDLMaterialImporter.h"

#include "generator/MaterialTextureFactory.h"
#include "mdl/ApiContext.h"
#include "mdl/MaterialCollection.h"
#include "mdl/MaterialDistiller.h"
#include "mdl/Utility.h"

#include "AssetToolsModule.h"
#include "Engine/Texture2D.h"
#include "Engine/RendererSettings.h"
#include "Factories/TextureFactory.h"
#include "Materials/Material.h"
#include "Misc/Paths.h"
#include "UObject/StrongObjectPtr.h"

namespace MDLImporterImpl
{
	void SetupPostProcess(float MetersPerSceneUnit, Mdl::FMaterialCollection& Materials)
	{
		for (Mdl::FMaterial& Material : Materials)
		{
			Material.PostProcessFunction = [MetersPerSceneUnit](Mdl::FMaterial& Material)  //
			{
				if (Material.Scattering.WasValueBaked())
				{
					check(Material.Scattering.Texture.Path.IsEmpty());  // not supported for now

					// the baked value is the probability density in (per meter in world space), so convert it to a color
					for (int Index = 0; Index < 3; ++Index)
					{
						Material.Scattering.Value[Index] = FMath::Min(100.f, Material.Scattering.Value[Index]) / 100.f;
					}
				}

				if (Material.Absorption.WasValueBaked())
				{
					// the baked value is the probability density in (per meter in world space)
					const float Scale = 0.02f;
					for (int Index = 0; Index < 3; ++Index)
					{
						Material.BaseColor.Value[Index] = FMath::Exp(-Material.Absorption.Value[Index] * Scale);
					}
					const float Magnitude = FMath::Max3(Material.BaseColor.Value.X, Material.BaseColor.Value.Y, Material.BaseColor.Value.Z);
					if (Magnitude > 1.f)
					{
						Material.BaseColor.Value /= Magnitude;
					}
				}
			};
		}
	}

	bool GetParentPath(FString& Path)
	{
		const FString CPath = Path.LeftChop(1);
		int32         Pos   = CPath.FindLastCharByPredicate([](TCHAR C) { return C == TEXT('/') || C == TEXT('\\'); });
		if (Pos != INDEX_NONE)
		{
			Path = Path.Left(Pos + 1);
			return true;
		}
		return false;
	}

	void ClearMaterial(UMaterial* Material)
	{
		UMaterialEditorOnlyData* MaterialEditorOnly = Material->GetEditorOnlyData();

		MaterialEditorOnly->BaseColor.Expression          = nullptr;
		MaterialEditorOnly->EmissiveColor.Expression      = nullptr;
		MaterialEditorOnly->SubsurfaceColor.Expression    = nullptr;
		MaterialEditorOnly->Roughness.Expression          = nullptr;
		MaterialEditorOnly->Metallic.Expression           = nullptr;
		MaterialEditorOnly->Specular.Expression           = nullptr;
		MaterialEditorOnly->Opacity.Expression            = nullptr;
		MaterialEditorOnly->Refraction.Expression         = nullptr;
		MaterialEditorOnly->OpacityMask.Expression        = nullptr;
		MaterialEditorOnly->ClearCoat.Expression          = nullptr;
		MaterialEditorOnly->ClearCoatRoughness.Expression = nullptr;
		MaterialEditorOnly->Normal.Expression             = nullptr;

		Material->GetExpressionCollection().Empty();
	}
}

FMDLImporter::FMDLImporter(const FString& PluginPath)
    : TextureFactory(new Generator::FMaterialTextureFactory())
    , MaterialFactory(new FMDLMaterialFactory(*TextureFactory))
{
	// initialize MDL libraries
	const FString ThirdPartyPath = FPaths::Combine(PluginPath, TEXT("/Binaries/ThirdParty/"));
#if PLATFORM_WINDOWS
	const FString Platform = TEXT("Win64");
#elif PLATFORM_MAC
	const FString Platform = TEXT("Mac");
#elif PLATFORM_LINUX
	const FString Platform = TEXT("Linux");
#else
#error "Unsupported platform!"
#endif

	MdlContext.Reset(new Mdl::FApiContext());
	
	if (MdlContext->Load(FPaths::Combine(ThirdPartyPath, TEXT("MDL"), Platform), UMDLImporterOptions::GetMdlSystemPath()))
	{
		const FString MdlUserPath = UMDLImporterOptions::GetMdlUserPath();
		if (FPaths::DirectoryExists(MdlUserPath))
		{
			MdlContext->AddSearchPath(MdlUserPath);
			MdlContext->AddResourceSearchPath(MdlUserPath);
		}
		else
		{
			UE_LOGF(LogMDLImporter, Warning, "No MDL user path: %ls", *MdlUserPath);
		}

		DistillationMapHandler.Reset(new FMDLMapHandler(*MdlContext));
		DistillationMapHandler->SetTextureFactory(TextureFactory.Get());
	}
	else
	{
		UE_LOGF(LogMDLImporter, Error, "The MDL SDK library failed to load.");
		MdlContext.Reset();
	}
}

FMDLImporter::~FMDLImporter()
{
	if (MdlContext)
	{
		MdlContext->Unload(false);
	}
}

void FMDLImporter::SetTextureFactory(UTextureFactory* Factory)
{
	TextureFactory->SetFactory(Factory);
}

const TArray<MDLImporterLogging::FLogMessage>& FMDLImporter::GetLogMessages() const
{
	if (!IsLoaded())
	{
		LogMessages.Emplace(MDLImporterLogging::EMessageSeverity::Error, TEXT("MDL SDK wasn't loaded correctly!"));
	}
	else
	{
		LogMessages.Append(MdlContext->GetLogMessages());
		LogMessages.Append(DistillationMapHandler->GetLogMessages());
	}
	return LogMessages;
}

bool FMDLImporter::OpenFile(const FString& InFileName, const UMDLImporterOptions& InImporterOptions, Mdl::FMaterialCollection& OutMaterials)
{
	if (!IsLoaded())
	{
		return false;
	}

	bool bSuccess = false;
	// load mdl module
	{
		const bool bExists = FPaths::DirectoryExists(InImporterOptions.ModulesDir.Path);
		if (bExists)
		{
			MdlContext->AddSearchPath(InImporterOptions.ModulesDir.Path);
			MdlContext->AddSearchPath(InImporterOptions.ResourcesDir.Path);
		}
		ActiveFilename          = FPaths::ConvertRelativePathToFull(InFileName);
		const FString Extension = FPaths::GetExtension(InFileName).ToLower();

		FMDLMapHandler* MapHandler = nullptr;

		FString Path = FPaths::GetPath(ActiveFilename) + TEXT("/");
		do
		{
			MdlContext->AddSearchPath(Path);
			MdlContext->AddResourceSearchPath(Path);
		} while (MDLImporterImpl::GetParentPath(Path));

		ActiveModuleName = UE::Mdl::Util::ConvertFilePathToModuleName(*FPaths::GetBaseFilename(ActiveFilename));
		bSuccess = LoadModule(ActiveModuleName, InImporterOptions, OutMaterials);

		Path = FPaths::GetPath(ActiveFilename) + TEXT("/");
		do
		{
			MdlContext->RemoveSearchPath(Path);
			MdlContext->RemoveResourceSearchPath(Path);
		} while (MDLImporterImpl::GetParentPath(Path));

		MapHandler = InImporterOptions.bForceBaking ? nullptr : DistillationMapHandler.Get();

		MdlContext->GetDistiller()->SetMapHanlder(MapHandler);
		if (bExists)
		{
			MdlContext->RemoveSearchPath(InImporterOptions.ModulesDir.Path);
			MdlContext->RemoveSearchPath(InImporterOptions.ResourcesDir.Path);
		}
	}

	return bSuccess;
}

bool FMDLImporter::LoadModule(const FString& InModuleName, const UMDLImporterOptions& InImporterOptions, Mdl::FMaterialCollection& OutMaterials)
{
	MaterialFactory->CleanUp();
	LogMessages.Empty();

	if (!IsLoaded())
	{
		return false;
	}

	bool bSuccess = false;
	// load mdl module
	{
		FMDLMapHandler* MapHandler = nullptr;

		bSuccess = MdlContext->LoadModule(InModuleName, OutMaterials);
		MDLImporterImpl::SetupPostProcess(InImporterOptions.MetersPerSceneUnit, OutMaterials);

		MapHandler = InImporterOptions.bForceBaking ? nullptr : DistillationMapHandler.Get();

		MdlContext->GetDistiller()->SetMapHanlder(MapHandler);
	}

	MdlContext->GetDistiller()->SetBakingSettings(InImporterOptions.BakingResolution, InImporterOptions.BakingSamples);
	MdlContext->GetDistiller()->SetMetersPerSceneUnit(InImporterOptions.MetersPerSceneUnit);

	UE_LOGF(LogMDLImporter, Log, "MDL module %ls has %d materials", *InModuleName, OutMaterials.Count());
	if (OutMaterials.Count() == 0)
	{
		LogMessages.Emplace(MDLImporterLogging::EMessageSeverity::Error, TEXT("No materials are present in the MDL module!"));

		bSuccess = false;
		// clear MDL database
		MdlContext->UnloadModule(ActiveModuleName);
		MdlContext->Unload(true);
	}

	return bSuccess;
}

bool FMDLImporter::DistillMaterials(const TMap<FString, UMaterial*>& MaterialsMap, Mdl::FMaterialCollection& Materials, FProgressFunc ProgressFunc)
{
	DistillationMapHandler->SetMaterials(MaterialsMap);
	bool bSuccess = MdlContext->GetDistiller()->Distil(Materials, ProgressFunc);

	// clear MDL database
	MdlContext->UnloadModule(ActiveModuleName);
	MdlContext->Unload(true);
	return bSuccess;
}

void FMDLImporter::ConvertUnsuportedVirtualTextures() const 
{
	//This function only converts virtual textures connected to opacity mask.
	//Early out if the project supports this.
	if (GetDefault<URendererSettings>()->bEnableVirtualTextureOpacityMask)
	{
		return;
	}

	const TArray<UMaterialInterface*>& CreatedMaterials = MaterialFactory->GetCreatedMaterials();
	TArray<UTexture2D*> VirtualTexturesToConvert;
	TArray<UMaterial*> MaterialsToRefreshAfterVirtualTextureConversion;

	//First gather the textures that causes problem in materials
	for (UMaterialInterface* CurrentMaterialInterface : CreatedMaterials)
	{
		TArray<UTexture*> Textures;
		if (CurrentMaterialInterface->GetTexturesInPropertyChain(EMaterialProperty::MP_OpacityMask, Textures, nullptr, nullptr))
		{
			for (UTexture* CurrentTexture : Textures)
			{
				UTexture2D* CurrentTexture2D = Cast<UTexture2D>(CurrentTexture);
				if (CurrentTexture2D && CurrentTexture2D->VirtualTextureStreaming)
				{
					VirtualTexturesToConvert.AddUnique(CurrentTexture2D);
				}
			}
		}
	}

	//Second identify the materials that will need to be updated.
	//We need to loop a second time because a virtual texture can be supported in a material but not into another, in that case, both materials needs to be updated.
	for (UMaterialInterface* CurrentMaterialInterface : CreatedMaterials)
	{
		if (UMaterial* CurrentMaterial = Cast<UMaterial>(CurrentMaterialInterface))
		{
			for (UTexture2D* VirtualTexture : VirtualTexturesToConvert)
			{
				if (CurrentMaterial->GetReferencedTextures().Contains(VirtualTexture))
				{
					MaterialsToRefreshAfterVirtualTextureConversion.Add(CurrentMaterial);
					break;
				}
			}
		}
	}

	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	AssetTools.ConvertVirtualTextures(VirtualTexturesToConvert, true, &MaterialsToRefreshAfterVirtualTextureConversion);
}

bool FMDLImporter::ImportMaterials(UObject* ParentPackage, EObjectFlags Flags, Mdl::FMaterialCollection& Materials,
                                   FProgressFunc ProgressFunc /*= nullptr*/)
{
	MaterialFactory->CleanUp();
	if (!IsLoaded())
	{
		return false;
	}

	if ( ParentPackage == GetTransientPackage() )
	{
		DistillationMapHandler->SetFunctionAssetPath(*ParentPackage->GetPathName());
	}
	DistillationMapHandler->SetObjectFlags(Flags);

	TStrongObjectPtr<UTextureFactory> Factory(NewObject<UTextureFactory>());
	SetTextureFactory(Factory.Get());

	if (!MaterialFactory->CreateMaterials(ActiveFilename, ParentPackage, Flags, Materials))
	{
		return false;
	}

	if (!DistillMaterials(MaterialFactory->GetNameMaterialMap(), Materials, ProgressFunc))
	{
		return false;
	}

	if (ProgressFunc)
	{
		ProgressFunc(TEXT("Creating materials"), -1);
	}

	MaterialFactory->PostImport(Materials);
	DistillationMapHandler->Cleanup();

	ConvertUnsuportedVirtualTextures();

	return true;
}

bool FMDLImporter::Reimport(const FString& InFileName, const UMDLImporterOptions& InImporterOptions, UMaterialInterface* OutMaterial)
{
	if (!IsLoaded())
	{
		return false;
	}

	Mdl::FMaterialCollection Materials;
	if (!OpenFile(InFileName, InImporterOptions, Materials))
	{
		return false;
	}

	Mdl::FMaterial* FoundMdlMaterial = nullptr;
	for (Mdl::FMaterial& MdlMaterial : Materials)
	{
		if (MdlMaterial.Name != OutMaterial->GetName())
		{
			// disable other materials for processing
			MdlMaterial.Disable();
		}
		else
		{
			check(FoundMdlMaterial == nullptr);
			FoundMdlMaterial = &MdlMaterial;
		}
	}

	if (!FoundMdlMaterial)
	{
		return false;
	}

	TStrongObjectPtr<UTextureFactory> Factory(NewObject<UTextureFactory>());
	SetTextureFactory(Factory.Get());

	Mdl::FMaterial& MdlMaterial = *FoundMdlMaterial;
	UMaterial*      Material    = Cast<UMaterial>(OutMaterial);
	MDLImporterImpl::ClearMaterial(Material);

	const FString                   DbName       = Mdl::Util::GetMaterialDatabaseName(Materials.Name, MdlMaterial.BaseName, true);
	const TMap<FString, UMaterial*> MaterialsMap = {{DbName, Material}};

	MdlContext->GetDistiller()->SetBakingSettings(InImporterOptions.BakingResolution, InImporterOptions.BakingSamples);
	MdlContext->GetDistiller()->SetMetersPerSceneUnit(InImporterOptions.MetersPerSceneUnit);
	if (!DistillMaterials(MaterialsMap, Materials, nullptr))
	{
		return false;
	}

	MaterialFactory->Reimport(MdlMaterial, *Material);

	return true;
}

const TArray<UMaterialInterface*>& FMDLImporter::GetCreatedMaterials() const
{
	return MaterialFactory->GetCreatedMaterials();
}

void FMDLImporter::CleanUp()
{
	MaterialFactory->CleanUp();
}

void FMDLImporter::AddSearchPath(const FString& SearchPath)
{
	if ( !IsLoaded() )
	{
		return;
	}

	MdlContext->AddSearchPath(SearchPath);
}

void FMDLImporter::RemoveSearchPath(const FString& SearchPath)
{
	if ( !IsLoaded() )
	{
		return;
	}

	MdlContext->RemoveSearchPath(SearchPath);
}

#else

FMDLImporter::FMDLImporter(const FString&) {}

FMDLImporter::~FMDLImporter() {}

bool FMDLImporter::IsLoaded() const
{
	return false;
}

const TArray<UMaterialInterface*>& FMDLImporter::GetCreatedMaterials() const
{
	static const TArray<UMaterialInterface*> None;
	return None;
}

const TArray<MDLImporterLogging::FLogMessage>& FMDLImporter::GetLogMessages() const
{
	return LogMessages;
}

void FMDLImporter::SetTextureFactory(UTextureFactory*) {}

bool FMDLImporter::OpenFile(const FString&, const UMDLImporterOptions&, Mdl::FMaterialCollection&)
{
	return false;
}

bool FMDLImporter::LoadModule(const FString& InModuleName, const UMDLImporterOptions& InImporterOptions, Mdl::FMaterialCollection& OutMaterials)
{
	return false;
}

bool FMDLImporter::ImportMaterials(UObject*, EObjectFlags, Mdl::FMaterialCollection&, FProgressFunc)
{
	return false;
}

bool FMDLImporter::Reimport(const FString&, const UMDLImporterOptions&, UMaterialInterface*)
{
	return false;
}

void FMDLImporter::CleanUp()
{

}

void FMDLImporter::AddSearchPath(const FString& SearchPath)
{
}

void FMDLImporter::RemoveSearchPath(const FString& SearchPath)
{
}
#endif
