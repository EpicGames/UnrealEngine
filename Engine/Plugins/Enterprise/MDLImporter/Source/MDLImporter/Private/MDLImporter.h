// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "common/Logging.h"


class UMaterialInterface;
class UMDLImporterOptions;
class UTextureFactory;
class FMDLMaterialFactory;
class FMDLMapHandler;
class UMaterial;
namespace Mdl
{
	class FApiContext;
	class FMaterialCollection;
}
namespace Generator
{
	class FMaterialTextureFactory;
}

class MDLIMPORTER_API FMDLImporter
{
public:
	using FProgressFunc = TFunction<void(const FString& MsgName, int MaterialIndex)>;

	FMDLImporter(const FString& PluginPath);
	~FMDLImporter();

	bool IsLoaded() const;
	// Returns newly created materials.
	const TArray<UMaterialInterface*>& GetCreatedMaterials() const;
	// Returns any logged messages and clears them afterwards.
	const TArray<MDLImporterLogging::FLogMessage>& GetLogMessages() const;

	// Imports materials from an MDL file.
	bool OpenFile(const FString& InFileName, const UMDLImporterOptions& InImporterOptions, Mdl::FMaterialCollection& OutMaterials);
	bool LoadModule(const FString& InModuleName, const UMDLImporterOptions& InImporterOptions, Mdl::FMaterialCollection& OutMaterials);

	// Finalize import process and create the materials.
	bool ImportMaterials(UObject* ParentPackage, EObjectFlags Flags, Mdl::FMaterialCollection& Materials, FProgressFunc ProgressFunc = nullptr);

	// Re-imports a material from the given filename.
	bool Reimport(const FString& InFileName, const UMDLImporterOptions& InImporterOptions, UMaterialInterface* OutMaterial);

	void CleanUp();

	void AddSearchPath(const FString& SearchPath);
	void RemoveSearchPath(const FString& SearchPath);

private:
	void SetTextureFactory(UTextureFactory* Factory);
	bool DistillMaterials(const TMap<FString, UMaterial*>& MaterialsMap, Mdl::FMaterialCollection& Materials, FProgressFunc ProgressFunc);

	void ConvertUnsuportedVirtualTextures() const;

private:
#ifdef USE_MDLSDK
	TUniquePtr<Mdl::FApiContext>                   MdlContext;
	TUniquePtr<Generator::FMaterialTextureFactory> TextureFactory;
	TUniquePtr<FMDLMaterialFactory>                MaterialFactory;
	TUniquePtr<FMDLMapHandler>                     DistillationMapHandler;
	FString                                        ActiveFilename;
	FString                                        ActiveModuleName;

#endif
	mutable TArray<MDLImporterLogging::FLogMessage> LogMessages;
};

#ifdef USE_MDLSDK
inline bool FMDLImporter::IsLoaded() const
{
	return MdlContext.IsValid();
}

#endif
