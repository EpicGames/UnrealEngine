// Copyright Epic Games, Inc. All Rights Reserved.

#include "MDLImporterFactory.h"

#include "IAnalyticsProviderET.h"
#include "MDLImporterModule.h"
#include "MDLImporterOptions.h"
#include "Misc/App.h"
#include "UI/MDLOptionsWindow.h"
#include "MdlFileImporter.h"


#include "Editor.h"
#include "EditorFramework/AssetImportData.h"
#include "EngineAnalytics.h"
#include "Framework/Application/SlateApplication.h"
#include "IMessageLogListing.h"
#include "Interfaces/IMainFrameModule.h"
#include "Materials/Material.h"
#include "MessageLogModule.h"
#include "Misc/FeedbackContext.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "Subsystems/ImportSubsystem.h"
#include "UObject/StrongObjectPtr.h"
#include "Widgets/SWindow.h"

#define LOCTEXT_NAMESPACE "MDL Importer"

namespace MDLImporterImpl
{
	const TArray<FString> ListOfExtensions     = {TEXT("mdl")};
	const TArray<FString> ListOfExtensionsInfo = {TEXT("MDL material files")};

	int GetMaterialCount(const FString& Filepath)
	{
		FString Buffer;
		FFileHelper::LoadFileToString(Buffer, *Filepath);

		int                  Count = 0;
		int                  Found;
		static const FString EportStr = TEXT("export material");
		while ((Found = Buffer.Find(EportStr)) != INDEX_NONE)
		{
			Count++;
			Buffer = Buffer.Mid(Found + EportStr.Len());
		}
		return Count;
	}

	bool ShowOptionsWindow(const FString& Filepath, const FString& PackagePath, UMDLImporterOptions& ImporterOptions)
	{
		TSharedPtr<SWindow> ParentWindow;
		if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
		{
			IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
			ParentWindow                = MainFrame.GetParentWindow();
		}

		TSharedRef<SWindow> Window =
		    SNew(SWindow).Title(LOCTEXT("MDLImportOptionsTitle", "MDL Import Options")).SizingRule(ESizingRule::Autosized);

		TSharedPtr<SMDLOptionsWindow> OptionsWindow;
		Window->SetContent(
		    SAssignNew(OptionsWindow, SMDLOptionsWindow)
		        .ImportOptions(&ImporterOptions)
		        .WidgetWindow(Window)
		        .FileNameText(FText::Format(LOCTEXT("MDLImportOptionsFileName", "  Import File  :    {0}"),
		                                    FText::FromString(FPaths::GetCleanFilename(Filepath))))
		        .FilePathText(FText::FromString(Filepath))
		        .PackagePathText(FText::Format(LOCTEXT("MDLImportOptionsPackagePath", "  Import To   :    {0}"), FText::FromString(PackagePath)))
		        .MaterialCount(GetMaterialCount(Filepath)));
		FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);
		return OptionsWindow->ShouldImport();
	}

	void ShowLogMessages(const TArray<MDLImporterLogging::FLogMessage>& Errors)
	{
		if (Errors.Num() > 0)
		{
			FMessageLogModule&             MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
			TSharedRef<IMessageLogListing> LogListing       = (MessageLogModule.GetLogListing("LoadErrors"));
			LogListing->ClearMessages();
			for (const MDLImporterLogging::FLogMessage& Error : Errors)
			{
				EMessageSeverity::Type Severity =
				    Error.Get<0>() == MDLImporterLogging::EMessageSeverity::Error ? EMessageSeverity::Error : EMessageSeverity::Warning;
				LogListing->AddMessage(FTokenizedMessage::Create(Severity, FText::FromString(Error.Get<1>())));
			}
			MessageLogModule.OpenMessageLog("LoadErrors");
		}
	}
}

UMDLImporterFactory::UMDLImporterFactory(const FObjectInitializer& ObjectInitializer)
    : Super(ObjectInitializer)
    , MDLImporterModule(&IMDLImporterModule::Get())
{
	using namespace MDLImporterImpl;

#ifdef USE_MDLSDK
	bCreateNew = bText = false;
	bEditorImport      = true;
	SupportedClass     = UMaterial::StaticClass();

	ImportPriority += 100;

	Formats.Reset();
	for (int Index = 0; Index < ListOfExtensions.Num(); ++Index)
	{
		FString Extension = ListOfExtensions[Index] + TEXT(";") + ListOfExtensionsInfo[Index];
		Formats.Add(Extension);
	}
#else
	UE_LOGF(LogMDLImporter, Log, "MDL SDK was not available when plugin was build - MDL plugin won't be functional!");
#endif
}

bool UMDLImporterFactory::FactoryCanImport(const FString& Filename)
{
	const FString Extension = FPaths::GetExtension(Filename.ToLower());
	return MDLImporterModule->IsLoaded() && MDLImporterImpl::ListOfExtensions.Contains(Extension);
}

UObject* UMDLImporterFactory::FactoryCreateFile(UClass*           InClass,
                                                UObject*          InParent,
                                                FName             InName,
                                                EObjectFlags      InFlags,
                                                const FString&    InFilename,
                                                const TCHAR*      InParms,
                                                FFeedbackContext* Warn,
                                                bool&             bOutOperationCanceled)
{
	check(IMDLImporterModule::IsAvailable());

	AdditionalImportedObjects.Empty();
	UObject* Object = nullptr;
#ifdef USE_MDLSDK
	TStrongObjectPtr<UMDLImporterOptions> ImporterOptions(NewObject<UMDLImporterOptions>(GetTransientPackage(), TEXT("MDL Importer Options")));

	const FString Filepath    = FPaths::ConvertRelativePathToFull(InFilename);
	const FString PackagePath = InParent->GetPathName();

	
	if (!(IsRunningCommandlet() || GIsRunningUnattendedScript || FApp::IsUnattended() || IsAutomatedImport()))
	{
		if (!MDLImporterImpl::ShowOptionsWindow(Filepath, PackagePath, *ImporterOptions))
		{
			bOutOperationCanceled = true;
			return nullptr;
		}
	}

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, InClass, InParent, InName, TEXT("MDL"));

	TUniquePtr<IMdlFileImporter> Importer(IMdlFileImporter::Create());

	const uint64 StartTime = FPlatformTime::Cycles64();
	const bool   bImported = LoadMaterials(Importer.Get(), InParent, InFlags, Filepath, *ImporterOptions, *Warn);
	if (bImported)
	{
		const TMap<FString, UMaterialInterface*> CreatedMaterials = Importer->GetCreatedMaterials();

		if (CreatedMaterials.Num() == 1)
		{
			TArray<FString> Names;
			CreatedMaterials.GetKeys(Names);
			Object = *CreatedMaterials.Find(Names[0]);
		}
		else if (CreatedMaterials.Num() != 0)
		{
			TArray<FString> Names;
			CreatedMaterials.GetKeys(Names);
			Object = (*CreatedMaterials.Find(Names[0]))->GetOutermost();

			for (const FString& Name : Names)
			{
				AdditionalImportedObjects.Add(*CreatedMaterials.Find(Name));
			}
		}
	}
	Importer.Reset();

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, Object);

	double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);
	SendAnalytics(FMath::RoundToInt(ElapsedSeconds), bImported, InFilename);

	// Log time spent to import incoming file in minutes and seconds
	const int ElapsedMin = int(ElapsedSeconds / 60.f);
	ElapsedSeconds -= 60.0 * (double)ElapsedMin;

	UE_LOGF(LogMDLImporter, Log, "Imported %ls in [%d min %.3f s]", *Filepath, ElapsedMin, ElapsedSeconds);
#else
	TArray<MDLImporterLogging::FLogMessage> Error;
	Error.Emplace(MDLImporterLogging::EMessageSeverity::Error, TEXT("MDL Plugin was not compiled with MDL SDK!"));
	MDLImporterImpl::ShowLogMessages(Error);
	bOutOperationCanceled = true;
#endif

	return Object;
}

void UMDLImporterFactory::SendAnalytics(int32 ImportDurationInSeconds, bool bImportSuccess, const FString& InFilename) const
{
	if (FEngineAnalytics::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> EventAttributes;

		FString ImporterName = TEXT("MDL");

		EventAttributes.Emplace(TEXT("ImporterType"), ImporterName);
		EventAttributes.Emplace(TEXT("ImportDuration"), ImportDurationInSeconds);

		if (bImportSuccess)
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("MDL.Import"), EventAttributes);
		}
		else
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("MDL.ImportFail"), EventAttributes);
		}
	}
}

void UMDLImporterFactory::CleanUp()
{
	Super::CleanUp();
	FReimportManager::Instance()->UnregisterHandler(*this);
}

bool UMDLImporterFactory::LoadMaterials(IMdlFileImporter* Importer,
    UObject* InPackage, EObjectFlags InFlags, const FString& InFilename, const UMDLImporterOptions& InImporterOptions, FFeedbackContext& OutWarn)
{
	FScopedSlowTask Progress(100.0f, LOCTEXT("ImportingFiles", "Importing File ..."), true, OutWarn);
	Progress.MakeDialog();

	const float StartProgress = 25.f;
	Progress.EnterProgressFrame(StartProgress, FText::Format(LOCTEXT("ImportingMaterials", "Importing materials: {0} ..."), FText::FromString(InFilename)));

	bool bSuccess = Importer->OpenFile(InFilename, InImporterOptions);
	if (bSuccess)
	{
		// TODO: show materials selection

		const int MaterialCount = Importer->GetMaterialCountInFile();
		auto      ProgressFunc  = [StartProgress, &Progress, &InFilename, MaterialCount](const FString& MaterialName, int MaterialIndex)  //
		{
			if (MaterialIndex >= 0)
			{
				const float FinishProgress   = 90.f;
				const float MaterialProgress = (FinishProgress - StartProgress) / MaterialCount;
				Progress.EnterProgressFrame(MaterialProgress,
				                            FText::Format(LOCTEXT("MaterialDistillation", "Material distillation: {0} ..."), FText::FromString(MaterialName)));
			}
			else
			{
				Progress.EnterProgressFrame(5.f, FText::Format(LOCTEXT("FinishingMaterials", "Finishing materials: {0} ..."), FText::FromString(InFilename)));
			}
		};

		// processes and distills any MDL materials
		bSuccess &= Importer->ImportMaterials(InPackage, InFlags, ProgressFunc);
		if (!bSuccess)
		{
			OutWarn.Logf(ELogVerbosity::Error, TEXT("Failed to import materials for: '%s'"), *InFilename);
		}
	}
	else
	{
		OutWarn.Logf(ELogVerbosity::Error, TEXT("Failed to load: '%s': failed to find or load the module"), *InFilename);
	}

	const TArray<MDLImporterLogging::FLogMessage> Messages = Importer->GetLogMessages();
	MDLImporterImpl::ShowLogMessages(Messages);

	return bSuccess;
}

bool UMDLImporterFactory::CanReimport(UObject* Obj, TArray<FString>& OutFilenames)
{
	UMaterialInterface* Material = Cast<UMaterialInterface>(Obj);
	if (Material != nullptr)
	{
		if (Material->AssetImportData)
		{
			Material->AssetImportData->ExtractFilenames(OutFilenames);

			const FString FileName(Material->AssetImportData->GetFirstFilename());
			const FString Extension = FPaths::GetExtension(FileName);
			if (MDLImporterImpl::ListOfExtensions.FindByPredicate([&Extension](const FString& Current) { return Current == Extension; }) != nullptr)
				return true;
		}
	}

	return false;
}

EReimportResult::Type UMDLImporterFactory::Reimport(UObject* Obj)
{
	if (!Obj || !Obj->IsA(UMaterialInterface::StaticClass()))
	{
		return EReimportResult::Failed;
	}

	TArray<MDLImporterLogging::FLogMessage> Errors;
#ifdef USE_MDLSDK
	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetReimport(Obj);

	UMaterialInterface* Material    = Cast<UMaterialInterface>(Obj);

	const FString FileName(Material->AssetImportData->GetFirstFilename());
	const FString Extension = FPaths::GetExtension(FileName);
	check(MDLImporterImpl::ListOfExtensions.FindByPredicate([&Extension](const FString& Current) { return Current == Extension; }) != nullptr);
	if (!FileName.Len())
	{
		// If there is no file path provided, can't reimport from source.
		Errors.Emplace(MDLImporterLogging::EMessageSeverity::Error,
		               FString::Printf(TEXT("Attempt to reimport material with empty file name: %s"), *Material->GetName()));
	}
	if (!FPaths::FileExists(FileName))
	{
		Errors.Emplace(MDLImporterLogging::EMessageSeverity::Error,
		               FString::Printf(TEXT("Attempt to reimport material with non existing file name: %s"), *FileName));
	}
	TUniquePtr<IMdlFileImporter> Importer(IMdlFileImporter::Create());
	if (Errors.Num() == 0)
	{
		TStrongObjectPtr<UMDLImporterOptions> ImporterOptions(NewObject<UMDLImporterOptions>(GetTransientPackage(), TEXT("MDL Importer Options")));
		if (!Importer->Reimport(FileName, *ImporterOptions, Material))
		{
			Errors.Emplace(MDLImporterLogging::EMessageSeverity::Error, FString::Printf(TEXT("Failed to reimport material: %s"), *Material->GetName()));
		}
	}

	TArray<MDLImporterLogging::FLogMessage> Messages = Importer->GetLogMessages();
	Messages.Append(Errors);
	MDLImporterImpl::ShowLogMessages(Messages);

#else
	Errors.Emplace(MDLImporterLogging::EMessageSeverity::Error, TEXT("MDL Plugin was not compiled with MDL SDK!"));
	MDLImporterImpl::ShowLogMessages(Errors);
#endif

	return Errors.Num() == 0 ? EReimportResult::Succeeded : EReimportResult::Failed;
}

void UMDLImporterFactory::SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths)
{
	if (NewReimportPaths.Num() == 0)
		return;

	UMaterialInterface* Material = Cast<UMaterialInterface>(Obj);
	if (Material != nullptr)
	{
		if (FactoryCanImport(NewReimportPaths[0]))
		{
			Material->AssetImportData->UpdateFilenameOnly(NewReimportPaths[0]);
		}
	}
}

#undef LOCTEXT_NAMESPACE
