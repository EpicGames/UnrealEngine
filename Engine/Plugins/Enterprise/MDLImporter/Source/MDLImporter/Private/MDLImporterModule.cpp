// Copyright Epic Games, Inc. All Rights Reserved.

#include "MDLImporterModule.h"

#include "AssetRegistry/AssetData.h"
#include "MDLImporter.h"
#include "MdlFileImporter.h"

#include "ContentBrowserModule.h"
#include "EditorFramework/AssetImportData.h"
#include "EditorReimportHandler.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Misc/Paths.h"
#include "Interfaces/IPluginManager.h"
#include "ShaderCore.h"

#define LOCTEXT_NAMESPACE "MDL Importer"

const TCHAR* IMDLImporterModule::ModuleName = TEXT("MDLImporter");

class FMDLImporterModule : public IMDLImporterModule
{
public:
	virtual FMDLImporter& GetMDLImporter() override;

	virtual TUniquePtr<IMdlFileImporter> CreateFileImporter() override
	{
		return IMdlFileImporter::Create();
	}

	bool IsLoaded() override
	{
		return GetMDLImporter().IsLoaded();
	}


private:
	void AddContentBrowserContextMenuExtender();
	void RemoveContentBrowserContextMenuExtender();

	static TSharedRef<FExtender> OnExtendContentBrowserAssetSelectionMenu(const TArray<FAssetData>& SelectedAssets);
	static void                  ExecuteReimportMaterials(TArray<FAssetData> AssetData);

private:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	static const FString PluginPath;

	TUniquePtr<FMDLImporter> MDLImporter;
	FDelegateHandle          ContentBrowserExtenderDelegateHandle;
};

const FString FMDLImporterModule::PluginPath = FPaths::Combine(FPaths::EnginePluginsDir(), TEXT("Enterprise/MDLImporter"));

FMDLImporter& FMDLImporterModule::GetMDLImporter()
{
	// lazy init the importer
	if (!MDLImporter)
		MDLImporter.Reset(new FMDLImporter(PluginPath));
	return *MDLImporter;
}

void FMDLImporterModule::StartupModule()
{
    // Maps virtual shader source directory /Plugin/MDLImporter to the plugin's actual Shaders directory.
	FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("MDLImporter"))->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/Plugin/MDLImporter"), PluginShaderDir);

	if (!IsRunningCommandlet())
	{
		AddContentBrowserContextMenuExtender();
	}
}

void FMDLImporterModule::ShutdownModule()
{
	if (!IsRunningCommandlet())
	{
		RemoveContentBrowserContextMenuExtender();
	}
	MDLImporter.Reset();
}

void FMDLImporterModule::AddContentBrowserContextMenuExtender()
{
#ifdef USE_MDLSDK
	if (!IsRunningCommandlet())
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		TArray<FContentBrowserMenuExtender_SelectedAssets>& CBMenuExtenderDelegates = ContentBrowserModule.GetAllAssetViewContextMenuExtenders();

		CBMenuExtenderDelegates.Add(
		    FContentBrowserMenuExtender_SelectedAssets::CreateStatic(&FMDLImporterModule::OnExtendContentBrowserAssetSelectionMenu));
		ContentBrowserExtenderDelegateHandle = CBMenuExtenderDelegates.Last().GetHandle();
	}
#endif
}

void FMDLImporterModule::RemoveContentBrowserContextMenuExtender()
{
	if (ContentBrowserExtenderDelegateHandle.IsValid() && FModuleManager::Get().IsModuleLoaded("ContentBrowser"))
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::GetModuleChecked<FContentBrowserModule>("ContentBrowser");
		TArray<FContentBrowserMenuExtender_SelectedAssets>& CBMenuExtenderDelegates = ContentBrowserModule.GetAllAssetViewContextMenuExtenders();
		CBMenuExtenderDelegates.RemoveAll([this](const FContentBrowserMenuExtender_SelectedAssets& Delegate) {
			return Delegate.GetHandle() == ContentBrowserExtenderDelegateHandle;
		});
	}
}

TSharedRef<FExtender> FMDLImporterModule::OnExtendContentBrowserAssetSelectionMenu(const TArray<FAssetData>& SelectedAssets)
{
	TSharedRef<FExtender> Extender = MakeShared<FExtender>();

	// Run through the assets to determine if any meet our criteria
	bool bShouldExtendAssetActions = false;
	for (const FAssetData& Asset : SelectedAssets)
	{
		if (Asset.AssetClassPath == UMaterial::StaticClass()->GetClassPathName() || Asset.AssetClassPath == UMaterialInstance::StaticClass()->GetClassPathName())
		{
			// Need to load the material at this point to figure out the type of the AssetImportData
			UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(Asset.GetAsset());

			if (MaterialInterface && MaterialInterface->AssetImportData)
			{
				const FString FileName(MaterialInterface->AssetImportData->GetFirstFilename());
				const FString Extension = FPaths::GetExtension(FileName);
				if (Extension == TEXT("mdl"))
				{
					bShouldExtendAssetActions = true;
					break;
				}
			}
		}
	}

	if (bShouldExtendAssetActions)
	{
		// Add the MDL material actions sub-menu extender
		Extender->AddMenuExtension(
		    "GetAssetActions",
		    EExtensionHook::After,
		    nullptr,
		    FMenuExtensionDelegate::CreateLambda(
		        [SelectedAssets](FMenuBuilder& MenuBuilder)  //
		        {
			        // Add the reimport MDL material sub-menu extender
			        MenuBuilder.AddMenuEntry(
			            NSLOCTEXT("AssetTypeActions_Material", "ObjectContext_ReimportMDLMaterial", "Reimport Material"),
			            NSLOCTEXT("AssetTypeActions_Material", "ObjectContext_ReimportMDLMaterialTooltip", "Reimport the selected material(s)."),
			            FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.AssetActions.ReimportAsset"),
			            FUIAction(FExecuteAction::CreateStatic(&FMDLImporterModule::ExecuteReimportMaterials, SelectedAssets), FCanExecuteAction()));
		        }));
	}

	return Extender;
}

void FMDLImporterModule::ExecuteReimportMaterials(TArray<FAssetData> SelectedAssets)
{
	for (const FAssetData& AssetData : SelectedAssets)
	{
		if (AssetData.AssetClassPath == UMaterial::StaticClass()->GetClassPathName() || AssetData.AssetClassPath == UMaterialInstance::StaticClass()->GetClassPathName())
		{
			UObject* AssetToReimport = AssetData.GetAsset();

			FReimportManager::Instance()->Reimport(AssetToReimport, /*bAskForNewFileIfMissing=*/false);
		}
	}
}

IMPLEMENT_MODULE(FMDLImporterModule, MDLImporter);

#undef LOCTEXT_NAMESPACE
