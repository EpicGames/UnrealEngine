// Copyright Epic Games, Inc. All Rights Reserved.

#include "MDLImporterOptions.h"

#if PLATFORM_WINDOWS
#include "Misc/Paths.h"
#endif

UMDLImporterOptions::UMDLImporterOptions(const FObjectInitializer& ObjectInitializer)
    : Super(ObjectInitializer)
{
	BakingResolution   = 1024;
	BakingSamples      = 2;
	MetersPerSceneUnit = 0.16f;
	bForceBaking        = false;
	ModulesDir.Path    = GetMdlUserPath();
	ResourcesDir.Path  = GetMdlUserPath();
}

FString UMDLImporterOptions::GetMdlSystemPath()
{
	FString Path = FPlatformMisc::GetEnvironmentVariable(TEXT("MDL_SYSTEM_PATH"));

	if (Path.IsEmpty())
	{
#if PLATFORM_WINDOWS
		Path = TEXT("C:/ProgramData/NVIDIA Corporation/mdl/");
#elif PLATFORM_MAC
		Path = TEXT("/Library/Application/NVIDIA Corporation/mdl/");
#elif PLATFORM_LINUX
		Path = TEXT("/opt/nvidia/mdl/");
#else
#error "Unsupported platform!"
#endif
	}
	return Path;
}

FString UMDLImporterOptions::GetMdlUserPath()
{
	FString Path = FPlatformMisc::GetEnvironmentVariable(TEXT("MDL_USER_PATH"));

	if (Path.IsEmpty())
	{
#if PLATFORM_WINDOWS
		Path = FPaths::Combine(FPlatformProcess::UserDir(), TEXT("mdl/"));
#elif PLATFORM_MAC || PLATFORM_LINUX
		Path = TEXT("~/Documents/mdl/");
#else
#error "Unsupported platform!"
#endif
	}
	return Path;
}
