// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef USE_MDLSDK

#include "MDLMapHandler.h"

#include "Materials/MaterialExpressionSquareRoot.h"
#include "generator/FunctionLoader.h"
#include "generator/MaterialExpressions.h"
#include "generator/MaterialTextureFactory.h"
#include "mdl/BakeParam.h"
#include "mdl/Utility.h"

#include "Materials/MaterialExpressionMaterialFunctionCall.h"

MDLSDK_INCLUDES_START
#include "mi/neuraylib/imaterial_definition.h"
#include <mi/base/handle.h>
#include <mi/base/iinterface.h>
#include <mi/base/types.h>
#include <mi/neuraylib/iexpression.h>
MDLSDK_INCLUDES_END

FMDLMapHandler::FMDLMapHandler(const Mdl::FApiContext& MdlContext)
    : MaterialExpressionFactory(MdlContext)
    , FunctionLoader(new Generator::FFunctionLoader())
    , CurrentMaterial(nullptr)
    , CurrentNormalExpression(nullptr)
    , FirstNormalExpression(nullptr)
{
	FunctionLoader->SetAssetPath(TEXT("/Game/MDL/Functions"));
	MaterialExpressionFactory.SetFunctionLoader(FunctionLoader.Get());
}

FMDLMapHandler::~FMDLMapHandler()
{
}

void FMDLMapHandler::PreImport(const mi::neuraylib::IMaterial_definition& MDLMaterialDefinition,
                               const mi::neuraylib::ICompiled_material&   MDLMaterial,
                               mi::neuraylib::ITransaction&               MDLTransaction)
{
	const FString MaterialName = ANSI_TO_TCHAR(MDLMaterialDefinition.get_mdl_name());
	const FString ModuleName   = Mdl::Util::GetModuleName(MaterialName);

	UMaterial* Material = Materials[MaterialName];
	check(Material);
	CurrentMaterial = Material;
	TextureFactory->SetAssetPrefix(ModuleName);
	MaterialExpressionFactory.Cleanup();
	MaterialExpressionFactory.SetCurrentMaterial(MDLMaterialDefinition, MDLMaterial, MDLTransaction, *Material);
	MaterialExpressionFactory.CreateParameterExpressions();

	FirstNormalExpression = CurrentNormalExpression = nullptr;
}

void FMDLMapHandler::SetupNormalExpression(const FString& MapName)
{
	bool bIsGeometryNormal = false;
	if (MapName.Find(TEXT("normal")) != INDEX_NONE)
	{
		// for the geometry part, use the geometry_normal, instead of the normal, to prevent issues with shader domains
		if (CurrentNormalExpression == nullptr)
		{
			CurrentNormalExpression =
			    Generator::NewMaterialExpressionFunctionCall(CurrentMaterial, FunctionLoader->Load(TEXT("mdl_state_geometry_normal")), {});
		}

		bIsGeometryNormal = true;
	}
	else
	{
		if ((CurrentNormalExpression != FirstNormalExpression) && FirstNormalExpression)
		{
			// from now on, the normal expression is what we've got out of the Geometry field
			CurrentNormalExpression = FirstNormalExpression;
		}
		else
		{
			// otherwise, use the standard normal from now on
			CurrentNormalExpression =
			    Generator::NewMaterialExpressionFunctionCall(CurrentMaterial, FunctionLoader->Load(TEXT("mdl_state_normal")), {});
		}
	}
	MaterialExpressionFactory.SetCurrentNormal(CurrentNormalExpression, bIsGeometryNormal);
}

bool FMDLMapHandler::Import(const FString& MapName, bool bIsTexture, Mdl::FBakeParam& MapBakeParam)
{
	if (!MapBakeParam.InputExpression)
	{
		return false;
	}

	const auto Handle = mi::base::make_handle(MapBakeParam.InputExpression);
	Handle->retain();

	const mi::neuraylib::IExpression::Kind Kind = Handle->get_kind();
	if (Kind == mi::neuraylib::IExpression::EK_CONSTANT)
	{
		// ignore constants and just bake them
		return false;
	}

	const FString MapNameLower = MapName.ToLower();
	SetupNormalExpression(MapNameLower);

	const Generator::FMaterialExpressionConnectionList Outputs = MaterialExpressionFactory.CreateExpression(Handle, MapBakeParam.InputBakePath);
	if (Outputs.Num() == 0 && Kind == mi::neuraylib::IExpression::EK_DIRECT_CALL)
	{
		return false;
	}

	if (Outputs.Num() == 0)
	{
		// Map expression has no output - thus invalid to use. This happens when, for example, referenced texture file is not found
		UE_LOGF(LogMDLImporter, Warning, "Failed to create valid expression for: %ls from: %ls", *MapName,
			*MapBakeParam.InputBakePath);
		return false;
	}


	if (!FirstNormalExpression && MapNameLower.Find(TEXT("normal")) != INDEX_NONE)
	{
		// setup the first normal map expression
		FirstNormalExpression = Outputs[0].GetExpressionAndMaybeUse();
	}

	if (MapNameLower.Find(TEXT("roughness")) != INDEX_NONE)
	{
		// convert from GGX to Unreal
		UMaterialExpression* Roughness =
		    Generator::NewMaterialExpressionSquareRoot(CurrentMaterial, {Outputs[0].GetExpressionAndUse(), Outputs[0].GetExpressionOutputIndex()});
		MapBakeParam.SetExpression(Roughness, bIsTexture, 0);
	}
	else
	{
		MapBakeParam.SetExpression(Outputs[0].GetExpressionAndUse(), bIsTexture, Outputs[0].GetExpressionOutputIndex());
	}

	if (Outputs[0].HasExpression())
	{
		UE_LOGF(LogMDLImporter, Log, "Created expression %ls for: %ls from: %ls", *Outputs[0].GetExpressionName(), *MapName,
		       *MapBakeParam.InputBakePath);
	}
	else
	{
		UE_LOGF(LogMDLImporter, Log, "No expression for: %ls from: %ls", *MapName, *MapBakeParam.InputBakePath);
	}

	return true;
}

void FMDLMapHandler::PostImport()
{
	MaterialExpressionFactory.CleanupMaterialExpressions();
}

void FMDLMapHandler::SetTextureFactory(Generator::FMaterialTextureFactory* Factory)
{
	MaterialExpressionFactory.SetTextureFactory(Factory);
	TextureFactory = Factory;
}

void FMDLMapHandler::Cleanup()
{
	MaterialExpressionFactory.Cleanup();
	Materials.Empty();
	FirstNormalExpression = CurrentNormalExpression = nullptr;
}

void FMDLMapHandler::SetFunctionAssetPath(const TCHAR* AssetPath)
{
	if (FunctionLoader)
	{
		FunctionLoader->SetAssetPath(AssetPath);
	}
}

void FMDLMapHandler::SetObjectFlags(EObjectFlags ObjectFlags)
{
	FunctionLoader->SetObjectFlags(ObjectFlags);
}

TArray<MDLImporterLogging::FLogMessage> FMDLMapHandler::GetLogMessages()
{
	return MaterialExpressionFactory.GetLogMessages();
}

#endif // #ifdef USE_MDLSDK
