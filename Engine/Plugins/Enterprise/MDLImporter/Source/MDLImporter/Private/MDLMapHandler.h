// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "generator/MaterialExpressionFactory.h"
#include "mdl/MapDistilHandler.h"


class UTextureFactory;
namespace Mdl
{
	class FApiContext;
}
namespace Generator
{
	class FMaterialTextureFactory;
	class FFunctionLoader;
}

class FMDLMapHandler : public Mdl::IMapDistilHandler
{
public:
	explicit FMDLMapHandler(const Mdl::FApiContext& MdlContext);
	virtual ~FMDLMapHandler();

	virtual void PreImport(const mi::neuraylib::IMaterial_definition& MDLMaterialDefinition,
	                       const mi::neuraylib::ICompiled_material&   MDLMaterial,
	                       mi::neuraylib::ITransaction&               MDLTransaction) override;
	virtual bool Import(const FString& MapName, bool bIsTexture, Mdl::FBakeParam& MapBakeParam) override;

	virtual void PostImport() override;


	void SetMaterials(const TMap<FString, UMaterial*>& Materials);
	void SetTextureFactory(Generator::FMaterialTextureFactory* Factory);

	void Cleanup();

	TArray<MDLImporterLogging::FLogMessage> GetLogMessages();
	void SetFunctionAssetPath(const TCHAR* AssetPath);
	void SetObjectFlags(EObjectFlags ObjectFlags);

private:
	void SetupNormalExpression(const FString& MapName);

private:
	Generator::FMaterialExpressionFactory  MaterialExpressionFactory;
	TMap<FString, UMaterial*>              Materials;
	TUniquePtr<Generator::FFunctionLoader> FunctionLoader;
	Generator::FMaterialTextureFactory*    TextureFactory;
	UMaterial*                             CurrentMaterial;
	UMaterialExpression*                   CurrentNormalExpression;
	UMaterialExpression*                   FirstNormalExpression;
};

inline void FMDLMapHandler::SetMaterials(const TMap<FString, UMaterial*>& InMaterials)
{
	Materials = InMaterials;
}
