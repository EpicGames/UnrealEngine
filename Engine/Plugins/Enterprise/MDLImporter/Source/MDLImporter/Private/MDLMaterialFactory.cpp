// Copyright Epic Games, Inc. All Rights Reserved.

#include "MDLMaterialFactory.h"

#include "MDLMaterialPropertyFactory.h"
#include "MDLMaterialSelector.h"

#include "Materials/MaterialExpressionClearCoatNormalCustomOutput.h"
#include "generator/MaterialExpressions.h"
#include "common/Logging.h"
#include "mdl/MaterialCollection.h"
#include "generator/MaterialExpressionConnection.h"
#include "mdl/Utility.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "EditorFramework/AssetImportData.h"
#include "MaterialEditingLibrary.h"
#include "ObjectTools.h"
#include "PackageTools.h"

namespace MDLImporterImpl
{
	template <typename T>
	void Connect(T& Target, const Mdl::FMaterial::FExpressionEntry& ExpressionEntry)
	{
		check(Target.Expression == nullptr || ExpressionEntry.Expression == nullptr);
		Generator::Connect(Target, Generator::FMaterialExpressionConnection(ExpressionEntry.Expression, ExpressionEntry.Index));
	}

	void SetupMaterial(const Mdl::FMaterial& MdlMaterial, const FMDLMaterialSelector& MaterialSelector,
	                   FMDLMaterialPropertyFactory& MaterialPropertyFactory, UMaterial& Material)
	{
		UMaterialExpressionClearCoatNormalCustomOutput* UnderClearcoatNormal = nullptr;

		const FMDLMaterialSelector::EMaterialType MaterialType = MaterialSelector.GetMaterialType(MdlMaterial);

		UMaterialEditorOnlyData& MaterialEditorOnly = *Material.GetEditorOnlyData();

		// set material settings
		Material.bTangentSpaceNormal = true;
		switch (MaterialType)
		{
			case FMDLMaterialSelector::EMaterialType::Opaque:
				break;
			case FMDLMaterialSelector::EMaterialType::Masked:
				Material.BlendMode = EBlendMode::BLEND_Masked;
				Material.TwoSided  = true;
				break;
			case FMDLMaterialSelector::EMaterialType::Translucent:
				Material.BlendMode                = EBlendMode::BLEND_Translucent;
				Material.TwoSided                 = true;
				Material.TranslucencyLightingMode = ETranslucencyLightingMode::TLM_Surface;
				break;
			case FMDLMaterialSelector::EMaterialType::Clearcoat:
				Material.SetShadingModel(EMaterialShadingModel::MSM_ClearCoat);
				break;
			case FMDLMaterialSelector::EMaterialType::Emissive:
				Material.BlendMode = EBlendMode::BLEND_Opaque;
				break;
			case FMDLMaterialSelector::EMaterialType::Carpaint:
				Material.SetShadingModel(EMaterialShadingModel::MSM_ClearCoat);
				break;
			case FMDLMaterialSelector::EMaterialType::Subsurface:
				Material.SetShadingModel(EMaterialShadingModel::MSM_Subsurface);
				break;
			case FMDLMaterialSelector::EMaterialType::Count:
				break;
			default:
				break;
		}

		if (Material.GetShadingModels().HasShadingModel(EMaterialShadingModel::MSM_ClearCoat))
		{
			UnderClearcoatNormal = Generator::NewMaterialExpression<UMaterialExpressionClearCoatNormalCustomOutput>(&Material);
		}

		// create material parameters/constants
		Mat::FParameterMap ParameterMap = MaterialPropertyFactory.CreateProperties(Material.GetFlags(), MdlMaterial, Material);
		if (MaterialType == FMDLMaterialSelector::EMaterialType::Translucent)
		{
			// add expression maps

			check(MdlMaterial.IOR.ExpressionData.Index == 0);
			check(MdlMaterial.Opacity.ExpressionData.Index == 0);
			check(MdlMaterial.BaseColor.ExpressionData.Index == 0);

			if (ParameterMap.Contains(Mat::EMaterialParameter::IOR))
			{
				if (MdlMaterial.IOR.ExpressionData.Expression)
				{
					ParameterMap[Mat::EMaterialParameter::IOR] = MdlMaterial.IOR.ExpressionData.Expression;
				}
			}
			else
				ParameterMap.Add(Mat::EMaterialParameter::IOR) = MdlMaterial.IOR.ExpressionData.Expression;

			if (ParameterMap.Contains(Mat::EMaterialParameter::Opacity))
			{
				if (MdlMaterial.Opacity.ExpressionData.Expression)
				{
					ParameterMap[Mat::EMaterialParameter::Opacity] = MdlMaterial.Opacity.ExpressionData.Expression;
				}
			}
			else
				ParameterMap.Add(Mat::EMaterialParameter::Opacity) = MdlMaterial.Opacity.ExpressionData.Expression;

			if (ParameterMap.Contains(Mat::EMaterialParameter::BaseColor))
			{
				if (MdlMaterial.BaseColor.ExpressionData.Expression)
				{
					ParameterMap[Mat::EMaterialParameter::BaseColor] = MdlMaterial.BaseColor.ExpressionData.Expression;
				}
			}
			else
				ParameterMap.Add(Mat::EMaterialParameter::BaseColor) = MdlMaterial.BaseColor.ExpressionData.Expression;
		}

		// create the baked maps of the material
		const Mat::IMaterialFactory& MaterialFactory = MaterialSelector.GetMaterialFactory(MaterialType);
		MaterialFactory.Create(MdlMaterial, ParameterMap, Material);

		// setup material connections
		{
			if (MaterialType != FMDLMaterialSelector::EMaterialType::Translucent)
				// TODO: find a nicer way to handle this cases
				Connect(MaterialEditorOnly.BaseColor, MdlMaterial.BaseColor.ExpressionData);
			Connect(MaterialEditorOnly.EmissiveColor, MdlMaterial.Emission.ExpressionData);
			Connect(MaterialEditorOnly.SubsurfaceColor, MdlMaterial.Scattering.ExpressionData);
			Connect(MaterialEditorOnly.Roughness, MdlMaterial.Roughness.ExpressionData);
			Connect(MaterialEditorOnly.Metallic, MdlMaterial.Metallic.ExpressionData);
			Connect(MaterialEditorOnly.Specular, MdlMaterial.Specular.ExpressionData);

			if (MaterialType == FMDLMaterialSelector::EMaterialType::Translucent)
			{
				// ignored, as they don't map 1 on 1
			}
			else if (MaterialType == FMDLMaterialSelector::EMaterialType::Masked)
			{
				Connect(MaterialEditorOnly.OpacityMask, MdlMaterial.Opacity.ExpressionData);
			}

			Connect(MaterialEditorOnly.ClearCoat, MdlMaterial.Clearcoat.Weight.ExpressionData);
			Connect(MaterialEditorOnly.ClearCoatRoughness, MdlMaterial.Clearcoat.Roughness.ExpressionData);
			if (UnderClearcoatNormal)
			{
				Connect(MaterialEditorOnly.Normal, MdlMaterial.Clearcoat.Normal.ExpressionData);
				Connect(UnderClearcoatNormal->Input, MdlMaterial.Normal.ExpressionData);
			}
			else
			{
				Connect(MaterialEditorOnly.Normal, MdlMaterial.Normal.ExpressionData);
			}
		}

		if (UnderClearcoatNormal && !UnderClearcoatNormal->Input.Expression)
		{
			// delete if unused
			Material.GetExpressionCollection().RemoveExpression(UnderClearcoatNormal);
			UnderClearcoatNormal = nullptr;
		}

		UMaterialEditingLibrary::LayoutMaterialExpressions(&Material);

		Material.MarkPackageDirty();
		Material.PostEditChange();
	}
}

FMDLMaterialFactory::FMDLMaterialFactory(Generator::FMaterialTextureFactory& MaterialTextureFactory)
#ifdef USE_MDLSDK
    : MaterialSelector(new FMDLMaterialSelector())
    , MaterialPropertyFactory(new FMDLMaterialPropertyFactory())
#endif
{
#ifdef USE_MDLSDK
	MaterialPropertyFactory->SetTextureFactory(&MaterialTextureFactory);
#endif
}

FMDLMaterialFactory::~FMDLMaterialFactory() {}

bool FMDLMaterialFactory::CreateMaterials(const FString& Filename, UObject* ParentPackage, EObjectFlags Flags, Mdl::FMaterialCollection& Materials)
{
	CleanUp();

	for (const Mdl::FMaterial& MdlMaterial : Materials)
	{
		if (MdlMaterial.IsDisabled())
		{
			continue;
		}

		FString  MaterialPackageName = UPackageTools::SanitizePackageName(*(ParentPackage->GetName() / MdlMaterial.Name));
		UObject* MaterialPackage = ParentPackage == GetTransientPackage() ? ParentPackage : CreatePackage(*MaterialPackageName);

		FString MaterialName;

		if ( MaterialPackage == GetTransientPackage() )
		{
			// Avoid conflicts in the transient package by adding the module name to the material name since it's not part of the package path
			MaterialName = Materials.Name + TEXT("_") + MdlMaterial.Name;
		}
		else
		{
			MaterialName = MdlMaterial.Name;
		}
		MaterialName = ObjectTools::SanitizeObjectName( MaterialName );

		UMaterial* NewMaterial = NewObject<UMaterial>(MaterialPackage, UMaterial::StaticClass(), FName(*MaterialName), Flags);
		check(NewMaterial != nullptr);
		NewMaterial->AssetImportData = NewObject<UAssetImportData>(NewMaterial, TEXT("AssetImportData"));
		NewMaterial->AssetImportData->Update(Filename);

		const FString DbName        = Mdl::Util::GetMaterialDatabaseName(Materials.Name, MdlMaterial.BaseName, true);
		NameMaterialMap.Add(DbName) = NewMaterial;
	}

	return true;
}

void FMDLMaterialFactory::PostImport(Mdl::FMaterialCollection& Materials)
{
#ifdef USE_MDLSDK
	for (const Mdl::FMaterial& MdlMaterial : Materials)
	{
		if (MdlMaterial.IsDisabled())
		{
			continue;
		}

		const FString DbName   = Mdl::Util::GetMaterialDatabaseName(Materials.Name, MdlMaterial.BaseName, true);
		UMaterial*    Material = NameMaterialMap[DbName];
		if (Material == nullptr)
		{
			continue;
		}

		MDLImporterImpl::SetupMaterial(MdlMaterial, *MaterialSelector, *MaterialPropertyFactory, *Material);

		CreatedMaterials.Add(Material);
		FAssetRegistryModule::AssetCreated(Material);

		const FString MaterialTypeName = FMDLMaterialSelector::ToString(MaterialSelector->GetMaterialType(MdlMaterial));
		UE_LOGF(LogMDLImporter, Log, "Created material %ls based on %ls", *MdlMaterial.Name, *MaterialTypeName);
	}
#endif
}

void FMDLMaterialFactory::Reimport(const Mdl::FMaterial& MdlMaterial, UMaterial& Material)
{
#ifdef USE_MDLSDK
	MDLImporterImpl::SetupMaterial(MdlMaterial, *MaterialSelector, *MaterialPropertyFactory, Material);

	// Material.AssetImportData->Update(InFileName);

	const FString MaterialTypeName = FMDLMaterialSelector::ToString(MaterialSelector->GetMaterialType(MdlMaterial));
	UE_LOGF(LogMDLImporter, Log, "Reimported material %ls based on %ls", *MdlMaterial.Name, *MaterialTypeName);
#endif
}

void FMDLMaterialFactory::CleanUp()
{
	CreatedMaterials.Empty();
	NameMaterialMap.Empty();
}
