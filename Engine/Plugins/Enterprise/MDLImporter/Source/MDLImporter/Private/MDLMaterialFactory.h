// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"

class UMaterialInterface;
class UMaterial;
class FMDLMaterialPropertyFactory;
class FMDLMaterialSelector;
namespace Mdl
{
	struct FMaterial;
	class FMaterialCollection;
}
namespace Generator
{
	class FMaterialTextureFactory;
}

class FMDLMaterialFactory
{
public:
	FMDLMaterialFactory(Generator::FMaterialTextureFactory& MaterialTextureFactory);
	~FMDLMaterialFactory();

	bool CreateMaterials(const FString& Filename, UObject* ParentPackage, EObjectFlags Flags, Mdl::FMaterialCollection& Materials);

	void PostImport(Mdl::FMaterialCollection& Materials);

	void Reimport(const Mdl::FMaterial& MdlMaterial, UMaterial& Material);

	// Returns newly created materials.
	const TArray<UMaterialInterface*>& GetCreatedMaterials() const;
	// Returns a map with the MDL DB material names
	const TMap<FString, UMaterial*>& GetNameMaterialMap() const;

	void CleanUp();

private:
#ifdef USE_MDLSDK
	TUniquePtr<FMDLMaterialSelector>        MaterialSelector;
	TUniquePtr<FMDLMaterialPropertyFactory> MaterialPropertyFactory;
#endif
	TMap<FString, UMaterial*>   NameMaterialMap;
	TArray<UMaterialInterface*> CreatedMaterials;
};

inline const TArray<UMaterialInterface*>& FMDLMaterialFactory::GetCreatedMaterials() const
{
	return CreatedMaterials;
}

inline const TMap<FString, UMaterial*>& FMDLMaterialFactory::GetNameMaterialMap() const
{
	return NameMaterialMap;
}
