// Copyright Epic Games, Inc. All Rights Reserved.

#include "MDLMaterialImporter.h"

#include "MDLImporter.h"
#include "MDLImporterModule.h"

#include "mdl/MaterialCollection.h"

#include "Materials/MaterialInterface.h"
#include "Misc/Paths.h"
#include "UObject/Package.h"

void FMdlMaterialImporter::AddSearchPath(const FString& SearchPath)
{
	FModuleManager& ModuleManager = FModuleManager::Get();
	IMDLImporterModule* MDLImporterModule = ModuleManager.GetModulePtr<IMDLImporterModule>(FName("MDLImporter"));

	FMDLImporter& MDLImporter = MDLImporterModule->GetMDLImporter();
	MDLImporter.AddSearchPath(SearchPath);
}

void FMdlMaterialImporter::RemoveSearchPath(const FString& SearchPath)
{
	FModuleManager& ModuleManager = FModuleManager::Get();
	IMDLImporterModule* MDLImporterModule = ModuleManager.GetModulePtr<IMDLImporterModule>(FName("MDLImporter"));

	FMDLImporter& MDLImporter = MDLImporterModule->GetMDLImporter();
	MDLImporter.RemoveSearchPath(SearchPath);
}

UMaterialInterface* FMdlMaterialImporter::ImportMaterialFromModule(UPackage* ParentPackage, EObjectFlags ObjectFlags, const FString& MdlModuleName,
	const FString& MdlDefinitionName, const UMDLImporterOptions& ImporterOptions)
{
	FModuleManager& ModuleManager = FModuleManager::Get();
	IMDLImporterModule* MDLImporterModule = ModuleManager.GetModulePtr<IMDLImporterModule>(FName("MDLImporter"));

	FMDLImporter& MDLImporter = MDLImporterModule->GetMDLImporter();

	// Load module
	Mdl::FMaterialCollection MaterialsInModule;
	MDLImporter.LoadModule( MdlModuleName, ImporterOptions, MaterialsInModule );

	for ( int32 MaterialIndex = 0; MaterialIndex < MaterialsInModule.Count(); ++MaterialIndex )
	{
		Mdl::FMaterial& Material = MaterialsInModule[ MaterialIndex ];

		if ( !Material.Name.Equals( MdlDefinitionName, ESearchCase::IgnoreCase ) ) // TODO: Check if MDL Definition names are case sensitive
		{
			MaterialsInModule.RemoveAt( MaterialIndex );
			--MaterialIndex;
		}
	}

	if ( MaterialsInModule.Count() <= 0 )
	{
		// TODO: Log Error
		return nullptr;
	}

	// Import
	MDLImporter.ImportMaterials( ParentPackage, ObjectFlags, MaterialsInModule );

	TArray< UMaterialInterface* > UEMaterials = MDLImporter.GetCreatedMaterials();

	if ( UEMaterials.Num() > 0 )
	{
		return UEMaterials[0];
	}
	else
	{
		return nullptr;
	}
}

FString UE::Mdl::Util::ConvertFilePathToModuleName( const TCHAR* FilePath )
{
	// Remove file extension
	const bool bRemovePath = false;
	FString ModuleName = FPaths::GetBaseFilename( FilePath, bRemovePath );

	// Remove drive letter
	int32 DotIndex = INDEX_NONE;
	if ( ModuleName.FindChar( TEXT(':'), DotIndex ) )
	{
		ModuleName.RightChopInline( DotIndex + 1 );
	}

	// Replace directory separator with "::"
	ModuleName.ReplaceInline( TEXT("/"), TEXT("::") );

	if ( !ModuleName.StartsWith( TEXT("::") ) && !ModuleName.StartsWith( TEXT(".") ) && !ModuleName.StartsWith( TEXT("..") ) )
	{
		ModuleName.InsertAt( 0, TEXT("::") );
	}

	return ModuleName;
}
