// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "common/TextureProperty.h"


struct FMDLMaterialProperty
{
	enum class EPropertyType
	{
		Texture,
		Vector,
		Scalar,
		Boolean
	};

	int           Id;
	FString       Name;
	EPropertyType Type;

	// @note Either a texture or a simple value.
	Common::FTextureProperty Texture;
	union {
		float Color[4];  // FLinearColor
		float Value;
		bool  bValue;
	};
	bool bIsConstant;

	FMDLMaterialProperty()
	    : Id(-1)
	    , bIsConstant(false)
	{
	}
};
