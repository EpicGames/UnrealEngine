// Copyright Epic Games, Inc. All Rights Reserved.

#include "MDLMaterialPropertyFactory.h"

#include "MDLMaterialProperty.h"

#include "Materials/MaterialExpressionConstant.h"
#include "generator/MaterialExpressions.h"
#include "Materials/MaterialExpressionConstant4Vector.h"
#include "generator/MaterialTextureFactory.h"
#include "Materials/MaterialExpressionScalarParameter.h"
#include "mdl/Material.h"

#include "Engine/Texture2D.h"
#include "Materials/MaterialExpressionStaticBool.h"
#include "Materials/MaterialExpressionStaticBoolParameter.h"
#include "Materials/MaterialExpressionTextureObject.h"
#include "Materials/MaterialExpressionTextureObjectParameter.h"
#include "Materials/MaterialExpressionVectorParameter.h"

namespace MDLImporterImpl
{
	struct FParameterProperty
	{
		FVector2D            Range;
		EMaterialSamplerType SamplerType;

		void operator=(const FVector2D& Rhs)
		{
			Range = Rhs;
		}

		void operator=(EMaterialSamplerType Rhs)
		{
			SamplerType = Rhs;
		}
	};

	void AddValueProperty(const FVector3f&                VectorProperty,
	                      const FString&                PropertyName,
	                      Mat::EMaterialParameter       ParameterType,
	                      TArray<FMDLMaterialProperty>& MaterialProperties)
	{
		const int32           NewIndex         = MaterialProperties.Emplace();
		FMDLMaterialProperty& MaterialProperty = MaterialProperties[NewIndex];
		MaterialProperty.Type                  = FMDLMaterialProperty::EPropertyType::Vector;
		MaterialProperty.Name                  = PropertyName;
		MaterialProperty.Id                    = (int)ParameterType;

		static_assert(sizeof(MaterialProperty.Color) == sizeof(FLinearColor), "INVALID_SIZES");
		new (MaterialProperty.Color) FLinearColor(VectorProperty);
	}

	template <typename T>
	void AddValueProperty(T                             ValueProperty,
	                      const FString&                PropertyName,
	                      Mat::EMaterialParameter       ParameterType,
	                      TArray<FMDLMaterialProperty>& MaterialProperties)
	{
		const int32           NewIndex         = MaterialProperties.Emplace();
		FMDLMaterialProperty& MaterialProperty = MaterialProperties[NewIndex];
		MaterialProperty.Type                  = FMDLMaterialProperty::EPropertyType::Scalar;
		MaterialProperty.Name                  = PropertyName;
		MaterialProperty.Value                 = static_cast<float>(ValueProperty);
		MaterialProperty.Id                    = (int)ParameterType;
	}

	bool AddTextureProperty(const Common::FTextureProperty& Texture,
	                        const FString&                  PropertyName,
	                        Mat::EMaterialParameter         ParameterType,
	                        TArray<FMDLMaterialProperty>&   MaterialProperties)
	{
		if (Texture.Path.IsEmpty())
		{
			return false;
		}

		const int32           NewIndex         = MaterialProperties.Emplace();
		FMDLMaterialProperty& MaterialProperty = MaterialProperties[NewIndex];
		MaterialProperty.Type                  = FMDLMaterialProperty::EPropertyType::Texture;
		MaterialProperty.Name                  = PropertyName;
		MaterialProperty.Texture               = Texture;
		MaterialProperty.Id                    = (int)ParameterType;

		return true;
	}

	Mat::EMaterialParameter GetMapParamter(Mat::EMaterialParameter ParameterType)
	{
		ParameterType = Mat::EMaterialParameter((int)ParameterType + 1);
		return ParameterType;
	}

	bool AddTextureProperty(const Mdl::FMaterial::FNormalMapEntry& NormalMapProperty,
	                        const FString&                         PropertyName,
	                        Mat::EMaterialParameter                ParameterType,
	                        TArray<FMDLMaterialProperty>&          MaterialProperties)
	{
		if (NormalMapProperty.HasExpression())
			return true;

		if (AddTextureProperty(NormalMapProperty.Texture, PropertyName, ParameterType, MaterialProperties))
		{
			const int Index = PropertyName.Find(TEXT(" Map"));
			check(Index != INDEX_NONE);
			const FString                 StrengthPropertyName = PropertyName.Mid(0, Index) + TEXT(" Strength");
			const Mat::EMaterialParameter StrengthType         = Mat::EMaterialParameter((int)ParameterType + 1);
			AddValueProperty(NormalMapProperty.Strength, StrengthPropertyName, StrengthType, MaterialProperties);
			MaterialProperties.Top().bIsConstant = false;

			return true;
		}
		return false;
	}

	template <typename T, int I>
	bool AddProperty(const Mdl::FMaterial::TMapEntry<T, I>& MapProperty,
	                 const FString&                         PropertyName,
	                 Mat::EMaterialParameter                ParameterType,
	                 TArray<FMDLMaterialProperty>&          MaterialProperties)
	{
		if (MapProperty.HasExpression())
			return false;

		if (!AddTextureProperty(MapProperty.Texture, PropertyName + TEXT(" Map"), GetMapParamter(ParameterType), MaterialProperties))
		{
			// no texture present
			AddValueProperty(MapProperty.Value, PropertyName, ParameterType, MaterialProperties);
			MaterialProperties.Top().bIsConstant = !MapProperty.WasValueBaked();
		}
		return true;
	}

	void SetMaterialProperties(const Mdl::FMaterial& Material, TArray<FMDLMaterialProperty>& MaterialProperties)
	{
		if (AddProperty(Material.BaseColor, TEXT("Base Color"), Mat::EMaterialParameter::BaseColor, MaterialProperties))
			MaterialProperties.Top().bIsConstant = false;  // always parameter
		AddProperty(Material.Specular, TEXT("Specular"), Mat::EMaterialParameter::Specular, MaterialProperties);
		if (AddProperty(Material.Roughness, TEXT("Roughness"), Mat::EMaterialParameter::Roughness, MaterialProperties))
			MaterialProperties.Top().bIsConstant = false;  // always parameter
		if (AddProperty(Material.Metallic, TEXT("Metallic"), Mat::EMaterialParameter::Metallic, MaterialProperties))
			MaterialProperties.Top().bIsConstant = false;  // always parameter
		AddProperty(Material.Scattering, TEXT("Subsurface Color"), Mat::EMaterialParameter::SubSurfaceColor, MaterialProperties);

		// clear coat properties
		if (AddProperty(Material.Clearcoat.Weight, TEXT("ClearCoat Weight"), Mat::EMaterialParameter::ClearCoatWeight, MaterialProperties))
			MaterialProperties.Top().bIsConstant = false;  // always parameter
		if (AddProperty(Material.Clearcoat.Roughness, TEXT("ClearCoat Roughness"), Mat::EMaterialParameter::ClearCoatRoughness, MaterialProperties))
			MaterialProperties.Top().bIsConstant = false;  // always parameter

		// normal maps
		if (Material.Clearcoat.Normal.Texture.Path.IsEmpty())
		{
			AddTextureProperty(Material.Normal, TEXT("Normal Map"), Mat::EMaterialParameter::NormalMap, MaterialProperties);
		}
		else
		{
			AddTextureProperty(
			    Material.Clearcoat.Normal, TEXT("ClearCoat Normal Map"), Mat::EMaterialParameter::ClearCoatNormalMap, MaterialProperties);
			AddTextureProperty(Material.Normal, TEXT("Normal Map"), Mat::EMaterialParameter::NormalMap, MaterialProperties);
		}
		AddTextureProperty(Material.Displacement, TEXT("Displacement Map"), Mat::EMaterialParameter::DisplacementMap, MaterialProperties);

		// emission properties
		if (!AddTextureProperty(Material.Emission.Texture, TEXT("Emission Map"), Mat::EMaterialParameter::EmissionColorMap, MaterialProperties))
		{
			FVector3f EmissionColor = Material.Emission.Value;
			if (Material.Emission.WasValueBaked() && Material.EmissionStrength.Value > 0.f && EmissionColor.Size() > 0.1f)
			{
				AddValueProperty(
				    Material.EmissionStrength.Value, TEXT("Emission Strength"), Mat::EMaterialParameter::EmissionStrength, MaterialProperties);
				AddValueProperty(Material.Emission.Value, TEXT("Emission"), Mat::EMaterialParameter::EmissionColor, MaterialProperties);
			}
		}

		// transparent properties
		AddProperty(Material.Opacity, TEXT("Opacity"), Mat::EMaterialParameter::Opacity, MaterialProperties);
		if (Material.IOR.Value.X != 1.f)
		{
			AddValueProperty(Material.IOR.Value.X, TEXT("IOR"), Mat::EMaterialParameter::IOR, MaterialProperties);
		}
		if (Material.Absorption.WasValueBaked())
		{
			AddValueProperty(Material.Absorption.Value, TEXT("AbsorptionColor"), Mat::EMaterialParameter::AbsorptionColor, MaterialProperties);
		}

		// other properties
		if (Material.TilingFactor != 0.f && Material.TilingFactor != 1.f)
		{
			AddValueProperty(Material.TilingFactor, TEXT("Tiling Factor"), Mat::EMaterialParameter::Tiling, MaterialProperties);
			UE_LOGF(LogMDLImporter, Log, "Material %ls has scale: %f", *Material.Name, Material.TilingFactor);
		}

		if (Material.Tiling != FVector2D(0.f, 0.f) && Material.Tiling != FVector2D(1.f, 1.f))
		{
			AddValueProperty(Material.Tiling.X, TEXT("U Tiling"), Mat::EMaterialParameter::TilingU, MaterialProperties);
			AddValueProperty(Material.Tiling.Y, TEXT("V Tiling"), Mat::EMaterialParameter::TilingV, MaterialProperties);
			UE_LOGF(LogMDLImporter, Log, "Material %ls UV tiling: %f %f", *Material.Name, Material.Tiling.X, Material.Tiling.Y);
		}

		// carpaint properties
		if (Material.Carpaint.bEnabled && Material.Carpaint.Flakes.Depth)
		{
			check(!Material.Carpaint.Flakes.Texture.Path.IsEmpty());
			AddTextureProperty(Material.Carpaint.Flakes.Texture, TEXT("Flakes Slices"), Mat::EMaterialParameter::CarFlakesMap, MaterialProperties);
			AddTextureProperty(
			    Material.Carpaint.ThetaFiLUT.Texture, TEXT("Theta Slice LUT"), Mat::EMaterialParameter::CarFlakesLut, MaterialProperties);
			MaterialProperties.Top().bIsConstant = true;
		}
	}

	TMap<Mat::EMaterialParameter, FParameterProperty> InitParameterPropertyMap()
	{
		TMap<Mat::EMaterialParameter, FParameterProperty> RangeMap;

		const FVector2D DefaultValue(0.f, 1.f);
		RangeMap.Add(Mat::EMaterialParameter::Opacity)                 = DefaultValue;
		RangeMap.Add(Mat::EMaterialParameter::Roughness)               = DefaultValue;
		RangeMap.Add(Mat::EMaterialParameter::Metallic)                = DefaultValue;
		RangeMap.Add(Mat::EMaterialParameter::Specular)                = DefaultValue;
		RangeMap.Add(Mat::EMaterialParameter::NormalStrength)          = DefaultValue;
		RangeMap.Add(Mat::EMaterialParameter::ClearCoatWeight)         = DefaultValue;
		RangeMap.Add(Mat::EMaterialParameter::ClearCoatRoughness)      = DefaultValue;
		RangeMap.Add(Mat::EMaterialParameter::ClearCoatNormalStrength) = DefaultValue;
		RangeMap.Add(Mat::EMaterialParameter::EmissionStrength)        = FVector2D(0.f, 1000.f);
		RangeMap.Add(Mat::EMaterialParameter::DisplacementStrength)    = FVector2D(0.f, 100.f);

		RangeMap.Add(Mat::EMaterialParameter::OpacityMap)            = EMaterialSamplerType::SAMPLERTYPE_LinearGrayscale;
		RangeMap.Add(Mat::EMaterialParameter::RoughnessMap)          = EMaterialSamplerType::SAMPLERTYPE_LinearGrayscale;
		RangeMap.Add(Mat::EMaterialParameter::MetallicMap)           = EMaterialSamplerType::SAMPLERTYPE_LinearGrayscale;
		RangeMap.Add(Mat::EMaterialParameter::SpecularMap)           = EMaterialSamplerType::SAMPLERTYPE_LinearGrayscale;
		RangeMap.Add(Mat::EMaterialParameter::NormalMap)             = EMaterialSamplerType::SAMPLERTYPE_Normal;
		RangeMap.Add(Mat::EMaterialParameter::DisplacementMap)       = EMaterialSamplerType::SAMPLERTYPE_LinearColor;
		RangeMap.Add(Mat::EMaterialParameter::ClearCoatWeightMap)    = EMaterialSamplerType::SAMPLERTYPE_LinearGrayscale;
		RangeMap.Add(Mat::EMaterialParameter::ClearCoatRoughnessMap) = EMaterialSamplerType::SAMPLERTYPE_LinearGrayscale;
		RangeMap.Add(Mat::EMaterialParameter::ClearCoatNormalMap)    = EMaterialSamplerType::SAMPLERTYPE_Normal;
		RangeMap.Add(Mat::EMaterialParameter::CarFlakesLut)          = EMaterialSamplerType::SAMPLERTYPE_LinearGrayscale;

		return RangeMap;
	}

	TMap<Mat::EMaterialParameter, FParameterProperty> InitVirtualTextureParameterPropertyMap()
	{
		TMap<Mat::EMaterialParameter, FParameterProperty> RangeMap;

		RangeMap.Add(Mat::EMaterialParameter::OpacityMap) = EMaterialSamplerType::SAMPLERTYPE_VirtualLinearGrayscale;
		RangeMap.Add(Mat::EMaterialParameter::RoughnessMap) = EMaterialSamplerType::SAMPLERTYPE_VirtualLinearGrayscale;
		RangeMap.Add(Mat::EMaterialParameter::MetallicMap) = EMaterialSamplerType::SAMPLERTYPE_VirtualLinearGrayscale;
		RangeMap.Add(Mat::EMaterialParameter::SpecularMap) = EMaterialSamplerType::SAMPLERTYPE_VirtualLinearGrayscale;
		RangeMap.Add(Mat::EMaterialParameter::NormalMap) = EMaterialSamplerType::SAMPLERTYPE_VirtualNormal;
		RangeMap.Add(Mat::EMaterialParameter::DisplacementMap) = EMaterialSamplerType::SAMPLERTYPE_VirtualLinearColor;
		RangeMap.Add(Mat::EMaterialParameter::ClearCoatWeightMap) = EMaterialSamplerType::SAMPLERTYPE_VirtualLinearGrayscale;
		RangeMap.Add(Mat::EMaterialParameter::ClearCoatRoughnessMap) = EMaterialSamplerType::SAMPLERTYPE_VirtualLinearGrayscale;
		RangeMap.Add(Mat::EMaterialParameter::ClearCoatNormalMap) = EMaterialSamplerType::SAMPLERTYPE_VirtualNormal;
		RangeMap.Add(Mat::EMaterialParameter::CarFlakesLut) = EMaterialSamplerType::SAMPLERTYPE_VirtualLinearGrayscale;

		return RangeMap;
	}

	const TMap<Mat::EMaterialParameter, FParameterProperty> ParameterPropertyMap = InitParameterPropertyMap();
	const TMap<Mat::EMaterialParameter, FParameterProperty> VirtualTextureParameterPropertyMap = InitVirtualTextureParameterPropertyMap();
}

FMDLMaterialPropertyFactory::FMDLMaterialPropertyFactory()
    : TextureFactory(nullptr)
{
}

FMDLMaterialPropertyFactory::~FMDLMaterialPropertyFactory() {}

Mat::FParameterMap FMDLMaterialPropertyFactory::CreateProperties(EObjectFlags Flags, const Mdl::FMaterial& MdlMaterial, UMaterial& Material)
{
	check(TextureFactory);

	MaterialProperties.Empty();
	MDLImporterImpl::SetMaterialProperties(MdlMaterial, MaterialProperties);

	UObject*           ParentPackage = Material.GetOuter();
	Mat::FParameterMap ParameterMap;
	for (const FMDLMaterialProperty& Property : MaterialProperties)
	{
		switch (Property.Type)
		{
			case FMDLMaterialProperty::EPropertyType::Vector:
			{
				// Vector Params
				const FLinearColor& Color = *reinterpret_cast<const FLinearColor*>(Property.Color);
				if (Property.bIsConstant)
				{
					ParameterMap.Add((Mat::EMaterialParameter)Property.Id) =
					    Generator::NewMaterialExpressionConstant(&Material, Color.R, Color.G, Color.B, Color.A);
				}
				else
				{
					ParameterMap.Add((Mat::EMaterialParameter)Property.Id) =
					    Generator::NewMaterialExpressionVectorParameter(&Material, Property.Name, Color);
				}
				break;
			}
			case FMDLMaterialProperty::EPropertyType::Scalar:
			{
				// Scalar Params
				if (Property.bIsConstant)
				{
					ParameterMap.Add((Mat::EMaterialParameter)Property.Id) = Generator::NewMaterialExpressionConstant(&Material, Property.Value);
				}
				else
				{
					UMaterialExpressionScalarParameter* Parameter =
					    Generator::NewMaterialExpressionScalarParameter(&Material, Property.Name, Property.Value);
					if (MDLImporterImpl::ParameterPropertyMap.Find((Mat::EMaterialParameter)Property.Id))
					{
						const FVector2D Range = MDLImporterImpl::ParameterPropertyMap[(Mat::EMaterialParameter)Property.Id].Range;
						Parameter->SliderMin  = Range.X;
						Parameter->SliderMax  = Range.Y;
					}
					ParameterMap.Add((Mat::EMaterialParameter)Property.Id) = Parameter;
				}
				break;
			}
			case FMDLMaterialProperty::EPropertyType::Boolean:
			{
				// Bool Params
				if (Property.bIsConstant)
				{
					ParameterMap.Add((Mat::EMaterialParameter)Property.Id) = Generator::NewMaterialExpressionStaticBool(&Material, Property.bValue);
				}
				else
				{
					ParameterMap.Add((Mat::EMaterialParameter)Property.Id) =
					    Generator::NewMaterialExpressionStaticBoolParameter(&Material, Property.Name, Property.bValue);
				}
				break;
			}
			case FMDLMaterialProperty::EPropertyType::Texture:
			{
				Generator::FMaterialTextureFactory::FTextureSourcePtr ptr = const_cast<FImage*>(Property.Texture.Source);
				if (UTexture2D* Texture = TextureFactory->CreateTexture(ParentPackage, Property.Texture, ptr, Flags))
				{
					const bool bIsVirtualTexture = Texture->VirtualTextureStreaming;
					// Texture Params
					EMaterialSamplerType SamplerType =
						Property.Texture.Mode == Common::ETextureMode::ColorLinear ? 
							(bIsVirtualTexture ? SAMPLERTYPE_VirtualLinearColor : SAMPLERTYPE_LinearColor) : 
							(bIsVirtualTexture ? SAMPLERTYPE_VirtualColor : SAMPLERTYPE_Color);
				
					const MDLImporterImpl::FParameterProperty* ParameterProperty = bIsVirtualTexture ?
						MDLImporterImpl::VirtualTextureParameterPropertyMap.Find((Mat::EMaterialParameter)Property.Id) :
						MDLImporterImpl::ParameterPropertyMap.Find((Mat::EMaterialParameter)Property.Id);
					if (ParameterProperty)
					{
						SamplerType = ParameterProperty->SamplerType;
					}

					if (Property.bIsConstant)
					{
						UMaterialExpressionTextureObject* TextureObject        = Generator::NewMaterialExpressionTextureObject(&Material, Texture);
						TextureObject->SamplerType                             = SamplerType;
						ParameterMap.Add((Mat::EMaterialParameter)Property.Id) = TextureObject;
					}
					else
					{
						UMaterialExpressionTextureObjectParameter* TextureObject =
						    Generator::NewMaterialExpressionTextureObjectParameter(&Material, Property.Name, Texture);
						TextureObject->SamplerType                             = SamplerType;
						ParameterMap.Add((Mat::EMaterialParameter)Property.Id) = TextureObject;
					}
				}
				break;
			}
			default:
				break;
		}
	}

	return ParameterMap;
}
