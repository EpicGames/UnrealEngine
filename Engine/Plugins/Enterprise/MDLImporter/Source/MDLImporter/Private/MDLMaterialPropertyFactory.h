// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "material/MaterialFactory.h"

#include "UObject/ObjectMacros.h"

class UMaterial;
struct FMDLMaterialProperty;
namespace Generator
{
	class FMaterialTextureFactory;
}
namespace Mdl
{
	struct FMaterial;
}

class FMDLMaterialPropertyFactory
{
public:
	FMDLMaterialPropertyFactory();
	~FMDLMaterialPropertyFactory();

	void SetTextureFactory(Generator::FMaterialTextureFactory* Factory);

	Mat::FParameterMap CreateProperties(EObjectFlags Flags, const Mdl::FMaterial& MdlMaterial, UMaterial& Material);

private:
	TArray<FMDLMaterialProperty>        MaterialProperties;
	Generator::FMaterialTextureFactory* TextureFactory;
};

inline void FMDLMaterialPropertyFactory::SetTextureFactory(Generator::FMaterialTextureFactory* Factory)
{
	TextureFactory = Factory;
}
