// Copyright Epic Games, Inc. All Rights Reserved.

#include "MDLMaterialSelector.h"

#include "generator/FunctionLoader.h"
#include "material/BakedMaterialFactory.h"
#include "material/CarpaintMaterialFactory.h"
#include "material/TranslucentMaterialFactory.h"
#include "mdl/Material.h"


namespace MDLImporterImpl
{
	bool IsCarpaintMaterial(const Mdl::FMaterial& MdlMaterial)
	{
		return MdlMaterial.Carpaint.bEnabled;
	}

	bool IsClearcoatMaterial(const Mdl::FMaterial& MdlMaterial)
	{
		if (MdlMaterial.Clearcoat.Roughness.WasProcessed() || MdlMaterial.Clearcoat.Normal.WasProcessed())
		{
			return true;
		}

		float Roughness          = 1.f;
		float ClearCoatRoughness = 1.f;
		float ClearCoatWeight    = 1.f;

		if (MdlMaterial.Clearcoat.Roughness.Texture.Path.IsEmpty() && MdlMaterial.Clearcoat.Roughness.WasValueBaked())
		{
			ClearCoatRoughness = MdlMaterial.Clearcoat.Roughness.Value;
			if (MdlMaterial.Roughness.Texture.Path.IsEmpty())
			{
				Roughness = MdlMaterial.Roughness.Value;
			}
			if (MdlMaterial.Clearcoat.Weight.Texture.Path.IsEmpty())
			{
				ClearCoatWeight = MdlMaterial.Clearcoat.Weight.Value;
			}
			return (Roughness != ClearCoatRoughness) && (ClearCoatWeight > 0.01f);
		}
		return false;
	}

	bool IsSubsurfaceMaterial(const Mdl::FMaterial& MdlMaterial)
	{
		return MdlMaterial.Scattering.WasProcessed() || MdlMaterial.Scattering.WasValueBaked();
	}

	bool IsMaskedMaterial(const Mdl::FMaterial& MdlMaterial)
	{
		return MdlMaterial.Opacity.HasTexture();
	}

	bool IsEmissiveMaterial(const Mdl::FMaterial& MdlMaterial)
	{
		if (MdlMaterial.Emission.WasProcessed())
		{
			return true;
		}

		FLinearColor EmissionColor(MdlMaterial.Emission.Value);
		float        EmissionStrength = 1.f;

		if (MdlMaterial.Emission.WasValueBaked() && MdlMaterial.EmissionStrength.Value > 0.f && MdlMaterial.Emission.Value.Size() > 0.1f)
		{
			EmissionStrength = MdlMaterial.EmissionStrength.Value;

			EmissionColor = EmissionColor.Desaturate(1.f);
			return EmissionStrength > 0.1f && EmissionColor.R > 0.1f;
		}

		return false;
	}

	bool IsTransparentMaterial(const Mdl::FMaterial& MdlMaterial)
	{
		float Opacity = 1.f;
		float IOR     = 1.f;
		if (MdlMaterial.Opacity.WasValueBaked())
		{
			Opacity = MdlMaterial.Opacity.Value;
			if (MdlMaterial.IOR.WasValueBaked())
			{
				IOR = MdlMaterial.IOR.Value.X;
			}
		}

		return (Opacity < 1.f && IOR != 1.f) || MdlMaterial.Absorption.WasValueBaked();
	}
}

FMDLMaterialSelector::FMDLMaterialSelector()
    : FunctionLoader(new Generator::FFunctionLoader())
{
	MaterialFactories.SetNum((int)EMaterialType::Count);

	TSharedPtr<Mat::IMaterialFactory> BakedMaterialFactory(new Mat::FBakedMaterialFactory(*FunctionLoader));
	for (TSharedPtr<Mat::IMaterialFactory>& Factory : MaterialFactories)
	{
		Factory = BakedMaterialFactory;
	}
	TSharedPtr<Mat::IMaterialFactory> TranslucentMaterialFactory(new Mat::FTranslucentMaterialFactory(*FunctionLoader));
	MaterialFactories[(int)EMaterialType::Translucent] = TranslucentMaterialFactory;
	MaterialFactories[(int)EMaterialType::Masked]      = TranslucentMaterialFactory;

	MaterialFactories[(int)EMaterialType::Carpaint] = TSharedPtr<Mat::IMaterialFactory>(new Mat::FCarpaintMaterialFactory(*FunctionLoader));
}

FMDLMaterialSelector::~FMDLMaterialSelector() {}

FMDLMaterialSelector::EMaterialType FMDLMaterialSelector::GetMaterialType(const Mdl::FMaterial& MdlMaterial) const
{
	EMaterialType Type = EMaterialType::Opaque;

	if (MDLImporterImpl::IsMaskedMaterial(MdlMaterial))
	{
		Type = EMaterialType::Masked;
	}
	else if (MDLImporterImpl::IsTransparentMaterial(MdlMaterial))
	{
		Type = EMaterialType::Translucent;
	}
	else if (MDLImporterImpl::IsCarpaintMaterial(MdlMaterial))
	{
		Type = EMaterialType::Carpaint;
	}
	else if (MDLImporterImpl::IsSubsurfaceMaterial(MdlMaterial))
	{
		Type = EMaterialType::Subsurface;
	}
	else if (MDLImporterImpl::IsClearcoatMaterial(MdlMaterial))
	{
		Type = EMaterialType::Clearcoat;
	}
	else if (MDLImporterImpl::IsEmissiveMaterial(MdlMaterial))
	{
		Type = EMaterialType::Emissive;
	}

	return Type;
}

const Mat::IMaterialFactory& FMDLMaterialSelector::GetMaterialFactory(const Mdl::FMaterial& Material) const
{
	return GetMaterialFactory(GetMaterialType(Material));
}

const Mat::IMaterialFactory& FMDLMaterialSelector::GetMaterialFactory(EMaterialType MaterialType) const
{
	return *MaterialFactories[static_cast<int>(MaterialType)];
}

FString FMDLMaterialSelector::ToString(EMaterialType Type)
{
	switch (Type)
	{
		case FMDLMaterialSelector::EMaterialType::Opaque:
			return TEXT("Opaque");
		case FMDLMaterialSelector::EMaterialType::Masked:
			return TEXT("Masked");
		case FMDLMaterialSelector::EMaterialType::Translucent:
			return TEXT("Transparent");
		case FMDLMaterialSelector::EMaterialType::Carpaint:
			return TEXT("Carpaint");
		case FMDLMaterialSelector::EMaterialType::Subsurface:
			return TEXT("Subsurface");
		case FMDLMaterialSelector::EMaterialType::Clearcoat:
			return TEXT("Clearcoat");
		case FMDLMaterialSelector::EMaterialType::Emissive:
			return TEXT("Emissive");
		case FMDLMaterialSelector::EMaterialType::Count:
		default:
			check(false);
			break;
	}
	return TEXT("Unknown");
}
