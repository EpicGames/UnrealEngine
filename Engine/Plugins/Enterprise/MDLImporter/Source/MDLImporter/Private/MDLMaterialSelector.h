// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Templates/SharedPointer.h"

namespace Mat { class IMaterialFactory; }

namespace Mdl
{
	struct FMaterial;
}

namespace Generator
{
	class FFunctionLoader;
}
class FMDLMaterialSelector
{
public:
	enum class EMaterialType
	{
		Opaque = 0,
		Masked,
		Translucent,
		Clearcoat,
		Emissive,
		Carpaint,
		Subsurface,
		Count
	};

	FMDLMaterialSelector();
	~FMDLMaterialSelector();

	EMaterialType GetMaterialType(const Mdl::FMaterial& Material) const;

	const Mat::IMaterialFactory& GetMaterialFactory(EMaterialType MaterialType) const;
	const Mat::IMaterialFactory& GetMaterialFactory(const Mdl::FMaterial& Material) const;

	static FString ToString(EMaterialType Type);

private:
	TArray<TSharedPtr<Mat::IMaterialFactory>, TFixedAllocator<(int)EMaterialType::Count> > MaterialFactories;

	TUniquePtr<Generator::FFunctionLoader> FunctionLoader;
};
