// Copyright Epic Games, Inc. All Rights Reserved.

#include "MdlFileImporter.h"
#include "MDLImporterModule.h"

#include "mdl/MaterialCollection.h"


#include "MDLImporter.h"


class FMdlFileImporterImpl : public IMdlFileImporter
{
public:

	~FMdlFileImporterImpl()
	{
		FMDLImporter& MDLImporter = GetImporterInstance();
		MDLImporter.CleanUp();
	}

	FMDLImporter& GetImporterInstance()
	{
		FModuleManager& ModuleManager = FModuleManager::Get();
		IMDLImporterModule* MDLImporterModule = ModuleManager.GetModulePtr<IMDLImporterModule>(FName("MDLImporter"));
		return MDLImporterModule->GetMDLImporter();
	}

	virtual bool OpenFile(const FString& InFilename, const UMDLImporterOptions& InImporterOptions) override
	{
		FMDLImporter& MDLImporter = GetImporterInstance();
		return MDLImporter.OpenFile(InFilename, InImporterOptions, Materials);
	}

	virtual int GetMaterialCountInFile() override
	{
		return Materials.Count();;
	}

	virtual bool ImportMaterials(UObject* ParentPackage, EObjectFlags Flags, FProgressFunc ProgressFunc = nullptr) override
	{
		FMDLImporter& MDLImporter = GetImporterInstance();
		return MDLImporter.ImportMaterials(ParentPackage, Flags, Materials);
	}

	virtual TMap<FString, UMaterialInterface*> GetCreatedMaterials() override
	{
		FMDLImporter& MDLImporter = GetImporterInstance();
		TMap<FString, UMaterialInterface*> Result;

		const TArray<UMaterialInterface*> ImportedMaterials = MDLImporter.GetCreatedMaterials();
		for (int32 i = 0; i < ImportedMaterials.Num(); i++)
		{
			Mdl::FMaterial& Material = Materials[i];
			UMaterialInterface* ImportedMaterial = ImportedMaterials[i];
			Result.Add(Material.Name, ImportedMaterial);
		}
		return Result;
	}

	virtual const TArray<MDLImporterLogging::FLogMessage> GetLogMessages() override
	{
		FMDLImporter& MDLImporter = GetImporterInstance();
		return MDLImporter.GetLogMessages();
	}

	bool Reimport(const FString& InFileName, const UMDLImporterOptions& InImporterOptions, UMaterialInterface* OutMaterial)
	{
		FMDLImporter& MDLImporter = GetImporterInstance();
		return MDLImporter.Reimport(InFileName, InImporterOptions, OutMaterial);
	}

	Mdl::FMaterialCollection Materials;
};

TUniquePtr<IMdlFileImporter> IMdlFileImporter::Create()
{
	return MakeUnique< FMdlFileImporterImpl >();
}
