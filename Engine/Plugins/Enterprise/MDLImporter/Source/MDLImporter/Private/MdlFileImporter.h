// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "common/Logging.h"

#include "UObject/ObjectMacros.h"

class UMaterialInterface;
class UMDLImporterOptions;
class UObject;
class UTextureFactory;
class UMaterialInterface;

class IMdlFileImporter
{
public:
	using FProgressFunc = TFunction<void(const FString& MsgName, int MaterialIndex)>;

	virtual ~IMdlFileImporter() {}

	virtual bool OpenFile(const FString& InFileName, const UMDLImporterOptions& InImporterOptions) = 0;
	virtual int GetMaterialCountInFile() = 0;
	virtual bool ImportMaterials(UObject* ParentPackage, EObjectFlags Flags, FProgressFunc ProgressFunc = nullptr) = 0;
	virtual TMap<FString, UMaterialInterface*> GetCreatedMaterials() = 0;
	virtual bool Reimport(const FString& InFileName, const UMDLImporterOptions& InImporterOptions, UMaterialInterface* OutMaterial) = 0;

	virtual const TArray<MDLImporterLogging::FLogMessage> GetLogMessages() = 0;

    static TUniquePtr<IMdlFileImporter> Create();
};
