// Copyright Epic Games, Inc. All Rights Reserved.

#include "MDLOptionsWindow.h"

#include "DetailsViewArgs.h"
#include "IDetailsView.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"

#define LOCTEXT_NAMESPACE "MDLOptionsWindow"

void SMDLOptionsWindow::Construct(const FArguments& InArgs)
{
	ImportOptions = InArgs._ImportOptions;
	Window = InArgs._WidgetWindow;
	bShouldImport = false;

	TSharedPtr<SBox> DetailsViewBox;
	TSharedPtr<SInlineEditableTextBlock> FileNameLabel;
	const FText VersionText(FText::Format(LOCTEXT("MDLOptionWindow_Version", " Version   {0}"), FText::FromString("1.0")));
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(0.0f, 10.0f)
		.AutoHeight()
		[
			SAssignNew(FileNameLabel, SInlineEditableTextBlock)
			.IsReadOnly(true)
			.Text(InArgs._FileNameText)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SInlineEditableTextBlock)
			.IsReadOnly(true)
			.Text(InArgs._PackagePathText)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)
		[
			SAssignNew(DetailsViewBox, SBox)
			.MinDesiredHeight(320.0f)
			.MinDesiredWidth(450.0f)
		]
		+ SVerticalBox::Slot()
		.MaxHeight(50)
		[
			SNew(SUniformGridPanel)
			.SlotPadding(5)
			+ SUniformGridPanel::Slot(0, 0)
			.HAlign(HAlign_Left)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(5)
				+ SUniformGridPanel::Slot(0, 0)
				.HAlign(HAlign_Left)
				[
					SNew(SInlineEditableTextBlock)
					.IsReadOnly(true)
					.Text(VersionText)
				]
				+ SUniformGridPanel::Slot(1, 0)
				.HAlign(HAlign_Left)
				[
					SNew(SBox)
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.WidthOverride(16)
					.HeightOverride(16)
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FAppStyle::GetBrush("Icons.Help"))
						.OnMouseButtonDown(this, &SMDLOptionsWindow::OnHelp)
					]
				]
			]
			+ SUniformGridPanel::Slot(1, 0)
			.HAlign(HAlign_Right)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(5)
				+ SUniformGridPanel::Slot(0, 0)
				.HAlign(HAlign_Right)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Text(LOCTEXT("MDLOptionWindow_ImportMaterials", "Import"))
					.ToolTipText(LOCTEXT("MDLOptionWindow_ImportMaterials_ToolTip", "Import the file and add to the current Level"))
					.OnClicked(this, &SMDLOptionsWindow::OnImport)
				]
				+ SUniformGridPanel::Slot(1, 0)
				.HAlign(HAlign_Right)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Text(LOCTEXT("MDLOptionWindow_Cancel", "Cancel"))
					.ToolTipText(LOCTEXT("MDLOptionWindow_Cancel_ToolTip", "Cancel importing this file"))
					.OnClicked(this, &SMDLOptionsWindow::OnCancel)
				]
			]
		]
	];

	// Add file's full path and material count as a tooltip
	{
		FString Label = InArgs._FilePathText.ToString();
		if (InArgs._MaterialCount == 1)
		{
			Label += TEXT("\none material");
		}
		else
		{
			Label += TEXT("\n") + FString::FromInt(InArgs._MaterialCount) + TEXT(" materials");
		}
		FileNameLabel->SetToolTipText(FText::FromString(Label));
	}

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bAllowMultipleTopLevelObjects = true;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;

	TSharedPtr<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsViewBox->SetContent(DetailsView.ToSharedRef());
	DetailsView->SetObject(ImportOptions);
}

FReply SMDLOptionsWindow::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) /*override*/
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		return OnCancel();
	}

	return FReply::Unhandled();
}

FReply SMDLOptionsWindow::OnImport()
{
	bShouldImport = true;
	if (Window.IsValid())
	{
		Window.Pin()->RequestDestroyWindow();
	}
	return FReply::Handled();
}


FReply SMDLOptionsWindow::OnCancel()
{
	bShouldImport = false;
	if (Window.IsValid())
	{
		Window.Pin()->RequestDestroyWindow();
	}
	return FReply::Handled();
}

FReply SMDLOptionsWindow::OnHelp(const FGeometry& SenderGeometry, const FPointerEvent& MouseEvent)
{
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
