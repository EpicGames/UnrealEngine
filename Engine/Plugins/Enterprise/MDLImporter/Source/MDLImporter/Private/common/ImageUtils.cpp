// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
this is MDLImporter ImageUtils
NOT the ImageUtils in Engine
=============================================================================*/

#include "ImageUtils.h"
#include "Engine/TextureRenderTarget2D.h"
#include "TextureResource.h"

DEFINE_LOG_CATEGORY_STATIC(LogImageUtils, Log, All);

#define LOCTEXT_NAMESPACE "ImageUtils"

namespace ImageUtilsImpl
{

	// TODO: get rid of all this, use FImage instead.

	bool GetRawData(UTextureRenderTarget2D* TexRT, TArray<uint8>& RawData)
	{
		FRenderTarget* RenderTarget = TexRT->GameThread_GetRenderTargetResource();
		EPixelFormat Format = TexRT->GetFormat();

		int32 ImageBytes = CalculateImageBytes(TexRT->SizeX, TexRT->SizeY, 0, Format);
		RawData.AddUninitialized(ImageBytes);
		bool bReadSuccess = false;
		switch (Format)
		{
		case PF_FloatRGBA:
		{
			TArray<FFloat16Color> FloatColors;
			bReadSuccess = RenderTarget->ReadFloat16Pixels(FloatColors);
			FMemory::Memcpy(RawData.GetData(), FloatColors.GetData(), ImageBytes);
		}
		break;
		case PF_B8G8R8A8:
			bReadSuccess = RenderTarget->ReadPixelsPtr((FColor*)RawData.GetData());
			break;
		}
		if (bReadSuccess == false)
		{
			RawData.Empty();
		}
		return bReadSuccess;
	}

	void SetImageValue(const FVector4f& SrcColor, bool bLinearSpace, float* DstColor, int32 Channels)
	{
		static_assert(sizeof(FVector4f) == 4 * sizeof(float), "INVALID_VALUE");
		check(!bLinearSpace);

		memcpy(DstColor, &SrcColor, sizeof(float) * Channels);
	}

	void SetImageValue(const FVector4f& SrcColor, bool bLinearSpace, uint8* DstColor, int32 Channels)
	{
		if (bLinearSpace)
		{
			// Convert back from linear space to gamma space.
			const FColor GammaColor = FLinearColor(SrcColor).ToFColor(true);
			const uint8 GammaColorRGB[] = {GammaColor.R, GammaColor.G, GammaColor.B, 0};
			const uint8* GammaColorArr = Channels == 4 ? reinterpret_cast<const uint8*>(&GammaColor) : GammaColorRGB;
			for (int32 Channel = 0; Channel < Channels; ++Channel)
			{
				DstColor[Channel] = GammaColorArr[Channel];
			}
		}
		else
		{
			for (int32 Channel = 0; Channel < Channels; ++Channel)
			{
				DstColor[Channel] = FMath::Clamp(FMath::TruncToInt(SrcColor[Channel] * 255.f), 0, 255);
			}
		}
	}

	FColor GetImageGammaColor(const float* SrcColor, int32 Channels)
	{
		check(false);
		return FColor();
	}

	FColor GetImageGammaColor(const uint8* SrcColor, int32 Channels)
	{
		uint8 Color[] = {0, 0, 0, 0};
		for (int32 Channel = 0; Channel < Channels; ++Channel)
		{
			Color[Channel] = SrcColor[Channel];
		}
		return *reinterpret_cast<FColor*>(Color);
	}

	template <typename T>
	void ImageResize(int32 SrcWidth, int32 SrcHeight, int32 SrcChannels, const T* SrcData, int32 DstWidth, int32 DstHeight, T* DstData,
	                     bool bLinearSpace)
	{
		check(SrcChannels > 0 && SrcChannels <= 4);

		// TODO: do not use this, use FImageCore::ResizeImage instead

		for (int32 Index = 0; Index < DstWidth * DstHeight * SrcChannels; ++Index)
		{
			DstData[Index] = (T)0;
		}

		float SrcX;
		float SrcY = 0.f;

		const float StepSizeX = SrcWidth / (float)DstWidth;
		const float StepSizeY = SrcHeight / (float)DstHeight;
		for (int32 Y = 0; Y < DstHeight; Y++)
		{
			const int32 PixelPos = Y * DstWidth;
			SrcX                 = 0.0f;

			for (int32 X = 0; X < DstWidth; X++)
			{
				const float EndX = SrcX + StepSizeX;
				const float EndY = SrcY + StepSizeY;

				// Generate a rectangular region of pixels and then find the average color of the region.
				int32 PosY = FMath::TruncToInt(SrcY + 0.5f);
				PosY       = FMath::Clamp<int32>(PosY, 0, (SrcHeight - 1));

				int32 PosX = FMath::TruncToInt(SrcX + 0.5f);
				PosX       = FMath::Clamp<int32>(PosX, 0, (SrcWidth - 1));

				int32 EndPosY = FMath::TruncToInt(EndY + 0.5f);
				EndPosY       = FMath::Clamp<int32>(EndPosY, 0, (SrcHeight - 1));

				int32 EndPosX = FMath::TruncToInt(EndX + 0.5f);
				EndPosX       = FMath::Clamp<int32>(EndPosX, 0, (SrcWidth - 1));

				// Accumulate color
				FVector4f 	LinearStepColor(0.f, 0.f, 0.f, 0.f);
				int32 		PixelCount = 0;
				if (bLinearSpace)
				{
					for (int32 PixelX = PosX; PixelX <= EndPosX; PixelX++)
					{
						for (int32 PixelY = PosY; PixelY <= EndPosY; PixelY++)
						{
							const int32  StartPixel = (PixelX + PixelY * SrcWidth) * SrcChannels;
							const FColor GammaColor = GetImageGammaColor(&SrcData[StartPixel], SrcChannels);
							// Convert from gamma space to linear space before the addition.
							LinearStepColor += FLinearColor(GammaColor);
							++PixelCount;
						}
					}
				}
				else
				{
					for (int32 PixelX = PosX; PixelX <= EndPosX; PixelX++)
					{
						for (int32 PixelY = PosY; PixelY <= EndPosY; PixelY++)
						{
							const int32 StartPixel = (PixelX + PixelY * SrcWidth) * SrcChannels;
							for (int32 Channel = 0; Channel < SrcChannels; ++Channel)
							{
								LinearStepColor[Channel] += SrcData[StartPixel + Channel];
							}
							++PixelCount;
						}
					}
				}

				// Store the final averaged pixel color value.
				LinearStepColor *= 1.f / PixelCount;
				T* CurrentDstData = &DstData[(PixelPos + X) * SrcChannels];
				SetImageValue(LinearStepColor, bLinearSpace, CurrentDstData, SrcChannels);

				SrcX = EndX;
			}

			SrcY += StepSizeY;
		}
	}
}

namespace Common{

/**
 * Resizes the given image using a simple average filter and stores it in the destination array.
 *
 * TODO: do not use this, use FImageCore::ResizeImage instead
 *
 * @param SrcWidth		Source image width.
 * @param SrcHeight		Source image height.
 * @param SrcData		Source image data.
 * @param DstWidth		Destination image width.
 * @param DstHeight		Destination image height.
 * @param DstData		Destination image data.
 * @param bLinearSpace	If true, convert colors into linear space before interpolating (slower but more accurate)
 */
void ImageResize(int32 SrcWidth, int32 SrcHeight, const TArray<FColor> &SrcData, int32 DstWidth, int32 DstHeight, TArray<FColor> &DstData, bool bLinearSpace )
{
	static_assert(sizeof(FColor) == sizeof(uint32), "INVALID_VALUE");

	DstData.SetNumUninitialized(DstWidth * DstHeight);
	ImageUtilsImpl::ImageResize<uint8>(SrcWidth, SrcHeight, 4, reinterpret_cast<const uint8*>(SrcData.GetData()), DstWidth, DstHeight, reinterpret_cast<uint8*>(DstData.GetData()), bLinearSpace);
}

void ImageResize(int32 SrcWidth, int32 SrcHeight, int32 SrcChannels, const float* SrcData, int32 DstWidth, int32 DstHeight, float* DstData)
{
	ImageUtilsImpl::ImageResize<float>(SrcWidth, SrcHeight, SrcChannels, SrcData, DstWidth, DstHeight, DstData, false);
}

void ImageResize(int32 SrcWidth, int32 SrcHeight, int32 SrcChannels, const uint8* SrcData, int32 DstWidth, int32 DstHeight, uint8* DstData, bool bLinearSpace)
{
	ImageUtilsImpl::ImageResize<uint8>(SrcWidth, SrcHeight, SrcChannels, SrcData, DstWidth, DstHeight, DstData, bLinearSpace);
}

}


#undef LOCTEXT_NAMESPACE
