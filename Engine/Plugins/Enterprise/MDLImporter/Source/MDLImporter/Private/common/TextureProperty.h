// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <Engine/Texture.h>

struct FImage;

namespace Common
{
	enum class ETextureMode
	{
		Color,
		ColorLinear,
		Grayscale,
		Normal,
		NormalGreenInv,
		Displace,
		Bump,
		Other
	};

	struct FTextureProperty
	{
		//@note Can be a texture name or file path(i.e. also contains extension and it's disk path),
		// if it's a file path then texture will imported from a file otherwise from texture source/data.
		FString         Path;
		FImage* Source;

		ETextureMode               Mode;
		TextureMipGenSettings      MipGenSettings;
		TextureCompressionSettings CompressionSettings;
		TextureFilter              Filter;
		TextureGroup               LODGroup;
		TextureAddress             Address;
		bool                       bIsSRGB;
		bool                       bFlipGreenChannel;
		bool                       bCompressionNoAlpha;

		FTextureProperty();

		void SetTextureMode(ETextureMode Mode);
	};

	inline FTextureProperty::FTextureProperty()
	    : bIsSRGB(false)
	    , bFlipGreenChannel(false)
	    , bCompressionNoAlpha(true)
	{
		Source              = nullptr;
		LODGroup            = TEXTUREGROUP_World;
		CompressionSettings = TC_Default;
		MipGenSettings      = TMGS_FromTextureGroup;
		Filter              = TF_Default;
		Address             = TA_Wrap;
	}

	inline void FTextureProperty::SetTextureMode(ETextureMode InMode)
	{
		Mode = InMode;
		switch (Mode)
		{
			case ETextureMode::Color:
				bIsSRGB             = true;
				CompressionSettings = TC_Default;
				LODGroup            = TEXTUREGROUP_World;
				break;
			case ETextureMode::ColorLinear:
				bIsSRGB             = false;
				CompressionSettings = TC_Default;
				LODGroup            = TEXTUREGROUP_World;
				break;
			case ETextureMode::Grayscale:
				CompressionSettings = TC_Grayscale;
				LODGroup            = TEXTUREGROUP_WorldSpecular;
				break;
			case ETextureMode::Bump:
				CompressionSettings = TC_Grayscale;
				LODGroup            = TEXTUREGROUP_WorldNormalMap;
				break;
			case ETextureMode::NormalGreenInv:
				bFlipGreenChannel = true;
			case ETextureMode::Normal:
				CompressionSettings = TC_Normalmap;
				LODGroup            = TEXTUREGROUP_WorldNormalMap;
				break;
			case ETextureMode::Displace:
				bIsSRGB             = false;
				CompressionSettings = TC_HDR;
				LODGroup            = TEXTUREGROUP_WorldNormalMap;
				break;
			default:
				break;
		}
	}
}  // namespace Common
