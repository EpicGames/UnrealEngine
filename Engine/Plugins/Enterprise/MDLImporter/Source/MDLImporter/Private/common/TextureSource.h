// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ImageCore.h"
#include "common/Utility.h"

/**
* TextureSource.h in the MDL importer ; not in the Engine
* 
*/

namespace Common
{
	FORCEINLINE bool IsValueInRange(float Value)
	{
		return Value >= 0.f && Value <= 1.f;
	}

	// Converts a float texture to a corresponding texture source.
	inline FImage* CreateTextureSource(const float* InData, int InWidth, int InHeight, int InChannels, bool bFlipY)
	{
		// @todo Oodle : use FImageView / CopyImage
		//  just make an FImageView on the float * and use TextureSource->Init() from ImageView
		//  this whole function could be 2 lines
		// 
		//	also why are we converting to G8 or RGBA16 here ?
		// just use F32 ETextureSourceFormat ?

		// use 16 bpp for linear textures support(i.e. if more than 1 channel)
		const ERawImageFormat::Type Format = InChannels == 1 ? ERawImageFormat::G8 : ERawImageFormat::RGBA16;
		const int                  Size   = InWidth * InHeight * InChannels;

		FImage* Source = new FImage();
		Source->Init(InWidth, InHeight, 1, Format);

		FImageView SourceView = *Source;
		uint8*      DstBuf = static_cast<uint8*>(SourceView.RawData);
		const float Max8   = TNumericLimits<uint8>::Max();
		const float Max16  = TNumericLimits<uint16>::Max();
		switch (InChannels)
		{
			case 1:
			{
				for (int Y = 0; Y < InHeight; ++Y)
				{
					const int SrcY = bFlipY ? (InHeight - 1 - Y) : Y;
					for (int X = 0; X < InWidth; ++X)
					{
						int DstOffset = Y * InWidth + X;
						int SrcOffset = SrcY * InWidth + X;

						DstBuf[DstOffset] = Saturate(InData[SrcOffset]) * Max8;
					}
				}
				break;
			}
			case 3:
			{
				uint16* Dst = (uint16*)DstBuf;
				for (int Y = 0; Y < InHeight; ++Y)
				{
					const int SrcY = bFlipY ? (InHeight - 1 - Y) : Y;
					for (int X = 0; X < InWidth; ++X)
					{
						int DstOffset = Y * InWidth * 4 + X * 4;
						int SrcOffset = SrcY * InWidth * 3 + X * 3;

						// check(IsValueInRange(InData[SrcOffset]));
						// check(IsValueInRange(InData[SrcOffset + 1]));
						// check(IsValueInRange(InData[SrcOffset + 2]));
						Dst[DstOffset]     = Saturate(InData[SrcOffset]) * Max16;
						Dst[DstOffset + 1] = Saturate(InData[SrcOffset + 1]) * Max16;
						Dst[DstOffset + 2] = Saturate(InData[SrcOffset + 2]) * Max16;
						Dst[DstOffset + 3] = Max16;
					}
				}
				break;
			}
			case 4:
			{
				uint16* Dst = (uint16*)DstBuf;
				for (int Y = 0; Y < InHeight; ++Y)
				{
					const int SrcY = bFlipY ? (InHeight - 1 - Y) : Y;
					for (int X = 0; X < InWidth; ++X)
					{
						int DstOffset = Y * InWidth * 4 + X * 4;
						int SrcOffset = SrcY * InWidth * 4 + X * 4;

						// @todo Oodle: use FColor::QuantizeUNormFloatTo16
						//	or just use FImage
						check(IsValueInRange(InData[SrcOffset]));
						check(IsValueInRange(InData[SrcOffset + 1]));
						check(IsValueInRange(InData[SrcOffset + 2]));
						check(IsValueInRange(InData[SrcOffset + 3]));
						Dst[DstOffset]     = Saturate(InData[SrcOffset]) * Max16;
						Dst[DstOffset + 1] = Saturate(InData[SrcOffset + 1]) * Max16;
						Dst[DstOffset + 2] = Saturate(InData[SrcOffset + 2]) * Max16;
						Dst[DstOffset + 3] = Saturate(InData[SrcOffset + 3]) * Max16;
					}
				}
				break;
			}
			default:
				check(false);
				break;
		}
		
		return Source;
	}
}
