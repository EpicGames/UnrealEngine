// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector.h"

namespace Common
{
	template <typename T>
	T Saturate(T Value);

	int HighestPowerofTwo(int Value);

	// Convert normal values from the interval [-1.0, 1.0] to [0.0, 1.0].
	void RemapNormal(int Size, float* Buffer);
	// @note Also flips green channel.
	void RemapNormalFlipGreen(int Size, float* Buffer);

	//

	template <typename T>
	FORCEINLINE T Saturate(T Value)
	{
		return FMath::Clamp(Value, (T)0, (T)1);
	}

	FORCEINLINE int HighestPowerofTwo(int Value)
	{
		const int P = (int)FMath::CeilToFloat(FMath::Log2(static_cast<float>(Value)));
		return (int)FMath::Pow(2.f, P);
	}

	inline void RemapNormal(int Size, float* Buffer)
	{
		for (int Index = 0; Index < Size; Index += 3)
		{
			FVector Normal(Buffer[Index], Buffer[Index + 1], Buffer[Index + 2]);
			Normal.Normalize();
			Buffer[Index]     = (Normal.X + 1.f) * 0.5f;
			Buffer[Index + 1] = (Normal.Y + 1.f) * 0.5f;
			Buffer[Index + 2] = (Normal.Z + 1.f) * 0.5f;
		}
	}

	inline void RemapNormalFlipGreen(int Size, float* Buffer)
	{
		for (int Index = 0; Index < Size; Index += 3)
		{
			FVector Normal(Buffer[Index], Buffer[Index + 1], Buffer[Index + 2]);
			Normal.Normalize();
			Buffer[Index]     = (Normal.X + 1.f) * 0.5f;
			Buffer[Index + 1] = 1.f - (Normal.Y + 1.f) * 0.5f;
			Buffer[Index + 2] = (Normal.Z + 1.f) * 0.5f;
		}
	}

	void ImageResize(int32 SrcWidth, int32 SrcHeight, const TArray<FColor> &SrcData, int32 DstWidth, int32 DstHeight, TArray<FColor> &DstData, bool bLinearSpace);

	void ImageResize(int32 SrcWidth, int32 SrcHeight, int32 SrcChannels, const float* SrcData, int32 DstWidth, int32 DstHeight, float* DstData);

	void ImageResize(int32 SrcWidth, int32 SrcHeight, int32 SrcChannels, const uint8* SrcData, int32 DstWidth, int32 DstHeight, uint8* DstData, bool bLinearSpace);

}
