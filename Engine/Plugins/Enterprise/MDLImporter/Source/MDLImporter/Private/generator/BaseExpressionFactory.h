// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "common/Logging.h"


class UMaterial;
namespace mi
{
	namespace neuraylib
	{
		class ICompiled_material;
		class IMaterial_definition;
	}
}
namespace Mdl
{
	class FApiContext;
}

namespace Generator
{
	class FBaseExpressionFactory
	{
	public:
		FBaseExpressionFactory();

		void SetCurrentMaterial(const mi::neuraylib::IMaterial_definition& MDLMaterialDefinition,
		                        const mi::neuraylib::ICompiled_material& MDLMaterial, UMaterial& Material);

		TArray<MDLImporterLogging::FLogMessage> GetLogMessages();

		void Cleanup();

		void SetProcessingNormapMap(bool ProcessingNormapMap);

	protected:
		UMaterial*                                 CurrentMaterial;
		const mi::neuraylib::ICompiled_material*   CurrentMDLMaterial;
		const mi::neuraylib::IMaterial_definition* CurrentMDLMaterialDefinition;
		bool bProcessingNormapMap;

		mutable TArray<MDLImporterLogging::FLogMessage>        LogMessages;
	};

	inline FBaseExpressionFactory::FBaseExpressionFactory()
	    : CurrentMaterial(nullptr)
	    , CurrentMDLMaterial(nullptr)
	    , CurrentMDLMaterialDefinition(nullptr)
		, bProcessingNormapMap(false)
	{
	}

	inline void FBaseExpressionFactory::SetCurrentMaterial(const mi::neuraylib::IMaterial_definition& MDLMaterialDefinition,
	                                                       const mi::neuraylib::ICompiled_material& MDLMaterial, UMaterial& Material)
	{
		CurrentMaterial              = &Material;
		CurrentMDLMaterial           = &MDLMaterial;
		CurrentMDLMaterialDefinition = &MDLMaterialDefinition;
	}

	inline TArray<MDLImporterLogging::FLogMessage> FBaseExpressionFactory::GetLogMessages()
	{
		TArray<MDLImporterLogging::FLogMessage> Messages;
		Swap(LogMessages, Messages);
		return Messages;
	}

	inline void FBaseExpressionFactory::Cleanup()
	{
		CurrentMaterial              = nullptr;
		CurrentMDLMaterial           = nullptr;
		CurrentMDLMaterialDefinition = nullptr;
	}

	inline void FBaseExpressionFactory::SetProcessingNormapMap(bool bInProcessingNormapMap)
	{
		bProcessingNormapMap = bInProcessingNormapMap;
	}

}  // namespace Generator
