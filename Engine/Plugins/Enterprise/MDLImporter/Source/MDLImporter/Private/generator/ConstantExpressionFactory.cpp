// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef USE_MDLSDK

#include "ConstantExpressionFactory.h"

#include "Materials/MaterialExpressionConstant.h"
#include "generator/MaterialExpressions.h"
#include "Materials/MaterialExpressionConstant2Vector.h"
#include "generator/MaterialTextureFactory.h"
#include "Materials/MaterialExpressionConstant3Vector.h"
#include "mdl/Utility.h"

#include "Engine/Texture2D.h"

MDLSDK_INCLUDES_START
#include "Materials/MaterialExpressionConstant4Vector.h"
#include "mi/neuraylib/itexture.h"
#include "Materials/MaterialExpressionMakeMaterialAttributes.h"
#include "mi/neuraylib/itransaction.h"
#include "Materials/MaterialExpressionStaticBool.h"
#include "mi/neuraylib/ivalue.h"
#include "Materials/MaterialExpressionTextureObject.h"
#include <mi/base/handle.h>
#include <mi/base/iinterface.h>
#include <mi/base/types.h>
#include <mi/base/uuid.h>
#include <mi/neuraylib/itype.h>
MDLSDK_INCLUDES_END

namespace Generator
{
	template<typename T>
	FMaterialExpressionConnectionList FConstantExpressionFactory::AddNewMaterialExpressionConstantHelper(const mi::base::Handle<const mi::neuraylib::IValue_vector>& Value)
	{
		mi::base::Handle<T> Values[4];
		for (mi::Size i = 0; i < Value->get_size(); ++i)
		{
			auto Handle = mi::base::make_handle(Value->get_value(i));
			Values[i] = mi::base::make_handle(Handle->get_interface<T>());
		}

		switch (Value->get_size())
		{
			case 2:
				return { AddExpression(NewMaterialExpressionConstant(CurrentMaterial, static_cast<float>(Values[0]->get_value()), static_cast<float>(Values[1]->get_value()))) };
			case 3:
				return { AddExpression(NewMaterialExpressionConstant(  //
					CurrentMaterial, static_cast<float>(Values[0]->get_value()), static_cast<float>(Values[1]->get_value()), static_cast<float>(Values[2]->get_value()))) };
			case 4:
				return { AddExpression(NewMaterialExpressionConstant(  //
					CurrentMaterial, static_cast<float>(Values[0]->get_value()), static_cast<float>(Values[1]->get_value()), static_cast<float>(Values[2]->get_value()), static_cast<float>(Values[3]->get_value()))) };
		}
		
		return {};
		
	}

	FConstantExpressionFactory::FConstantExpressionFactory()
	    : TextureFactory(nullptr)
	{
	}

	FMaterialExpressionConnectionList FConstantExpressionFactory::CreateExpression(mi::neuraylib::ITransaction& Transaction,
	                                                                                   const mi::neuraylib::IValue& MDLConstant)
	{
		check(CurrentMDLMaterial);
		check(CurrentMaterial);
		check(TextureFactory);

		const mi::neuraylib::IValue::Kind Kind = MDLConstant.get_kind();
		switch (Kind)
		{
			case mi::neuraylib::IValue::VK_BOOL:
			{
				auto Value = mi::base::make_handle(MDLConstant.get_interface<const mi::neuraylib::IValue_bool>());
				return {AddExpression(NewMaterialExpressionStaticBool(CurrentMaterial, Value->get_value()))};
			}
			case mi::neuraylib::IValue::VK_INT:
			{
				auto Value = mi::base::make_handle(MDLConstant.get_interface<const mi::neuraylib::IValue_int>());
				return {AddExpression(NewMaterialExpressionConstant(CurrentMaterial, (float)Value->get_value()))};
			}
			case mi::neuraylib::IValue::VK_ENUM:
			{
				auto Value = mi::base::make_handle(MDLConstant.get_interface<const mi::neuraylib::IValue_enum>());
				return {AddExpression(NewMaterialExpressionConstant(CurrentMaterial, (float)Value->get_value()))};
			}
			case mi::neuraylib::IValue::VK_FLOAT:
			{
				auto Value = mi::base::make_handle(MDLConstant.get_interface<const mi::neuraylib::IValue_float>());
				return {AddExpression(NewMaterialExpressionConstant(CurrentMaterial, Value->get_value()))};
			}
			case mi::neuraylib::IValue::VK_DOUBLE:
			{
				auto Value = mi::base::make_handle(MDLConstant.get_interface<const mi::neuraylib::IValue_double>());
				return {AddExpression(NewMaterialExpressionConstant(CurrentMaterial, (float)Value->get_value()))};
			}
			case mi::neuraylib::IValue::VK_STRING:
			{
				return {};
			}
			case mi::neuraylib::IValue::VK_VECTOR:
			{
				auto Value = mi::base::make_handle(MDLConstant.get_interface<const mi::neuraylib::IValue_vector>());

				const mi::neuraylib::IType::Kind ElementKind = make_handle(make_handle(Value->get_type())->get_element_type())->get_kind();
				FMaterialExpressionConnectionList Result;
				switch (ElementKind)
				{
					case mi::neuraylib::IType::Kind::TK_BOOL:
						Result = AddNewMaterialExpressionConstantHelper<const mi::neuraylib::IValue_bool>(Value);
						break;
					case mi::neuraylib::IType::Kind::TK_INT:
						Result = AddNewMaterialExpressionConstantHelper<const mi::neuraylib::IValue_int>(Value);
						break;
					case mi::neuraylib::IType::Kind::TK_FLOAT:
						Result = AddNewMaterialExpressionConstantHelper<const mi::neuraylib::IValue_float>(Value);
						break;
					case mi::neuraylib::IType::Kind::TK_DOUBLE:
						Result = AddNewMaterialExpressionConstantHelper<const mi::neuraylib::IValue_double>(Value);
						break;
					default:
						break;
				}

				if (Result.Num() > 0)
				{
					return Result;
				}
				break;
			}
			case mi::neuraylib::IValue::VK_MATRIX:
			{
				auto Value = mi::base::make_handle(MDLConstant.get_interface<const mi::neuraylib::IValue_matrix>());

				FMaterialExpressionConnectionList Result;
				Result.Reserve(Value->get_size());
				for (mi::Size i = 0; i < Value->get_size(); i++)
				{
					auto Handle = mi::base::make_handle(Value->get_value(i));
					Result.Append(CreateExpression(Transaction, *Handle));
				}

				return Result;
			}
			case mi::neuraylib::IValue::VK_COLOR:
			{
				auto Value = mi::base::make_handle(MDLConstant.get_interface<const mi::neuraylib::IValue_color>());
				auto Red   = mi::base::make_handle<const mi::neuraylib::IValue_float>(Value->get_value(0));
				auto Green = mi::base::make_handle<const mi::neuraylib::IValue_float>(Value->get_value(1));
				auto Blue  = mi::base::make_handle<const mi::neuraylib::IValue_float>(Value->get_value(2));
				return {AddExpression(NewMaterialExpressionConstant(CurrentMaterial, Red->get_value(), Green->get_value(), Blue->get_value()))};
			}
			case mi::neuraylib::IValue::VK_ARRAY:
			{
				auto Value = mi::base::make_handle(MDLConstant.get_interface<const mi::neuraylib::IValue_array>());

				FMaterialExpressionConnectionList Result;
				for (mi::Size i = 0; i < Value->get_size(); i++)
				{
					auto Handle = mi::base::make_handle(Value->get_value(i));
					Result.Append(CreateExpression(Transaction, *Handle));
				}

				return Result;
			}
			case mi::neuraylib::IValue::VK_STRUCT:
			{
				auto Value = mi::base::make_handle(MDLConstant.get_interface<const mi::neuraylib::IValue_struct>());

				FMaterialExpressionConnectionList Result;
				Result.Reserve(Value->get_size());
				for (mi::Size i = 0; i < Value->get_size(); i++)
				{
					auto Handle = mi::base::make_handle(Value->get_value(i));
					Result.Append(CreateExpression(Transaction, *Handle));
				}

				return Result;
			}
			case mi::neuraylib::IValue::VK_INVALID_DF:
			{
				return {AddExpression(NewMaterialExpressionMakeMaterialAttributes(CurrentMaterial, {0.0f, 0.0f, 0.0f}, 0.0f, 0.0f, 0.0f))};
			}
			case mi::neuraylib::IValue::VK_TEXTURE:
			{
				auto Handle = mi::base::make_handle(MDLConstant.get_interface<const mi::neuraylib::IValue_texture>());
				const mi::base::Handle<const mi::neuraylib::ITexture> MDLTexture(Transaction.access<mi::neuraylib::ITexture>(Handle->get_value()));

				if (MDLTexture)
				{
					Common::FTextureProperty Property;
					Property.Path = Mdl::Util::GetTextureFileName(MDLTexture.get());
					float Gamma = MDLTexture->get_effective_gamma(0, 0);
					Property.bIsSRGB = Gamma != 1.0f;
					if (bProcessingNormapMap)
					{
						Property.CompressionSettings = TC_Normalmap;
					}
					UTexture2D* Texture = TextureFactory->CreateTexture(CurrentMaterial->GetOuter(), Property, CurrentMaterial->GetFlags());
					return {AddExpression(NewMaterialExpressionTextureObject(CurrentMaterial, Texture))};
				}
				else
				{
					LogMessages.Emplace(MDLImporterLogging::EMessageSeverity::Error, TEXT("Couldn't create texture."));
					return {};
				}
			}
			case mi::neuraylib::IValue::VK_BSDF_MEASUREMENT:
			{
				auto Handle = mi::base::make_handle(MDLConstant.get_interface<const mi::neuraylib::IValue_bsdf_measurement>());

				LogMessages.Emplace(MDLImporterLogging::EMessageSeverity::Error, TEXT("Measured BSDF or BTF textures aren't supported."));

				return {AddExpression(NewMaterialExpressionConstant(CurrentMaterial, 0.0f, 0.0f, 0.0f))};
			}
		}
		ensure(false);
		return {};
	}

	void FConstantExpressionFactory::CleanupMaterialExpressions()
	{
		for (FMaterialExpressionHandle& ExpressionHandle: Expressions)
		{
			if (!ExpressionHandle.Expression->UserCount)
			{
				CurrentMaterial->GetExpressionCollection().RemoveExpression(ExpressionHandle.GetMaterialExpression());
				ExpressionHandle.DestroyExpression();
			}
		}

	}

}  // namespace Generator

#endif
