// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "generator/BaseExpressionFactory.h"
#include "generator/MaterialExpressionConnection.h"
#include "mdl/MdlSdkDefines.h"

MDLSDK_INCLUDES_START

MDLSDK_INCLUDES_END
namespace mi::base { template <class Interface> class Handle; }
namespace mi::neuraylib { class IValue_vector; }


namespace mi
{
	namespace neuraylib
	{
		class IValue;
		class ITransaction;
	}
}

namespace Generator
{
	class FMaterialTextureFactory;

	class FConstantExpressionFactory : public FBaseExpressionFactory
	{
	public:
		FConstantExpressionFactory();

		void SetTextureFactory(FMaterialTextureFactory* Factory);

		FMaterialExpressionConnectionList CreateExpression(mi::neuraylib::ITransaction& Transaction, const mi::neuraylib::IValue& MDLConstant);

		void CleanupMaterialExpressions();

	private:
		FMaterialTextureFactory* TextureFactory;

		FMaterialExpressionHandle AddExpression(UMaterialExpression* Expression)
		{
			return Expressions.Add_GetRef(Expression);
		}

		template<typename T>
		FMaterialExpressionConnectionList AddNewMaterialExpressionConstantHelper(const mi::base::Handle<const mi::neuraylib::IValue_vector>& Value);
		
		TArray<FMaterialExpressionHandle> Expressions;
	};

	inline void FConstantExpressionFactory::SetTextureFactory(FMaterialTextureFactory* Factory)
	{
		TextureFactory = Factory;
	}

}  // namespace Generator
