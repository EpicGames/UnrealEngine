// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialExpressions.h"

#include "Engine/Texture2D.h"
#include "Materials/MaterialExpressionAbs.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialExpressionAdd.h"
#include "Materials/MaterialExpressionAppendVector.h"
#include "Materials/MaterialExpressionArccosine.h"
#include "Materials/MaterialExpressionArctangent2.h"
#include "Materials/MaterialExpressionBreakMaterialAttributes.h"
#include "Materials/MaterialExpressionCameraVectorWS.h"
#include "Materials/MaterialExpressionClamp.h"
#include "Materials/MaterialExpressionComponentMask.h"
#include "Materials/MaterialExpressionConstant.h"
#include "Materials/MaterialExpressionConstant3Vector.h"
#include "Materials/MaterialExpressionConstant4Vector.h"
#include "Materials/MaterialExpressionCosine.h"
#include "Materials/MaterialExpressionCrossProduct.h"
#include "Materials/MaterialExpressionDivide.h"
#include "Materials/MaterialExpressionDotProduct.h"
#include "Materials/MaterialExpressionFloor.h"
#include "Materials/MaterialExpressionFmod.h"
#include "Materials/MaterialExpressionFrac.h"
#include "Materials/MaterialExpressionFresnel.h"
#include "Materials/MaterialExpressionFunctionInput.h"
#include "Materials/MaterialExpressionIf.h"
#include "Materials/MaterialExpressionLinearInterpolate.h"
#include "Materials/MaterialExpressionLogarithm2.h"
#include "Materials/MaterialExpressionMakeMaterialAttributes.h"
#include "Materials/MaterialExpressionMax.h"
#include "Materials/MaterialExpressionMin.h"
#include "Materials/MaterialExpressionMultiply.h"
#include "Materials/MaterialExpressionNormalize.h"
#include "Materials/MaterialExpressionOneMinus.h"
#include "Materials/MaterialExpressionPixelNormalWS.h"
#include "Materials/MaterialExpressionPower.h"
#include "Materials/MaterialExpressionReflectionVectorWS.h"
#include "Materials/MaterialExpressionSaturate.h"
#include "Materials/MaterialExpressionSine.h"
#include "Materials/MaterialExpressionSquareRoot.h"
#include "Materials/MaterialExpressionStaticBool.h"
#include "Materials/MaterialExpressionStaticSwitch.h"
#include "Materials/MaterialExpressionSubtract.h"
#include "Materials/MaterialExpressionTextureCoordinate.h"
#include "Materials/MaterialExpressionTextureObject.h"
#include "Materials/MaterialExpressionTextureProperty.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Materials/MaterialExpressionTime.h"
#include "Materials/MaterialExpressionTransform.h"
#include "Materials/MaterialExpressionTransformPosition.h"
#include "Materials/MaterialExpressionTwoSidedSign.h"
#include "Materials/MaterialExpressionVectorNoise.h"
#include "Materials/MaterialExpressionVertexNormalWS.h"
#include "Materials/MaterialExpressionWorldPosition.h"
#include "generator/MaterialExpressionConnection.h"

namespace Generator
{
	class FBaseFunctionGenerator
	{
		enum class EColorLayerMode
		{
			Blend,
			Add,
			Multiply,
			Screen,
			Overlay,
			Brightness,
			Color,
			Exclusion,
			Average,
			Lighten,
			Darken,
			Sub,
			Negation,
			Difference,
			Softlight,
			ColorDodge,
			Reflect,
			ColorBurn,
			Phoenix,
			Hardlight,
			Pinlight,
			HardMix,
			LinearDodge,
			LinearBurn,
			Spotlight,
			SpotlightBlend,
			Hue,
			Saturation
		};

		enum class ECoordinateSpace
		{
			Internal,
			Object,
			World
		};

		enum class EMonoMode
		{
			Alpha,
			Average,
			Luminance,
			Maximum
		};

		enum class EScatterMode
		{
			Reflect,
			Transmit,
			ReflectTransmit
		};

		enum class ETextureCoordinateSystem
		{
			UVW,
			World,
			Object
		};

		enum EWrapMode
		{
			wrap_clamp,
			wrap_repeat,
			wrap_mirrored_repeat,
			wrap_clip
		};

	public:
		virtual ~FBaseFunctionGenerator() {}

		virtual UMaterialFunction* LoadFunction(const FString& AssetName)                           = 0;
		virtual UMaterialFunction* LoadFunction(const FString& AssetPath, const FString& AssetName) = 0;

		void BaseAbbeNumberIOR(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* MakeFloat3 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat3.MakeFloat3"));

			Function->Description = TEXT("Calculate spectral index of refraction.");

			UMaterialExpressionFunctionInput* IOR =
			    NewMaterialExpressionFunctionInput(Function, TEXT("ior"), EFunctionInputType::FunctionInput_Scalar, 1.5f);
			UMaterialExpressionFunctionInput* AbbeNumber =
			    NewMaterialExpressionFunctionInput(Function, TEXT("abbe_number"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionMaterialFunctionCall* Result = NewMaterialExpressionFunctionCall(Function, MakeFloat3, {IOR, IOR, IOR});

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Result);
		}

		/*static*/ void MathAverage(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("Returns the average of the vector elements.");

			UMaterialExpressionFunctionInput* A = NewMaterialExpressionFunctionInput(Function, TEXT("a"), EFunctionInputType::FunctionInput_Vector3);

			UMaterialExpressionDivide* Average = NewMaterialExpressionDivide(
			    Function,
			    NewMaterialExpressionAdd(Function,
			                             {NewMaterialExpressionComponentMask(Function, A, 1), NewMaterialExpressionComponentMask(Function, A, 2),
			                              NewMaterialExpressionComponentMask(Function, A, 4)}),
			    3.0f);

			NewMaterialExpressionFunctionOutput(Function, TEXT("average"), Average);
		}

		/*static*/ void ImporterPerlinNoise(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialExpressionFunctionInput* Pos =
			    NewMaterialExpressionFunctionInput(Function, TEXT("pos"), EFunctionInputType::FunctionInput_Vector4);
			UMaterialExpressionVectorNoise* Noise = NewMaterialExpressionVectorNoise(Function, NewMaterialExpressionComponentMask(Function, Pos, 7), 2);
			Noise->NoiseFunction = EVectorNoiseFunction::VNF_VectorALU; // Perlin 3D noise

			NewMaterialExpressionFunctionOutput(Function, TEXT("noise"), NewMaterialExpressionComponentMask(Function, Noise, 1));
		}

		void ImporterSummedPerlinNoise(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);

			UMaterialFunction* PerlinNoise = LoadFunction(TEXT("mdlimporter_perlin_noise"));

			UMaterialExpressionFunctionInput* Pos =
			    NewMaterialExpressionFunctionInput(Function, TEXT("pos"), EFunctionInputType::FunctionInput_Vector3);
			UMaterialExpressionFunctionInput* Time =
			    NewMaterialExpressionFunctionInput(Function, TEXT("time"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* Terms =
			    NewMaterialExpressionFunctionInput(Function, TEXT("terms"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* TurbulenceWeight =
			    NewMaterialExpressionFunctionInput(Function, TEXT("turbulence_weight"), EFunctionInputType::FunctionInput_Vector3);
			UMaterialExpressionFunctionInput* AbsNoise =
			    NewMaterialExpressionFunctionInput(Function, TEXT("abs_noise"), EFunctionInputType::FunctionInput_StaticBool);
			UMaterialExpressionFunctionInput* Ridged =
			    NewMaterialExpressionFunctionInput(Function, TEXT("ridged"), EFunctionInputType::FunctionInput_StaticBool);

			UMaterialExpressionAppendVector*         P1        = NewMaterialExpressionAppendVector(Function, Pos, Time);
			UMaterialExpressionStaticSwitch*         Weight1   = NewMaterialExpressionStaticSwitch(Function, Ridged, 0.625f, 1.0f);
			UMaterialExpressionMaterialFunctionCall* Noise1    = NewMaterialExpressionFunctionCall(Function, PerlinNoise, {P1});
			UMaterialExpressionAbs*                  AbsNoise1 = NewMaterialExpressionAbs(Function, Noise1);
			UMaterialExpressionStaticSwitch*         N21       = NewMaterialExpressionStaticSwitch(
                Function, Ridged, NewMaterialExpressionSquare(Function, NewMaterialExpressionOneMinus(Function, AbsNoise1)),
                NewMaterialExpressionStaticSwitch(Function, AbsNoise, AbsNoise1, Noise1));
			UMaterialExpressionMultiply* Sum1 = NewMaterialExpressionMultiply(Function, Weight1, N21);

			UMaterialExpressionAdd*                  P2        = NewMaterialExpressionAdd(Function, P1, P1);
			UMaterialExpressionStaticSwitch*         Prev2     = NewMaterialExpressionStaticSwitch(Function, Ridged, N21, 1.0f);
			UMaterialExpressionMultiply*             Weight2   = NewMaterialExpressionMultiply(Function, Weight1, 0.5f);
			UMaterialExpressionMaterialFunctionCall* Noise2    = NewMaterialExpressionFunctionCall(Function, PerlinNoise, {P2});
			UMaterialExpressionAbs*                  AbsNoise2 = NewMaterialExpressionAbs(Function, Noise2);
			UMaterialExpressionStaticSwitch*         N22       = NewMaterialExpressionStaticSwitch(
                Function, Ridged, NewMaterialExpressionSquare(Function, NewMaterialExpressionOneMinus(Function, AbsNoise2)),
                NewMaterialExpressionStaticSwitch(Function, AbsNoise, AbsNoise2, Noise2));
			UMaterialExpressionAdd* Sum2 = NewMaterialExpressionAdd(Function, Sum1, NewMaterialExpressionMultiply(Function, {Weight2, Prev2, N22}));

			UMaterialExpressionAdd*                  P3        = NewMaterialExpressionAdd(Function, P2, P2);
			UMaterialExpressionStaticSwitch*         Prev3     = NewMaterialExpressionStaticSwitch(Function, Ridged, N22, 1.0f);
			UMaterialExpressionMultiply*             Weight3   = NewMaterialExpressionMultiply(Function, Weight2, 0.5f);
			UMaterialExpressionMaterialFunctionCall* Noise3    = NewMaterialExpressionFunctionCall(Function, PerlinNoise, {P3});
			UMaterialExpressionAbs*                  AbsNoise3 = NewMaterialExpressionAbs(Function, Noise3);
			UMaterialExpressionStaticSwitch*         N23       = NewMaterialExpressionStaticSwitch(
                Function, Ridged, NewMaterialExpressionSquare(Function, NewMaterialExpressionOneMinus(Function, AbsNoise3)),
                NewMaterialExpressionStaticSwitch(Function, AbsNoise, AbsNoise3, Noise3));
			UMaterialExpressionAdd* Sum3 = NewMaterialExpressionAdd(Function, Sum2, NewMaterialExpressionMultiply(Function, {Weight3, Prev3, N23}));

			UMaterialExpressionAdd*                  P4        = NewMaterialExpressionAdd(Function, P3, P3);
			UMaterialExpressionStaticSwitch*         Prev4     = NewMaterialExpressionStaticSwitch(Function, Ridged, N23, 1.0f);
			UMaterialExpressionMultiply*             Weight4   = NewMaterialExpressionMultiply(Function, Weight3, 0.5f);
			UMaterialExpressionMaterialFunctionCall* Noise4    = NewMaterialExpressionFunctionCall(Function, PerlinNoise, {P4});
			UMaterialExpressionAbs*                  AbsNoise4 = NewMaterialExpressionAbs(Function, Noise4);
			UMaterialExpressionStaticSwitch*         N24       = NewMaterialExpressionStaticSwitch(
                Function, Ridged, NewMaterialExpressionSquare(Function, NewMaterialExpressionOneMinus(Function, AbsNoise4)),
                NewMaterialExpressionStaticSwitch(Function, AbsNoise, AbsNoise4, Noise4));
			UMaterialExpressionAdd* Sum4 = NewMaterialExpressionAdd(Function, Sum3, NewMaterialExpressionMultiply(Function, {Weight4, Prev4, N24}));

			UMaterialExpressionAdd*                  P5        = NewMaterialExpressionAdd(Function, P4, P4);
			UMaterialExpressionStaticSwitch*         Prev5     = NewMaterialExpressionStaticSwitch(Function, Ridged, N24, 1.0f);
			UMaterialExpressionMultiply*             Weight5   = NewMaterialExpressionMultiply(Function, Weight4, 0.5f);
			UMaterialExpressionMaterialFunctionCall* Noise5    = NewMaterialExpressionFunctionCall(Function, PerlinNoise, {P5});
			UMaterialExpressionAbs*                  AbsNoise5 = NewMaterialExpressionAbs(Function, Noise5);
			UMaterialExpressionStaticSwitch*         N25       = NewMaterialExpressionStaticSwitch(
                Function, Ridged, NewMaterialExpressionSquare(Function, NewMaterialExpressionOneMinus(Function, AbsNoise5)),
                NewMaterialExpressionStaticSwitch(Function, AbsNoise, AbsNoise5, Noise5));

			UMaterialExpressionAdd* Sum5   = NewMaterialExpressionAdd(Function, Sum4, NewMaterialExpressionMultiply(Function, {Weight5, Prev5, N25}));
			UMaterialExpressionIf*  Sum345 = NewMaterialExpressionIf(Function, Terms, 4.0f, Sum3, Sum4, Sum5);
			UMaterialExpressionIf*  SumAll = NewMaterialExpressionIf(Function, Terms, 2.0f, Sum1, Sum2, Sum345);

			UMaterialExpressionIf* Noise = NewMaterialExpressionIfEqual(
			    Function, TurbulenceWeight, {0.0f, 0.0f, 0.0f}, SumAll,
			    NewMaterialExpressionSine(
			        Function, NewMaterialExpressionAdd(Function, NewMaterialExpressionDotProduct(Function, Pos, TurbulenceWeight), SumAll)));
			UMaterialExpressionStaticSwitch* RidgedCheck = NewMaterialExpressionStaticSwitch(
			    Function, Ridged, Noise, NewMaterialExpressionAdd(Function, NewMaterialExpressionMultiply(Function, Noise, 0.5f), 0.5f));
			UMaterialExpressionStaticSwitch* Sum = NewMaterialExpressionStaticSwitch(Function, AbsNoise, Noise, RidgedCheck);

			NewMaterialExpressionFunctionOutput(Function, TEXT("sum"), Sum);
		}

		void BaseAnisotropyConversion(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* StateNormal          = LoadFunction(TEXT("mdl_state_normal"));
			UMaterialFunction* StateTextureTangentU = LoadFunction(TEXT("mdl_state_texture_tangent_u"));

			Function->Description = TEXT("Convert old anisotropy controls into new ones.");

			UMaterialExpressionFunctionInput* Roughness =
			    NewMaterialExpressionFunctionInput(Function, TEXT("roughness"), EFunctionInputType::FunctionInput_Scalar, 0.5f);
			UMaterialExpressionFunctionInput* Anisotropy =
			    NewMaterialExpressionFunctionInput(Function, TEXT("anisotropy"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* AnisotropyRotation =
			    NewMaterialExpressionFunctionInput(Function, TEXT("anisotropy_rotation"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* TangentU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tangent_u"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentU, { 0 }));
			UMaterialExpressionFunctionInput* MiaAnisotropySemantic =
			    NewMaterialExpressionFunctionInput(Function, TEXT("mia_anisotropy_semantic"), EFunctionInputType::FunctionInput_StaticBool,
			                                       NewMaterialExpressionStaticBool(Function, false));

			const float             MinimalIso = 0.0001f;
			UMaterialExpressionMax* IsoGreater = NewMaterialExpressionMax(
			    Function, NewMaterialExpressionSquareRoot(Function, NewMaterialExpressionDivide(Function, 1.0f, Anisotropy)), MinimalIso);
			UMaterialExpressionMax* IsoLess = NewMaterialExpressionMax(
			    Function, NewMaterialExpressionSquareRoot(Function, NewMaterialExpressionSquareRoot(Function, Anisotropy)), MinimalIso);

			UMaterialExpressionIf* AnisotropicRoughnessU =
			    NewMaterialExpressionIfGreater(Function, Anisotropy, 1.0f, NewMaterialExpressionDivide(Function, Roughness, IsoGreater),
			                                   NewMaterialExpressionMultiply(Function, Roughness, IsoLess));
			UMaterialExpressionStaticSwitch* RoughnessU =
			    NewMaterialExpressionStaticSwitch(Function, MiaAnisotropySemantic, AnisotropicRoughnessU, Roughness);

			UMaterialExpressionIf* AnisotropicRoughnessV =
			    NewMaterialExpressionIfGreater(Function, Anisotropy, 1.0f, NewMaterialExpressionMultiply(Function, Roughness, IsoGreater),
			                                   NewMaterialExpressionDivide(Function, Roughness, IsoLess));
			UMaterialExpressionStaticSwitch* RoughnessV = NewMaterialExpressionStaticSwitch(
			    Function, MiaAnisotropySemantic, AnisotropicRoughnessV,
			    NewMaterialExpressionDivide(
			        Function,
			        Roughness,
			        NewMaterialExpressionIfEqual(Function, Anisotropy, 1.0f, 0.0001f, NewMaterialExpressionOneMinus(Function, Anisotropy))));

			UMaterialExpressionMultiply* RotationAngle = NewMaterialExpressionMultiply(Function, 2.0f * PI, AnisotropyRotation);
			// As Unreal is left-handed, swap StateNormal and TangentU in this cross product to get the TangentV !
			UMaterialExpressionCrossProduct* TangentV =
			    NewMaterialExpressionCrossProduct(Function, TangentU, NewMaterialExpressionFunctionCall(Function, StateNormal, {}));
			UMaterialExpressionSubtract* RotatedTangentU =
			    NewMaterialExpressionSubtract(Function,
			                                  NewMaterialExpressionMultiply(Function, TangentU, NewMaterialExpressionCosine(Function, RotationAngle)),
			                                  NewMaterialExpressionMultiply(Function, TangentV, NewMaterialExpressionSine(Function, RotationAngle)));
			UMaterialExpressionIf* TangentUOut = NewMaterialExpressionIfEqual(Function, AnisotropyRotation, 0.0f, TangentU, RotatedTangentU);

			NewMaterialExpressionFunctionOutput(Function, TEXT("roughness_u"), RoughnessU);
			NewMaterialExpressionFunctionOutput(Function, TEXT("roughness_v"), RoughnessV);
			NewMaterialExpressionFunctionOutput(Function, TEXT("tangent_u"), TangentUOut);
		}

		/*static*/ void BaseArchitecturalGlossToRough(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("Glossiness according to mia material semantic.");

			UMaterialExpressionFunctionInput* Glossiness =
			    NewMaterialExpressionFunctionInput(Function, TEXT("glossiness"), EFunctionInputType::FunctionInput_Scalar);

			UMaterialExpressionIf* Result = NewMaterialExpressionIfLess(
			    Function, Glossiness, 1.0f,
			    NewMaterialExpressionSquareRoot(
			        Function,
			        NewMaterialExpressionMultiply(
			            Function,
			            2.0f,
			            NewMaterialExpressionPower(
			                Function, 2.0f,
			                NewMaterialExpressionSubtract(Function, -4.0f, NewMaterialExpressionMultiply(Function, 14.0f, Glossiness))))),
			    0.0f);

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Result);
		}

		void BaseBlendColorLayers(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 < ArraySize);

			UMaterialFunction* ImporterBlendColors = LoadFunction(TEXT("mdlimporter_blend_colors"));
			UMaterialFunction* MathAverage         = LoadFunction(TEXT("mdl_math_average"));
			UMaterialFunction* MathLuminance       = LoadFunction(TEXT("mdl_math_luminance"));
			UMaterialFunction* MathMaxValue        = LoadFunction(TEXT("mdl_math_max_value"));

			Function->Description = TEXT("Texture layering functionality similar to the functionality known from painting programs.");

			TArray<UMaterialExpressionFunctionInput*> LayerColors, Weights, Modes;
			for (int32 i = 0; i < ArraySize; i++)
			{
				LayerColors.Add(
				    NewMaterialExpressionFunctionInput(Function, TEXT("layer_color"), EFunctionInputType::FunctionInput_Vector3, {0.0f, 0.0f, 0.0f}));
				Weights.Add(NewMaterialExpressionFunctionInput(Function, TEXT("weight"), EFunctionInputType::FunctionInput_Scalar, 1.0f));
				Modes.Add(NewMaterialExpressionFunctionInput(Function, TEXT("mode"), EFunctionInputType::FunctionInput_Scalar,
				                                             (int)EColorLayerMode::Blend));
			}
			if (1 < ArraySize)
			{
				for (int32 i = 0; i < ArraySize; i++)
				{
					FString Appendix          = TEXT("_") + FString::FromInt(i);
					FString Buf               = LayerColors[i]->InputName.ToString() + Appendix;
					LayerColors[i]->InputName = *Buf;
					Buf                       = Weights[i]->InputName.ToString() + Appendix;
					Weights[i]->InputName     = *Buf;
					Buf                       = Modes[i]->InputName.ToString() + Appendix;
					Modes[i]->InputName       = *Buf;
				}
			}

			UMaterialExpressionFunctionInput* Base =
			    NewMaterialExpressionFunctionInput(Function, TEXT("base"), EFunctionInputType::FunctionInput_Vector3, {0.0f, 0.0f, 0.0f});
			UMaterialExpressionFunctionInput* MonoSource =
			    NewMaterialExpressionFunctionInput(Function, TEXT("mono_source"), EFunctionInputType::FunctionInput_Scalar, (int)EMonoMode::Average);

			TArray<UMaterialExpressionComponentMask*> Tints;
			Tints.Add(NewMaterialExpressionComponentMask(
			    Function, NewMaterialExpressionFunctionCall(Function, ImporterBlendColors, {LayerColors[0], Base, Weights[0], Modes[0]}), 7));
			for (int32 i = 1; i < ArraySize; i++)
			{
				Tints.Add(NewMaterialExpressionComponentMask(
				    Function, NewMaterialExpressionFunctionCall(Function, ImporterBlendColors, {LayerColors[i], Tints[i - 1], Weights[i], Modes[i]}),
				    7));
			}

			UMaterialExpressionMaterialFunctionCall* Average   = NewMaterialExpressionFunctionCall(Function, MathAverage, {Tints.Last()});
			UMaterialExpressionMaterialFunctionCall* Luminance = NewMaterialExpressionFunctionCall(Function, MathLuminance, {Tints.Last()});
			UMaterialExpressionMaterialFunctionCall* Maximum   = NewMaterialExpressionFunctionCall(Function, MathMaxValue, {Tints.Last()});
			UMaterialExpressionIf*                   Mono = NewMaterialExpressionSwitch(Function, MonoSource, {Maximum, Average, Luminance, Maximum});

			NewMaterialExpressionFunctionOutput(Function, TEXT("tint"), Tints.Last());
			NewMaterialExpressionFunctionOutput(Function, TEXT("mono"), Mono);
		}

		void BaseCheckerBumpTexture(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* ImporterEvalChecker    = LoadFunction(TEXT("mdlimporter_eval_checker"));
			UMaterialFunction* StateNormal            = LoadFunction(TEXT("mdl_state_normal"));
			UMaterialFunction* StateTextureCoordinate = LoadFunction(TEXT("mdl_state_texture_coordinate"));
			UMaterialFunction* StateTextureTangentU   = LoadFunction(TEXT("mdl_state_texture_tangent_u"));
			UMaterialFunction* StateTextureTangentV   = LoadFunction(TEXT("mdl_state_texture_tangent_v"));

			UMaterialExpressionFunctionInput* UVWPosition =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.position"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureCoordinate, {}));
			UMaterialExpressionFunctionInput* UVWTangentU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.tangent_u"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentU, { 0 }));
			UMaterialExpressionFunctionInput* UVWTangentV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.tangent_v"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentV, { 0 }));
			UMaterialExpressionFunctionInput* Factor =
			    NewMaterialExpressionFunctionInput(Function, TEXT("factor"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* Blur =
			    NewMaterialExpressionFunctionInput(Function, TEXT("blur"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* CheckerPosition =
			    NewMaterialExpressionFunctionInput(Function, TEXT("checker_position"), EFunctionInputType::FunctionInput_Scalar, 0.5f);
			UMaterialExpressionFunctionInput* Normal = NewMaterialExpressionFunctionInput(
			    Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3, NewMaterialExpressionFunctionCall(Function, StateNormal, {}));

			float Delta = 0.025f;  // magic, looks good with this value

			UMaterialExpressionMaterialFunctionCall* Result0 =
			    NewMaterialExpressionFunctionCall(Function, ImporterEvalChecker, {UVWPosition, CheckerPosition, Blur});
			UMaterialExpressionMaterialFunctionCall* Result1 = NewMaterialExpressionFunctionCall(
			    Function, ImporterEvalChecker,
			    {NewMaterialExpressionAdd(Function, UVWPosition, NewMaterialExpressionConstant(Function, Delta, 0.0f, 0.0f)), CheckerPosition, Blur});
			UMaterialExpressionMaterialFunctionCall* Result2 = NewMaterialExpressionFunctionCall(
			    Function, ImporterEvalChecker,
			    {NewMaterialExpressionAdd(Function, UVWPosition, NewMaterialExpressionConstant(Function, 0.0f, Delta, 0.0f)), CheckerPosition, Blur});
			UMaterialExpressionMaterialFunctionCall* Result3 = NewMaterialExpressionFunctionCall(
			    Function, ImporterEvalChecker,
			    {NewMaterialExpressionAdd(Function, UVWPosition, NewMaterialExpressionConstant(Function, 0.0f, 0.0f, Delta)), CheckerPosition, Blur});

			UMaterialExpressionNormalize* CalculatedNormal = NewMaterialExpressionNormalize(
			    Function,
			    NewMaterialExpressionAdd(
			        Function,
			        {NewMaterialExpressionMultiply(
			             Function,
			             Normal,
			             NewMaterialExpressionAdd(
			                 Function,
			                 NewMaterialExpressionMultiply(
			                     Function, NewMaterialExpressionAbs(Function, NewMaterialExpressionSubtract(Function, Result3, Result0)), Factor),
			                 1.0f)),
			         NewMaterialExpressionMultiply(Function, {UVWTangentU, NewMaterialExpressionSubtract(Function, Result1, Result0), Factor}),
			         NewMaterialExpressionMultiply(Function, {UVWTangentV, NewMaterialExpressionSubtract(Function, Result2, Result0)})}));
			UMaterialExpressionIf* Result = NewMaterialExpressionIfEqual(Function, Factor, 0.0f, Normal, CalculatedNormal);

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Result);
		}

		void BaseCheckerTexture(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* ImporterEvalChecker    = LoadFunction(TEXT("mdlimporter_eval_checker"));
			UMaterialFunction* MathLuminance          = LoadFunction(TEXT("mdl_math_luminance"));
			UMaterialFunction* StateTextureCoordinate = LoadFunction(TEXT("mdl_state_texture_coordinate"));
			UMaterialFunction* StateTextureTangentU   = LoadFunction(TEXT("mdl_state_texture_tangent_u"));
			UMaterialFunction* StateTextureTangentV   = LoadFunction(TEXT("mdl_state_texture_tangent_v"));

			Function->Description = TEXT("3D color checker pattern.");

			UMaterialExpressionFunctionInput* UVWPosition =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.position"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureCoordinate, {}));
			UMaterialExpressionFunctionInput* UVWTangentU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.tangent_u"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentU, { 0 }));
			UMaterialExpressionFunctionInput* UVWTangentV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.tangent_v"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentV, { 0 }));
			UMaterialExpressionFunctionInput* Color1 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("color1"), EFunctionInputType::FunctionInput_Vector3, {0.0f, 0.0f, 0.0f});
			UMaterialExpressionFunctionInput* Color2 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("color2"), EFunctionInputType::FunctionInput_Vector3, {1.0f, 1.0f, 1.0f});
			UMaterialExpressionFunctionInput* Blur =
			    NewMaterialExpressionFunctionInput(Function, TEXT("blur"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* CheckerPosition =
			    NewMaterialExpressionFunctionInput(Function, TEXT("checker_position"), EFunctionInputType::FunctionInput_Scalar, 0.5f);

			UMaterialExpressionMaterialFunctionCall* Alpha =
			    NewMaterialExpressionFunctionCall(Function, ImporterEvalChecker, {UVWPosition, CheckerPosition, Blur});
			UMaterialExpressionLinearInterpolate*    Tint = NewMaterialExpressionLinearInterpolate(Function, Color2, Color1, Alpha);
			UMaterialExpressionMaterialFunctionCall* Mono = NewMaterialExpressionFunctionCall(Function, MathLuminance, {Tint});

			NewMaterialExpressionFunctionOutput(Function, TEXT("tint"), Tint);
			NewMaterialExpressionFunctionOutput(Function, TEXT("mono"), Mono);
		}

		void BaseCoordinateProjection(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* ImporterComputeCubicTransform      = LoadFunction(TEXT("mdlimporter_compute_cubic_transform"));
			UMaterialFunction* ImporterComputeCylindricTransform  = LoadFunction(TEXT("mdlimporter_compute_cylindric_transform"));
			UMaterialFunction* ImporterComputeSphericTransform    = LoadFunction(TEXT("mdlimporter_compute_spheric_transform"));
			UMaterialFunction* ImporterComputeTangentsTransformed = LoadFunction(TEXT("mdlimporter_compute_tangents_transformed"));
			UMaterialFunction* MathMultiplyFloat4x4Float4         = LoadFunction(TEXT("mdl_math_multiply_float4x4_float4"));
			UMaterialFunction* StateNormal                        = LoadFunction(TEXT("mdl_state_normal"));
			UMaterialFunction* StatePosition                      = LoadFunction(TEXT("mdl_state_position"));
			UMaterialFunction* StateTextureCoordinate             = LoadFunction(TEXT("mdl_state_texture_coordinate"));
			UMaterialFunction* StateTransformPoint                = LoadFunction(TEXT("mdl_state_transform_point"));
			UMaterialFunction* StateTransformVector               = LoadFunction(TEXT("mdl_state_transform_vector"));

			enum projection_mode
			{
				projection_invalid,
				projection_cubic,
				projection_spherical,
				projection_cylindrical,
				projection_infinite_cylindrical,
				projection_planar,
				projection_spherical_normalized,
				projection_cylindrical_normalized,
				projection_infinite_cylindrical_normalized,
				projection_tri_planar
			};

			UMaterialExpressionFunctionInput* CoordinateSystem = NewMaterialExpressionFunctionInput(
			    Function, TEXT("coordinate_system"), EFunctionInputType::FunctionInput_Scalar, (int)ETextureCoordinateSystem::UVW);
			UMaterialExpressionFunctionInput* TextureSpace =
			    NewMaterialExpressionFunctionInput(Function, TEXT("texture_space"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* ProjectionType =
			    NewMaterialExpressionFunctionInput(Function, TEXT("projection_type"), EFunctionInputType::FunctionInput_Scalar, projection_planar);
			UMaterialExpressionFunctionInput* ProjectionTransform0 = NewMaterialExpressionFunctionInput(
			    Function, TEXT("projection_transform_0"), EFunctionInputType::FunctionInput_Vector4, {1.0f, 0.0f, 0.0f, 0.0f});
			UMaterialExpressionFunctionInput* ProjectionTransform1 = NewMaterialExpressionFunctionInput(
			    Function, TEXT("projection_transform_1"), EFunctionInputType::FunctionInput_Vector4, {0.0f, 1.0f, 0.0f, 0.0f});
			UMaterialExpressionFunctionInput* ProjectionTransform2 = NewMaterialExpressionFunctionInput(
			    Function, TEXT("projection_transform_2"), EFunctionInputType::FunctionInput_Vector4, {0.0f, 0.0f, 1.0f, 0.0f});
			UMaterialExpressionFunctionInput* ProjectionTransform3 = NewMaterialExpressionFunctionInput(
			    Function, TEXT("projection_transform_3"), EFunctionInputType::FunctionInput_Vector4, {0.0f, 0.0f, 0.0f, 1.0f});

			UMaterialExpressionMaterialFunctionCall* StatePositionCall = NewMaterialExpressionFunctionCall(Function, StatePosition, {});
			UMaterialExpressionMaterialFunctionCall* StateNormalCall   = NewMaterialExpressionFunctionCall(Function, StateNormal, {});

			UMaterialExpressionIf* IntPos = NewMaterialExpressionSwitch(
			    Function, CoordinateSystem,
			    {NewMaterialExpressionFunctionCall(Function, StateTextureCoordinate, {}),
			     NewMaterialExpressionFunctionCall(Function, StateTransformPoint,
			                                       {(int)ECoordinateSpace::Internal, (int)ECoordinateSpace::World, StatePositionCall}),
			     NewMaterialExpressionFunctionCall(Function, StateTransformPoint,
			                                       {(int)ECoordinateSpace::Internal, (int)ECoordinateSpace::Object, StatePositionCall})});

			UMaterialExpressionIf* CoordinateSpace = NewMaterialExpressionIfEqual(Function, CoordinateSystem, (int)ETextureCoordinateSystem::World,
			                                                                      (int)ECoordinateSpace::World, (int)ECoordinateSpace::Object);
			UMaterialExpressionMaterialFunctionCall* Normal = NewMaterialExpressionFunctionCall(
			    Function, StateTransformVector,
			    {(int)ECoordinateSpace::Internal, CoordinateSpace,
			     StateNormalCall});  // would need to transform a normal here! but I'm not aware of such a method in Unreal!

			UMaterialExpressionMaterialFunctionCall* Cubic = NewMaterialExpressionFunctionCall(
			    Function, ImporterComputeCubicTransform,
			    {ProjectionTransform0, ProjectionTransform1, ProjectionTransform2, ProjectionTransform3, Normal, IntPos});
			UMaterialExpressionMaterialFunctionCall* Spherical = NewMaterialExpressionFunctionCall(
			    Function, ImporterComputeSphericTransform,
			    {ProjectionTransform0, ProjectionTransform1, ProjectionTransform2, ProjectionTransform3, IntPos, false});
			UMaterialExpressionMaterialFunctionCall* Cylindrical = NewMaterialExpressionFunctionCall(
			    Function, ImporterComputeCylindricTransform,
			    {ProjectionTransform0, ProjectionTransform1, ProjectionTransform2, ProjectionTransform3, Normal, IntPos, false, false});
			UMaterialExpressionMaterialFunctionCall* InfiniteCylindrical = NewMaterialExpressionFunctionCall(
			    Function, ImporterComputeCylindricTransform,
			    {ProjectionTransform0, ProjectionTransform1, ProjectionTransform2, ProjectionTransform3, Normal, IntPos, true, false});
			UMaterialExpressionMaterialFunctionCall* SphericalNormalized = NewMaterialExpressionFunctionCall(
			    Function, ImporterComputeSphericTransform,
			    {ProjectionTransform0, ProjectionTransform1, ProjectionTransform2, ProjectionTransform3, IntPos, true});
			UMaterialExpressionMaterialFunctionCall* CylindricalNormalized = NewMaterialExpressionFunctionCall(
			    Function, ImporterComputeCylindricTransform,
			    {ProjectionTransform0, ProjectionTransform1, ProjectionTransform2, ProjectionTransform3, Normal, IntPos, false, true});
			UMaterialExpressionMaterialFunctionCall* InfiniteCylindricalNormalized = NewMaterialExpressionFunctionCall(
			    Function, ImporterComputeCylindricTransform,
			    {ProjectionTransform0, ProjectionTransform1, ProjectionTransform2, ProjectionTransform3, Normal, IntPos, true, true});
			UMaterialExpression* TriPlanar = Cubic;

			UMaterialExpressionIf* FinalTransform0 = NewMaterialExpressionSwitch(Function, ProjectionType,
			                                                                     {{},
			                                                                      {Cubic, 0},
			                                                                      {Spherical, 0},
			                                                                      {Cylindrical, 0},
			                                                                      {InfiniteCylindrical, 0},
			                                                                      ProjectionTransform0,
			                                                                      {SphericalNormalized, 0},
			                                                                      {CylindricalNormalized, 0},
			                                                                      {InfiniteCylindricalNormalized, 0},
			                                                                      {TriPlanar, 0}});
			UMaterialExpressionIf* FinalTransform1 = NewMaterialExpressionSwitch(Function, ProjectionType,
			                                                                     {{},
			                                                                      {Cubic, 1},
			                                                                      {Spherical, 1},
			                                                                      {Cylindrical, 1},
			                                                                      {InfiniteCylindrical, 1},
			                                                                      ProjectionTransform1,
			                                                                      {SphericalNormalized, 1},
			                                                                      {CylindricalNormalized, 1},
			                                                                      {InfiniteCylindricalNormalized, 1},
			                                                                      {TriPlanar, 1}});
			UMaterialExpressionIf* FinalTransform2 = NewMaterialExpressionSwitch(Function, ProjectionType,
			                                                                     {{},
			                                                                      {Cubic, 2},
			                                                                      {Spherical, 2},
			                                                                      {Cylindrical, 2},
			                                                                      {InfiniteCylindrical, 2},
			                                                                      ProjectionTransform2,
			                                                                      {SphericalNormalized, 2},
			                                                                      {CylindricalNormalized, 2},
			                                                                      {InfiniteCylindricalNormalized, 2},
			                                                                      {TriPlanar, 2}});
			UMaterialExpressionIf* FinalTransform3 = NewMaterialExpressionSwitch(Function, ProjectionType,
			                                                                     {{},
			                                                                      {Cubic, 3},
			                                                                      {Spherical, 3},
			                                                                      {Cylindrical, 3},
			                                                                      {InfiniteCylindrical, 3},
			                                                                      ProjectionTransform3,
			                                                                      {SphericalNormalized, 3},
			                                                                      {CylindricalNormalized, 3},
			                                                                      {InfiniteCylindricalNormalized, 3},
			                                                                      {TriPlanar, 3}});

			UMaterialExpressionMaterialFunctionCall* RPos = NewMaterialExpressionFunctionCall(
			    Function, MathMultiplyFloat4x4Float4,
			    {FinalTransform0, FinalTransform1, FinalTransform2, FinalTransform3, NewMaterialExpressionAppendVector(Function, IntPos, 1.0f)});
			UMaterialExpressionAppendVector* Position =
			    NewMaterialExpressionAppendVector(Function, NewMaterialExpressionComponentMask(Function, RPos, 3), 0.0f);

			UMaterialExpressionMaterialFunctionCall* Tangents = NewMaterialExpressionFunctionCall(
			    Function, ImporterComputeTangentsTransformed,
			    {CoordinateSystem, NewMaterialExpressionComponentMask(Function, FinalTransform0, 7),
			     NewMaterialExpressionComponentMask(Function, FinalTransform1, 7), NewMaterialExpressionComponentMask(Function, FinalTransform2, 7)});

			NewMaterialExpressionFunctionOutput(Function, TEXT("position"), Position);
			NewMaterialExpressionFunctionOutput(Function, TEXT("tangent_u"), {Tangents, 1});
			NewMaterialExpressionFunctionOutput(Function, TEXT("tangent_v"), {Tangents, 2});
		}

		void BaseCoordinateSource(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* ImporterComputeTangents = LoadFunction(TEXT("mdlimporter_compute_tangents"));
			UMaterialFunction* StatePosition           = LoadFunction(TEXT("mdl_state_position"));
			UMaterialFunction* StateTextureCoordinate  = LoadFunction(TEXT("mdl_state_texture_coordinate"));
			UMaterialFunction* StateTextureTangentU    = LoadFunction(TEXT("mdl_state_texture_tangent_u"));
			UMaterialFunction* StateTextureTangentV    = LoadFunction(TEXT("mdl_state_texture_tangent_v"));
			UMaterialFunction* StateTransformPoint     = LoadFunction(TEXT("mdl_state_transform_point"));

			Function->Description = TEXT("Access to world coordinates, object coordinates or specifically defined texture spaces.");

			UMaterialExpressionFunctionInput* CoordinateSystem = NewMaterialExpressionFunctionInput(
			    Function, TEXT("coordinate_system"), EFunctionInputType::FunctionInput_Scalar, (int)ETextureCoordinateSystem::UVW);
#if defined(USE_WORLD_ALIGNED_TEXTURES)
#if defined(USE_WAT_AS_SCALAR)
			UMaterialExpressionFunctionInput* UseWorldAlignedTexture =
			    NewMaterialExpressionFunctionInput(Function, TEXT("use_world_aligned_texture"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionIf* CoordinateSelection = NewMaterialExpressionIfEqual(Function, UseWorldAlignedTexture, 1.0f, 1.0f, CoordinateSystem);
#else
			UMaterialExpressionFunctionInput* UseWorldAlignedTexture =
			    NewMaterialExpressionFunctionInput(Function, TEXT("use_world_aligned_texture"), EFunctionInputType::FunctionInput_StaticBool);
			UMaterialExpressionStaticSwitch* CoordinateSelection =
			    NewMaterialExpressionStaticSwitch(Function, UseWorldAlignedTexture, 1.0f, CoordinateSystem);
#endif
#else
			UMaterialExpressionFunctionInput* CoordinateSelection = CoordinateSystem;
#endif

			UMaterialExpressionMaterialFunctionCall* StatePositionCall = NewMaterialExpressionFunctionCall(Function, StatePosition, {});
			UMaterialExpressionIf*                   Position          = NewMaterialExpressionIf(
                Function, CoordinateSelection, (int)ETextureCoordinateSystem::World,
                NewMaterialExpressionFunctionCall(Function, StateTextureCoordinate, {}),
                NewMaterialExpressionDivide(Function, StatePositionCall, 100.0f),
                NewMaterialExpressionFunctionCall(Function, StateTransformPoint,
                                                  {(int)ECoordinateSpace::Internal, (int)ECoordinateSpace::Object, StatePositionCall}));

			UMaterialExpressionMaterialFunctionCall* TangentsWorld =
			    NewMaterialExpressionFunctionCall(Function, ImporterComputeTangents, {(int)ECoordinateSpace::World});
			UMaterialExpressionMaterialFunctionCall* TangentsObject =
			    NewMaterialExpressionFunctionCall(Function, ImporterComputeTangents, {(int)ECoordinateSpace::Object});

			UMaterialExpressionIf* TangentU = NewMaterialExpressionSwitch(
			    Function, CoordinateSelection,
			    {NewMaterialExpressionFunctionCall(Function, StateTextureTangentU, {}), {TangentsWorld, 1}, {TangentsObject, 1}});
			UMaterialExpressionIf* TangentV = NewMaterialExpressionSwitch(
			    Function, CoordinateSelection,
			    {NewMaterialExpressionFunctionCall(Function, StateTextureTangentV, {}), {TangentsWorld, 2}, {TangentsObject, 2}});

			NewMaterialExpressionFunctionOutput(Function, TEXT("position"), Position);
			NewMaterialExpressionFunctionOutput(Function, TEXT("tangent_u"), TangentU);
			NewMaterialExpressionFunctionOutput(Function, TEXT("tangent_v"), TangentV);
		}

		void BaseFileBumpTexture(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* ImporterMonoMode = LoadFunction(TEXT("mdlimporter_mono_mode"));
			UMaterialFunction* MakeFloat2 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat2.MakeFloat2"));
			UMaterialFunction* MakeFloat4 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat4.MakeFloat4"));
			UMaterialFunction* StateNormal            = LoadFunction(TEXT("mdl_state_normal"));
			UMaterialFunction* StateTextureCoordinate = LoadFunction(TEXT("mdl_state_texture_coordinate"));
			UMaterialFunction* StateTextureTangentU   = LoadFunction(TEXT("mdl_state_texture_tangent_u"));
			UMaterialFunction* StateTextureTangentV   = LoadFunction(TEXT("mdl_state_texture_tangent_v"));
			UMaterialFunction* TexLookupFloat4        = LoadFunction(TEXT("mdl_tex_lookup_float4"));
#if defined(USE_WORLD_ALIGNED_TEXTURES)
			UMaterialFunction* WorldAlignedTextureFloat4 = LoadFunction(TEXT("mdlimporter_world_aligned_texture_float4"));
#endif

			Function->Description = TEXT("Computes a normal based on a heightfield-style bump texture.");

			UMaterialExpressionFunctionInput* Texture = NewMaterialExpressionFunctionInput(
			    Function, TEXT("texture"), EFunctionInputType::FunctionInput_Texture2D, NewMaterialExpressionTextureObject(Function, nullptr));
			UMaterialExpressionFunctionInput* Factor =
			    NewMaterialExpressionFunctionInput(Function, TEXT("factor"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* BumpSource =
			    NewMaterialExpressionFunctionInput(Function, TEXT("bump_source"), EFunctionInputType::FunctionInput_Scalar, (int)EMonoMode::Average);
			UMaterialExpressionFunctionInput* UVWPosition =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.position"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureCoordinate, {}));
			UMaterialExpressionFunctionInput* UVWTangentU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.tangent_u"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentU, { 0 }));
			UMaterialExpressionFunctionInput* UVWTangentV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.tangent_v"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentV, { 0 }));
			UMaterialExpressionFunctionInput* CropU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("crop_u"), EFunctionInputType::FunctionInput_Vector2, {0.0f, 1.0f});
			UMaterialExpressionFunctionInput* CropV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("crop_v"), EFunctionInputType::FunctionInput_Vector2, {0.0f, 1.0f});
			UMaterialExpressionFunctionInput* WrapU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("wrap_u"), EFunctionInputType::FunctionInput_Scalar, wrap_repeat);
			UMaterialExpressionFunctionInput* WrapV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("wrap_v"), EFunctionInputType::FunctionInput_Scalar, wrap_repeat);
			UMaterialExpressionFunctionInput* Normal = NewMaterialExpressionFunctionInput(
			    Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3, NewMaterialExpressionFunctionCall(Function, StateNormal, {}));
			UMaterialExpressionFunctionInput* Clip = NewMaterialExpressionFunctionInput(
			    Function, TEXT("clip"), EFunctionInputType::FunctionInput_StaticBool, NewMaterialExpressionStaticBool(Function, false));
			UMaterialExpressionFunctionInput* BSpline = NewMaterialExpressionFunctionInput(
			    Function, TEXT("b_spline"), EFunctionInputType::FunctionInput_StaticBool, NewMaterialExpressionStaticBool(Function, false));
			UMaterialExpressionFunctionInput* AnimationStartTime = NewMaterialExpressionFunctionInput(
			    Function, TEXT("animation_start_time"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* AnimationCrop = NewMaterialExpressionFunctionInput(
				Function, TEXT("animation_crop"), EFunctionInputType::FunctionInput_Vector2, {0.0f, 0.0f});
			UMaterialExpressionFunctionInput* AnimationWrap = NewMaterialExpressionFunctionInput(
			    Function, TEXT("animation_wrap"), EFunctionInputType::FunctionInput_Scalar, wrap_repeat); // wrap_mode
			UMaterialExpressionFunctionInput* AnimationFps = NewMaterialExpressionFunctionInput(
			    Function, TEXT("animation_fps"), EFunctionInputType::FunctionInput_Scalar, 30.0f);

#if defined(USE_WORLD_ALIGNED_TEXTURES)
#if defined(USE_WAT_AS_SCALAR)
			UMaterialExpressionFunctionInput* UseWorldAlignedTexture =
			    NewMaterialExpressionFunctionInput(Function, TEXT("use_world_aligned_texture"), EFunctionInputType::FunctionInput_Scalar);
#else
			UMaterialExpressionFunctionInput* UseWorldAlignedTexture =
			    NewMaterialExpressionFunctionInput(Function, TEXT("use_world_aligned_texture"), EFunctionInputType::FunctionInput_StaticBool);
#endif
#endif

			UMaterialExpressionComponentMask* UVWPositionX = NewMaterialExpressionComponentMask(Function, UVWPosition, 1);
			UMaterialExpressionComponentMask* UVWPositionY = NewMaterialExpressionComponentMask(Function, UVWPosition, 2);

#if 0
			// I think, this is exactly what base.mdl says... but it doesn't work -> created my own version below, using just central differences, ignoring cropping
			UMaterialFunction* ImporterInterpolateTexspace = LoadFunction(TEXT("mdlimporter_interpolate_texspace"));
			UMaterialFunction* ImporterTexremapu2 = LoadFunction(TEXT("mdlimporter_texremapu2"));

			UMaterialExpressionComponentMask* CropUX = NewMaterialExpressionComponentMask(Function, CropU, 1);
			UMaterialExpressionComponentMask* CropUY = NewMaterialExpressionComponentMask(Function, CropU, 2);
			UMaterialExpressionComponentMask* CropVX = NewMaterialExpressionComponentMask(Function, CropV, 1);
			UMaterialExpressionComponentMask* CropVY = NewMaterialExpressionComponentMask(Function, CropV, 2);

			UMaterialExpressionTextureProperty* TextureSize = NewMaterialExpressionTextureProperty(Function, Texture, TMTM_TextureSize);
			UMaterialExpressionMultiply* CropOffset = NewMaterialExpressionMultiply(Function,
				TextureSize, NewMaterialExpressionFunctionCall(Function, MakeFloat2, { CropUX, CropVX }));
			UMaterialExpressionMultiply* CropTexres =
				NewMaterialExpressionMultiply(Function,
					TextureSize,
					NewMaterialExpressionFunctionCall(Function, MakeFloat2,
						{
							NewMaterialExpressionSubtract(Function, CropUY, CropUX),
							NewMaterialExpressionSubtract(Function, CropVY, CropVX)
						}));
			UMaterialExpressionMultiply* Tex2 = NewMaterialExpressionMultiply(Function, NewMaterialExpressionComponentMask(Function, UVWPosition, 3), CropTexres);
			UMaterialExpressionMaterialFunctionCall* Texi0 = NewMaterialExpressionFunctionCall(Function,
				ImporterTexremapu2, { CropTexres, CropOffset, NewMaterialExpressionSubtract(Function, Tex2,{ 1.0f, 1.0f }), WrapU, WrapV });
			UMaterialExpressionMaterialFunctionCall* Texi1 = NewMaterialExpressionFunctionCall(Function,
				ImporterTexremapu2, { CropTexres, CropOffset, Tex2, WrapU, WrapV });
			UMaterialExpressionMaterialFunctionCall* Texi2 = NewMaterialExpressionFunctionCall(Function,
				ImporterTexremapu2, { CropTexres, CropOffset, NewMaterialExpressionAdd(Function, Tex2,{ 1.0f, 1.0f }), WrapU, WrapV });
			UMaterialExpressionMaterialFunctionCall* Texi3 = NewMaterialExpressionFunctionCall(Function,
				ImporterTexremapu2, { CropTexres, CropOffset, NewMaterialExpressionAdd(Function, Tex2,{ 2.0f, 2.0f }), WrapU, WrapV });
			UMaterialExpressionComponentMask* Texi0X = NewMaterialExpressionComponentMask(Function, Texi0, 1);
			UMaterialExpressionComponentMask* Texi0Y = NewMaterialExpressionComponentMask(Function, Texi0, 2);
			UMaterialExpressionComponentMask* Texi1X = NewMaterialExpressionComponentMask(Function, Texi1, 1);
			UMaterialExpressionComponentMask* Texi1Y = NewMaterialExpressionComponentMask(Function, Texi1, 2);
			UMaterialExpressionComponentMask* Texi2X = NewMaterialExpressionComponentMask(Function, Texi2, 1);
			UMaterialExpressionComponentMask* Texi2Y = NewMaterialExpressionComponentMask(Function, Texi2, 2);
			UMaterialExpressionComponentMask* Texi3X = NewMaterialExpressionComponentMask(Function, Texi3, 1);
			UMaterialExpressionComponentMask* Texi3Y = NewMaterialExpressionComponentMask(Function, Texi3, 2);
			UMaterialExpressionSubtract* L = NewMaterialExpressionSubtract(Function, Tex2, NewMaterialExpressionFloor(Function, Tex2));
			UMaterialExpressionMultiply* Lerp =
				NewMaterialExpressionMultiply(Function,
					{
						L,
						L,
						L,
						NewMaterialExpressionAdd(Function,
						NewMaterialExpressionMultiply(Function,
						L,
						NewMaterialExpressionSubtract(Function,
						NewMaterialExpressionMultiply(Function, L, 6.0f),
						15.f)),
						10.0f)
					});
			UMaterialExpressionComponentMask* LerpX = NewMaterialExpressionComponentMask(Function, Lerp, 1);
			UMaterialExpressionComponentMask* LerpY = NewMaterialExpressionComponentMask(Function, Lerp, 2);
			UMaterialExpressionMaterialFunctionCall* ST =
				NewMaterialExpressionFunctionCall(Function, MakeFloat4,
					{
						NewMaterialExpressionMultiply(Function, LerpX, LerpY),
						NewMaterialExpressionMultiply(Function, LerpX, NewMaterialExpressionOneMinus(Function, LerpY)),
						NewMaterialExpressionMultiply(Function, NewMaterialExpressionOneMinus(Function, LerpX), NewMaterialExpressionOneMinus(Function, LerpY)),
						NewMaterialExpressionMultiply(Function, NewMaterialExpressionOneMinus(Function, LerpX), LerpY)
					});
			UMaterialExpressionMaterialFunctionCall* BumpTexspace =
				NewMaterialExpressionFunctionCall(Function, MakeFloat2,
					{
						NewMaterialExpressionMultiply(Function,
						NewMaterialExpressionSubtract(Function,
						NewMaterialExpressionFunctionCall(Function,
						ImporterInterpolateTexspace,{ Texture, ST, NewMaterialExpressionFunctionCall(Function, MakeFloat4,{ Texi0X, Texi1Y, Texi1X, Texi2Y }), BumpSource }),
						NewMaterialExpressionFunctionCall(Function,
						ImporterInterpolateTexspace,{ Texture, ST, NewMaterialExpressionFunctionCall(Function, MakeFloat4,{ Texi2X, Texi1Y, Texi3X, Texi2Y }), BumpSource })),
						Factor),
						NewMaterialExpressionMultiply(Function,
						NewMaterialExpressionSubtract(Function,
						NewMaterialExpressionFunctionCall(Function,
						ImporterInterpolateTexspace,{ Texture, ST, NewMaterialExpressionFunctionCall(Function, MakeFloat4,{ Texi1X, Texi0Y, Texi2X, Texi1Y }), BumpSource }),
						NewMaterialExpressionFunctionCall(Function,
						ImporterInterpolateTexspace,{ Texture, ST, NewMaterialExpressionFunctionCall(Function, MakeFloat4,{ Texi1X, Texi2Y, Texi2X, Texi3Y }), BumpSource })),
						Factor)
					});

			UMaterialExpressionNormalize* UnclippedResult =
				NewMaterialExpressionNormalize(Function,
					NewMaterialExpressionAdd(Function,
						{
							Normal,
							NewMaterialExpressionMultiply(Function, UVWTangentU, NewMaterialExpressionComponentMask(Function, BumpTexspace, 1)),
							NewMaterialExpressionMultiply(Function, UVWTangentV, NewMaterialExpressionComponentMask(Function, BumpTexspace, 2))
						}));

#else

			UMaterialExpressionTextureProperty* TextureScale  = NewMaterialExpressionTextureProperty(Function, Texture, TMTM_TexelSize);
			UMaterialExpressionComponentMask*   OneOverWidth  = NewMaterialExpressionComponentMask(Function, TextureScale, 1);
			UMaterialExpressionComponentMask*   OneOverHeight = NewMaterialExpressionComponentMask(Function, TextureScale, 2);

			UMaterialExpressionComponentMask* Center = NewMaterialExpressionComponentMask(Function, UVWPosition, 3);

			UMaterialExpressionMaterialFunctionCall* OffsetU         = NewMaterialExpressionFunctionCall(Function, MakeFloat2, {OneOverWidth, 0.0f});
			UMaterialExpressionMaterialFunctionCall* TexCenterMinusU = NewMaterialExpressionFunctionCall(
			    Function, TexLookupFloat4, {Texture, NewMaterialExpressionSubtract(Function, Center, OffsetU), WrapU, WrapV, CropU, CropV});
			UMaterialExpressionMaterialFunctionCall* TexCenterPlusU = NewMaterialExpressionFunctionCall(
			    Function, TexLookupFloat4, {Texture, NewMaterialExpressionAdd(Function, Center, OffsetU), WrapU, WrapV, CropU, CropV});

#if defined(USE_WORLD_ALIGNED_TEXTURES)
			UMaterialExpressionMultiply*             OffsetTangentU         = NewMaterialExpressionMultiply(Function, OneOverWidth, UVWTangentU);
			UMaterialExpressionMaterialFunctionCall* TexCenterMinusTangentU = NewMaterialExpressionFunctionCall(
			    Function, WorldAlignedTextureFloat4, {Texture, NewMaterialExpressionSubtract(Function, UVWPosition, OffsetTangentU)});
			UMaterialExpressionMaterialFunctionCall* TexCenterPlusTangentU = NewMaterialExpressionFunctionCall(
			    Function, WorldAlignedTextureFloat4, {Texture, NewMaterialExpressionAdd(Function, UVWPosition, OffsetTangentU)});

#if defined(USE_WAT_AS_SCALAR)
			UMaterialExpressionIf* CentralDifferenceU = NewMaterialExpressionIfEqual(
			    Function, UseWorldAlignedTexture, 1.0f,
#else
			UMaterialExpressionStaticSwitch* CentralDifferenceU = NewMaterialExpressionStaticSwitch(
			    Function, UseWorldAlignedTexture,
#endif
			    NewMaterialExpressionSubtract(Function,
			                                  NewMaterialExpressionFunctionCall(Function, ImporterMonoMode, {TexCenterMinusTangentU, BumpSource}),
			                                  NewMaterialExpressionFunctionCall(Function, ImporterMonoMode, {TexCenterPlusTangentU, BumpSource})),
			    NewMaterialExpressionSubtract(Function,
			                                  NewMaterialExpressionFunctionCall(Function, ImporterMonoMode, {TexCenterMinusU, BumpSource}),
			                                  NewMaterialExpressionFunctionCall(Function, ImporterMonoMode, {TexCenterPlusU, BumpSource})));
#else
			UMaterialExpressionSubtract* CentralDifferenceU =
			    NewMaterialExpressionSubtract(Function,
			                                  NewMaterialExpressionFunctionCall(Function, ImporterMonoMode, {TexCenterMinusU, BumpSource}),
			                                  NewMaterialExpressionFunctionCall(Function, ImporterMonoMode, {TexCenterPlusU, BumpSource}));
#endif
			UMaterialExpressionMultiply* FactorU = NewMaterialExpressionMultiply(Function, Factor, CentralDifferenceU);

			UMaterialExpressionMaterialFunctionCall* OffsetV         = NewMaterialExpressionFunctionCall(Function, MakeFloat2, {0.0f, OneOverHeight});
			UMaterialExpressionMaterialFunctionCall* TexCenterMinusV = NewMaterialExpressionFunctionCall(
			    Function, TexLookupFloat4, {Texture, NewMaterialExpressionSubtract(Function, Center, OffsetV), WrapU, WrapV, CropU, CropV});
			UMaterialExpressionMaterialFunctionCall* TexCenterPlusV = NewMaterialExpressionFunctionCall(
			    Function, TexLookupFloat4, {Texture, NewMaterialExpressionAdd(Function, Center, OffsetV), WrapU, WrapV, CropU, CropV});

#if defined(USE_WORLD_ALIGNED_TEXTURES)
			UMaterialExpressionMultiply*             OffsetTangentV         = NewMaterialExpressionMultiply(Function, OneOverHeight, UVWTangentV);
			UMaterialExpressionMaterialFunctionCall* TexCenterMinusTangentV = NewMaterialExpressionFunctionCall(
			    Function, WorldAlignedTextureFloat4, {Texture, NewMaterialExpressionSubtract(Function, UVWPosition, OffsetTangentV)});
			UMaterialExpressionMaterialFunctionCall* TexCenterPlusTangentV = NewMaterialExpressionFunctionCall(
			    Function, WorldAlignedTextureFloat4, {Texture, NewMaterialExpressionAdd(Function, UVWPosition, OffsetTangentV)});

#if defined(USE_WAT_AS_SCALAR)
			UMaterialExpressionIf* CentralDifferenceV = NewMaterialExpressionIfEqual(
			    Function, UseWorldAlignedTexture, 1.0f,
#else
			UMaterialExpressionStaticSwitch* CentralDifferenceV = NewMaterialExpressionStaticSwitch(
			    Function, UseWorldAlignedTexture,
#endif
			    NewMaterialExpressionSubtract(Function,
			                                  NewMaterialExpressionFunctionCall(Function, ImporterMonoMode, {TexCenterMinusTangentV, BumpSource}),
			                                  NewMaterialExpressionFunctionCall(Function, ImporterMonoMode, {TexCenterPlusTangentV, BumpSource})),
			    NewMaterialExpressionSubtract(Function,
			                                  NewMaterialExpressionFunctionCall(Function, ImporterMonoMode, {TexCenterMinusV, BumpSource}),
			                                  NewMaterialExpressionFunctionCall(Function, ImporterMonoMode, {TexCenterPlusV, BumpSource})));
#else
			UMaterialExpressionSubtract* CentralDifferenceV =
			    NewMaterialExpressionSubtract(Function,
			                                  NewMaterialExpressionFunctionCall(Function, ImporterMonoMode, {TexCenterMinusV, BumpSource}),
			                                  NewMaterialExpressionFunctionCall(Function, ImporterMonoMode, {TexCenterPlusV, BumpSource}));
#endif
			UMaterialExpressionMultiply* FactorV = NewMaterialExpressionMultiply(Function, Factor, CentralDifferenceV);

			UMaterialExpressionNormalize* UnclippedResult =
			    NewMaterialExpressionNormalize(Function,
			                                   NewMaterialExpressionAdd(Function,
			                                                            {Normal, NewMaterialExpressionMultiply(Function, FactorU, UVWTangentU),
			                                                             NewMaterialExpressionMultiply(Function, FactorV, UVWTangentV)}));
#endif

			UMaterialExpressionIf* ClipVCheck = NewMaterialExpressionIfEqual(
			    Function, WrapV, wrap_clip,
			    NewMaterialExpressionIfLess(Function, UVWPositionY, 0.0f, Normal,
			                                NewMaterialExpressionIfGreater(Function, UVWPositionY, 1.0f, Normal, UnclippedResult)),
			    UnclippedResult);
			UMaterialExpressionIf* ClipUCheck = NewMaterialExpressionIfEqual(
			    Function, WrapU, wrap_clip,
			    NewMaterialExpressionIfLess(Function, UVWPositionX, 0.0f, Normal,
			                                NewMaterialExpressionIfGreater(Function, UVWPositionX, 1.0f, Normal, ClipVCheck)),
			    ClipVCheck);
			UMaterialExpressionIf* ClampVCheck = NewMaterialExpressionIfEqual(
			    Function, WrapV, wrap_clamp,
			    NewMaterialExpressionIfLess(Function, UVWPositionY, 0.0f, Normal,
			                                NewMaterialExpressionIfGreater(Function, UVWPositionY, 1.0f, Normal, ClipUCheck)),
			    ClipUCheck);
			UMaterialExpressionStaticSwitch* Result = NewMaterialExpressionStaticSwitch(
			    Function, Clip,
			    NewMaterialExpressionIfEqual(
			        Function, WrapU, wrap_clamp,
			        NewMaterialExpressionIfLess(Function, UVWPositionX, 0.0f, Normal,
			                                    NewMaterialExpressionIfGreater(Function, UVWPositionX, 1.0f, Normal, ClampVCheck)),
			        ClampVCheck),
			    ClipUCheck);

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Result);
		}

		void BaseFileTexture(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* MathAverage            = LoadFunction(TEXT("mdl_math_average"));
			UMaterialFunction* MathLuminance          = LoadFunction(TEXT("mdl_math_luminance"));
			UMaterialFunction* MathMaxValue           = LoadFunction(TEXT("mdl_math_max_value"));
			UMaterialFunction* StateTextureCoordinate = LoadFunction(TEXT("mdl_state_texture_coordinate"));
			UMaterialFunction* StateTextureTangentU   = LoadFunction(TEXT("mdl_state_texture_tangent_u"));
			UMaterialFunction* StateTextureTangentV   = LoadFunction(TEXT("mdl_state_texture_tangent_v"));
			UMaterialFunction* TexLookupFloat3        = LoadFunction(TEXT("mdl_tex_lookup_float3"));
			UMaterialFunction* TexLookupFloat4        = LoadFunction(TEXT("mdl_tex_lookup_float4"));
#if defined(USE_WORLD_ALIGNED_TEXTURES)
			UMaterialFunction* WorldAlignedTextureFloat3 = LoadFunction(TEXT("mdlimporter_world_aligned_texture_float3"));
			UMaterialFunction* WorldAlignedTextureFloat4 = LoadFunction(TEXT("mdlimporter_world_aligned_texture_float4"));
#endif

			Function->Description = TEXT("General texturing function for 2D bitmap texture stored in a file.");

			UMaterialExpressionFunctionInput* Texture = NewMaterialExpressionFunctionInput(
			    Function, TEXT("texture"), EFunctionInputType::FunctionInput_Texture2D, NewMaterialExpressionTextureObject(Function, nullptr));
			UMaterialExpressionFunctionInput* ColorOffset =
			    NewMaterialExpressionFunctionInput(Function, TEXT("color_offset"), EFunctionInputType::FunctionInput_Vector3, {0.0f, 0.0f, 0.0f});
			UMaterialExpressionFunctionInput* ColorScale =
			    NewMaterialExpressionFunctionInput(Function, TEXT("color_scale"), EFunctionInputType::FunctionInput_Vector3, {1.0f, 1.0f, 1.0f});
			UMaterialExpressionFunctionInput* MonoSource =
			    NewMaterialExpressionFunctionInput(Function, TEXT("mono_source"), EFunctionInputType::FunctionInput_Scalar, (int)EMonoMode::Alpha);
			UMaterialExpressionFunctionInput* UVWPosition =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.position"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureCoordinate, {}));
			UMaterialExpressionFunctionInput* UVWTangentU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.tangent_u"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentU, { 0 }));
			UMaterialExpressionFunctionInput* UVWTangentV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.tangent_v"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentV, { 0 }));
			UMaterialExpressionFunctionInput* CropU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("crop_u"), EFunctionInputType::FunctionInput_Vector2, {0.0f, 1.0f});
			UMaterialExpressionFunctionInput* CropV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("crop_v"), EFunctionInputType::FunctionInput_Vector2, {0.0f, 1.0f});
			UMaterialExpressionFunctionInput* WrapU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("wrap_u"), EFunctionInputType::FunctionInput_Scalar, wrap_repeat);
			UMaterialExpressionFunctionInput* WrapV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("wrap_v"), EFunctionInputType::FunctionInput_Scalar, wrap_repeat);
			UMaterialExpressionFunctionInput* Clip = NewMaterialExpressionFunctionInput(
			    Function, TEXT("clip"), EFunctionInputType::FunctionInput_StaticBool, NewMaterialExpressionStaticBool(Function, false));
			UMaterialExpressionFunctionInput* AnimationStartTime = NewMaterialExpressionFunctionInput(
			    Function, TEXT("animation_start_time"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* AnimationCrop = NewMaterialExpressionFunctionInput(
				Function, TEXT("animation_crop"), EFunctionInputType::FunctionInput_Vector2, {0.0f, 0.0f});
			UMaterialExpressionFunctionInput* AnimationWrap = NewMaterialExpressionFunctionInput(
			    Function, TEXT("animation_wrap"), EFunctionInputType::FunctionInput_Scalar, wrap_repeat); // wrap_mode
			UMaterialExpressionFunctionInput* AnimationFps = NewMaterialExpressionFunctionInput(
			    Function, TEXT("animation_fps"), EFunctionInputType::FunctionInput_Scalar, 30.0f);

#if defined(USE_WORLD_ALIGNED_TEXTURES)
#if defined(USE_WAT_AS_SCALAR)
			UMaterialExpressionFunctionInput* UseWorldAlignedTexture =
			    NewMaterialExpressionFunctionInput(Function, TEXT("use_world_aligned_texture"), EFunctionInputType::FunctionInput_Scalar);
#else
			UMaterialExpressionFunctionInput* UseWorldAlignedTexture =
			    NewMaterialExpressionFunctionInput(Function, TEXT("use_world_aligned_texture"), EFunctionInputType::FunctionInput_StaticBool);
#endif
#endif

			UMaterialExpressionAdd* Float4Result = NewMaterialExpressionAdd(
			    Function,
			    NewMaterialExpressionMultiply(
			        Function,
#if defined(USE_WORLD_ALIGNED_TEXTURES)
#if defined(USE_WAT_AS_SCALAR)
			        NewMaterialExpressionIfEqual(
			            Function, UseWorldAlignedTexture, 1.0f,
#else
			        NewMaterialExpressionStaticSwitch(
			            Function, UseWorldAlignedTexture,
#endif
			            NewMaterialExpressionFunctionCall(Function, WorldAlignedTextureFloat4, {Texture, UVWPosition}),
			            NewMaterialExpressionFunctionCall(
			                Function, TexLookupFloat4,
			                {Texture, NewMaterialExpressionComponentMask(Function, UVWPosition, 3), WrapU, WrapV, CropU, CropV})),
#else
			        NewMaterialExpressionFunctionCall(
			            Function, TexLookupFloat4,
			            {Texture, NewMaterialExpressionComponentMask(Function, UVWPosition, 3), WrapU, WrapV, CropU, CropV}),
#endif
			        NewMaterialExpressionAppendVector(Function, ColorScale, 1.0f)),
			    NewMaterialExpressionAppendVector(Function, ColorOffset, 0.0f));
			UMaterialExpressionAdd* ColorResult = NewMaterialExpressionAdd(
			    Function,
			    NewMaterialExpressionMultiply(
			        Function,
#if defined(USE_WORLD_ALIGNED_TEXTURES)
#if defined(USE_WAT_AS_SCALAR)
			        NewMaterialExpressionIfEqual(
			            Function, UseWorldAlignedTexture, 1.0f,
#else
			        NewMaterialExpressionStaticSwitch(
			            Function, UseWorldAlignedTexture,
#endif
			            NewMaterialExpressionFunctionCall(Function, WorldAlignedTextureFloat3, {Texture, UVWPosition}),
			            NewMaterialExpressionFunctionCall(
			                Function, TexLookupFloat3,
			                {Texture, NewMaterialExpressionComponentMask(Function, UVWPosition, 3), WrapU, WrapV, CropU, CropV})),
#else
			        NewMaterialExpressionFunctionCall(
			            Function, TexLookupFloat3,
			            {Texture, NewMaterialExpressionComponentMask(Function, UVWPosition, 3), WrapU, WrapV, CropU, CropV}),
#endif
			        ColorScale),
			    ColorOffset);
			UMaterialExpressionIf* UnclippedTint = NewMaterialExpressionIfEqual(
			    Function, MonoSource, (int)EMonoMode::Alpha, NewMaterialExpressionComponentMask(Function, Float4Result, 7), ColorResult);
			UMaterialExpressionIf* UnclippedMono =
			    NewMaterialExpressionSwitch(Function, MonoSource,
			                                {NewMaterialExpressionComponentMask(Function, Float4Result, 8),
			                                 NewMaterialExpressionFunctionCall(Function, MathAverage, {ColorResult}),
			                                 NewMaterialExpressionFunctionCall(Function, MathLuminance, {ColorResult}),
			                                 NewMaterialExpressionFunctionCall(Function, MathMaxValue, {ColorResult})});

			UMaterialExpressionComponentMask* UVWPositionX = NewMaterialExpressionComponentMask(Function, UVWPosition, 1);
			UMaterialExpressionComponentMask* UVWPositionY = NewMaterialExpressionComponentMask(Function, UVWPosition, 2);

			UMaterialExpressionConstant3Vector* TintBlack    = NewMaterialExpressionConstant(Function, 0.0f, 0.0f, 0.0f);
			UMaterialExpressionIf* TintPositionYGreaterCheck = NewMaterialExpressionIfGreater(Function, UVWPositionY, 1.0f, TintBlack, UnclippedTint);
			UMaterialExpressionIf* TintPositionYLessCheck =
			    NewMaterialExpressionIfLess(Function, UVWPositionY, 0.0f, TintBlack, TintPositionYGreaterCheck);
			UMaterialExpressionIf* TintWrapVCheck = NewMaterialExpressionIfEqual(Function, WrapV, wrap_clamp, TintPositionYLessCheck, UnclippedTint);
			UMaterialExpressionIf* TintPositionXGreaterCheck =
			    NewMaterialExpressionIfGreater(Function, UVWPositionX, 1.0f, TintBlack, TintWrapVCheck);
			UMaterialExpressionIf* TintPositionXLessCheck =
			    NewMaterialExpressionIfLess(Function, UVWPositionX, 0.0f, TintBlack, TintPositionXGreaterCheck);
			UMaterialExpressionIf* TintWrapUCheck = NewMaterialExpressionIfEqual(Function, WrapU, wrap_clamp, TintPositionXLessCheck, TintWrapVCheck);
			UMaterialExpressionStaticSwitch* Tint = NewMaterialExpressionStaticSwitch(Function, Clip, TintWrapUCheck, UnclippedTint);

			UMaterialExpressionConstant* MonoBlack           = NewMaterialExpressionConstant(Function, 0.0f);
			UMaterialExpressionIf* MonoPositionYGreaterCheck = NewMaterialExpressionIfGreater(Function, UVWPositionY, 1.0f, MonoBlack, UnclippedMono);
			UMaterialExpressionIf* MonoPositionYLessCheck =
			    NewMaterialExpressionIfLess(Function, UVWPositionY, 0.0f, MonoBlack, MonoPositionYGreaterCheck);
			UMaterialExpressionIf* MonoWrapVCheck = NewMaterialExpressionIfEqual(Function, WrapV, wrap_clamp, MonoPositionYLessCheck, UnclippedMono);
			UMaterialExpressionIf* MonoPositionXGreaterCheck =
			    NewMaterialExpressionIfGreater(Function, UVWPositionX, 1.0f, MonoBlack, MonoWrapVCheck);
			UMaterialExpressionIf* MonoPositionXLessCheck =
			    NewMaterialExpressionIfLess(Function, UVWPositionX, 0.0f, MonoBlack, MonoPositionXGreaterCheck);
			UMaterialExpressionIf* MonoWrapUCheck = NewMaterialExpressionIfEqual(Function, WrapU, wrap_clamp, MonoPositionXLessCheck, MonoWrapVCheck);
			UMaterialExpressionStaticSwitch* Mono = NewMaterialExpressionStaticSwitch(Function, Clip, MonoWrapUCheck, UnclippedMono);

			NewMaterialExpressionFunctionOutput(Function, TEXT("tint"), Tint);
			NewMaterialExpressionFunctionOutput(Function, TEXT("mono"), Mono);
		}

		void BaseFlakeNoiseBumpTexture(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* ImporterMiNoise        = LoadFunction(TEXT("mdlimporter_mi_noise"));
			UMaterialFunction* ImporterWorleyNoise    = LoadFunction(TEXT("mdlimporter_worley_noise"));
			UMaterialFunction* StateNormal            = LoadFunction(TEXT("mdl_state_normal"));
			UMaterialFunction* StateTextureCoordinate = LoadFunction(TEXT("mdl_state_texture_coordinate"));
			UMaterialFunction* StateTextureTangentU   = LoadFunction(TEXT("mdl_state_texture_tangent_u"));
			UMaterialFunction* StateTextureTangentV   = LoadFunction(TEXT("mdl_state_texture_tangent_v"));

			Function->Description = TEXT("Bump-mapping flake noise.");

			UMaterialExpressionFunctionInput* UVWPosition =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.position"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureCoordinate, {}));
			UMaterialExpressionFunctionInput* UVWTangentU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.tangent_u"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentU, { 0 }));
			UMaterialExpressionFunctionInput* UVWTangentV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.tangent_v"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentV, { 0 }));
			UMaterialExpressionFunctionInput* Scale =
			    NewMaterialExpressionFunctionInput(Function, TEXT("scale"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* Strength =
			    NewMaterialExpressionFunctionInput(Function, TEXT("strength"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* NoiseType =
			    NewMaterialExpressionFunctionInput(Function, TEXT("noise_type"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* MaximumSize =
			    NewMaterialExpressionFunctionInput(Function, TEXT("maximum_size"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* Metric =
			    NewMaterialExpressionFunctionInput(Function, TEXT("metric"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* Normal = NewMaterialExpressionFunctionInput(
			    Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3, NewMaterialExpressionFunctionCall(Function, StateNormal, {}));

			UMaterialExpressionDivide*               Pos = NewMaterialExpressionDivide(Function, UVWPosition, Scale);
			UMaterialExpressionMaterialFunctionCall* WorleyNoise =
			    NewMaterialExpressionFunctionCall(Function, ImporterWorleyNoise, {Pos, 1.0f, Metric});
			UMaterialExpressionIf* CellDistance =
			    NewMaterialExpressionIfEqual(Function, NoiseType, 1.0f, NewMaterialExpressionComponentMask(Function, {WorleyNoise, 2}, 1), 0.0f);
			UMaterialExpressionIf*                   Pos2    = NewMaterialExpressionIfEqual(Function, NoiseType, 1.0f, {WorleyNoise, 0}, Pos);
			UMaterialExpressionMaterialFunctionCall* MiNoise = NewMaterialExpressionFunctionCall(Function, ImporterMiNoise, {Pos2});
			UMaterialExpressionAdd* Pos3 = NewMaterialExpressionAdd(Function, Pos2, NewMaterialExpressionMultiply(Function, {MiNoise, 0}, 2.0f));
			UMaterialExpressionMaterialFunctionCall* MiNoise2 =
			    NewMaterialExpressionFunctionCall(Function, ImporterMiNoise, {NewMaterialExpressionFloor(Function, Pos3)});
			UMaterialExpressionIf* Grad = NewMaterialExpressionIfEqual(Function, NoiseType, 0.0f, {MiNoise2, 0}, {MiNoise, 0});

			UMaterialExpressionNormalize* CalculatedNormal = NewMaterialExpressionNormalize(
			    Function,
			    NewMaterialExpressionAdd(
			        Function,
			        {NewMaterialExpressionMultiply(
			             Function,
			             Normal,
			             NewMaterialExpressionAdd(Function,
			                                      NewMaterialExpressionAbs(Function, NewMaterialExpressionComponentMask(Function, Grad, 4)),
			                                      NewMaterialExpressionDivide(Function, 1.0f, Strength))),
			         NewMaterialExpressionMultiply(
			             Function,
			             UVWTangentU,
			             NewMaterialExpressionDivide(Function, NewMaterialExpressionComponentMask(Function, Grad, 1), Strength)),
			         NewMaterialExpressionMultiply(
			             Function,
			             UVWTangentV,
			             NewMaterialExpressionDivide(Function, NewMaterialExpressionComponentMask(Function, Grad, 2), Strength))}));
			UMaterialExpressionIf* MaximumSizeCheck = NewMaterialExpressionIfGreater(Function, CellDistance, MaximumSize, Normal, CalculatedNormal);
			UMaterialExpressionIf* Result           = NewMaterialExpressionIfEqual(
                Function, Strength, 0.0f, Normal,
                NewMaterialExpressionIfEqual(Function, Scale, 0.0f, Normal,
                                             NewMaterialExpressionIfEqual(Function, NoiseType, 1.0f, MaximumSizeCheck, CalculatedNormal)));

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Result);
		}

		void BaseFlakeNoiseTexture(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* ImporterMiNoise     = LoadFunction(TEXT("mdlimporter_mi_noise"));
			UMaterialFunction* ImporterWorleyNoise = LoadFunction(TEXT("mdlimporter_worley_noise"));
			UMaterialFunction* MakeFloat3 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat3.MakeFloat3"));
			UMaterialFunction* StateTextureCoordinate = LoadFunction(TEXT("mdl_state_texture_coordinate"));
			UMaterialFunction* StateTextureTangentU   = LoadFunction(TEXT("mdl_state_texture_tangent_u"));
			UMaterialFunction* StateTextureTangentV   = LoadFunction(TEXT("mdl_state_texture_tangent_v"));

			Function->Description = TEXT("Flake noise.");

			UMaterialExpressionFunctionInput* UVWPosition =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.position"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureCoordinate, {}));
			UMaterialExpressionFunctionInput* UVWTangentU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.tangent_u"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentU, { 0 }));
			UMaterialExpressionFunctionInput* UVWTangentV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.tangent_v"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentV, { 0 }));
			UMaterialExpressionFunctionInput* Intensity =
			    NewMaterialExpressionFunctionInput(Function, TEXT("intensity"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* Scale =
			    NewMaterialExpressionFunctionInput(Function, TEXT("scale"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* Density =
			    NewMaterialExpressionFunctionInput(Function, TEXT("density"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* NoiseType =
			    NewMaterialExpressionFunctionInput(Function, TEXT("noise_type"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* MaximumSize =
			    NewMaterialExpressionFunctionInput(Function, TEXT("maximum_size"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* Metric =
			    NewMaterialExpressionFunctionInput(Function, TEXT("metric"), EFunctionInputType::FunctionInput_Scalar, 0.0f);

			UMaterialExpressionDivide*               Pos = NewMaterialExpressionDivide(Function, UVWPosition, Scale);
			UMaterialExpressionMaterialFunctionCall* WorleyNoise =
			    NewMaterialExpressionFunctionCall(Function, ImporterWorleyNoise, {Pos, 1.0f, Metric});
			UMaterialExpressionComponentMask*        CellDistance = NewMaterialExpressionComponentMask(Function, {WorleyNoise, 2}, 1);
			UMaterialExpressionIf*                   Pos2         = NewMaterialExpressionIfEqual(Function, NoiseType, 1.0f, {WorleyNoise, 0}, Pos);
			UMaterialExpressionMaterialFunctionCall* MiNoise      = NewMaterialExpressionFunctionCall(Function, ImporterMiNoise, {Pos2});
			UMaterialExpressionAdd* Pos3 = NewMaterialExpressionAdd(Function, Pos2, NewMaterialExpressionMultiply(Function, {MiNoise, 0}, 2.0f));
			UMaterialExpressionMaterialFunctionCall* MiNoise2 =
			    NewMaterialExpressionFunctionCall(Function, ImporterMiNoise, {NewMaterialExpressionFloor(Function, Pos3)});
			UMaterialExpressionIf*       NoiseScale = NewMaterialExpressionIfEqual(Function, NoiseType, 0.0f, {MiNoise2, 1}, {MiNoise, 1});
			UMaterialExpressionMultiply* ReflectivityCalculation = NewMaterialExpressionMultiply(
			    Function, NewMaterialExpressionPower(Function, NoiseScale, NewMaterialExpressionDivide(Function, 1.0f, Density)), Intensity);
			UMaterialExpressionIf* CellDistanceCheck =
			    NewMaterialExpressionIfGreater(Function, CellDistance, MaximumSize, 0.0f, ReflectivityCalculation);
			UMaterialExpressionIf* Reflectivity = NewMaterialExpressionIfEqual(
			    Function, Scale, 0.0f, 0.0f, NewMaterialExpressionIfEqual(Function, NoiseType, 1.0f, CellDistanceCheck, ReflectivityCalculation));
			UMaterialExpressionIf* Tint =
			    NewMaterialExpressionIfEqual(Function, Scale, 0.0f, {0.0f, 0.0f, 0.0f},
			                                 NewMaterialExpressionFunctionCall(Function, MakeFloat3, {Reflectivity, Reflectivity, Reflectivity}));

			NewMaterialExpressionFunctionOutput(Function, TEXT("tint"), Tint);
			NewMaterialExpressionFunctionOutput(Function, TEXT("mono"), Reflectivity);
		}

		void BaseFlowNoiseBumpTexture(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* ImporterSummedFlowNoise = LoadFunction(TEXT("mdlimporter_summed_flow_noise"));
			UMaterialFunction* MakeFloat2 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat2.MakeFloat2"));
			UMaterialFunction* StateNormal            = LoadFunction(TEXT("mdl_state_normal"));
			UMaterialFunction* StateTextureCoordinate = LoadFunction(TEXT("mdl_state_texture_coordinate"));
			UMaterialFunction* StateTextureTangentU   = LoadFunction(TEXT("mdl_state_texture_tangent_u"));
			UMaterialFunction* StateTextureTangentV   = LoadFunction(TEXT("mdl_state_texture_tangent_v"));

			Function->Description = TEXT("Bump-mapping low noise.");

			UMaterialExpressionFunctionInput* UVWPosition =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.position"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureCoordinate, {}));
			UMaterialExpressionFunctionInput* UVWTangentU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.tangent_u"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentU, { 0 }));
			UMaterialExpressionFunctionInput* UVWTangentV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.tangent_v"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentV, { 0 }));
			UMaterialExpressionFunctionInput* Factor =
			    NewMaterialExpressionFunctionInput(Function, TEXT("factor"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* Size =
			    NewMaterialExpressionFunctionInput(Function, TEXT("size"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* Phase =
			    NewMaterialExpressionFunctionInput(Function, TEXT("phase"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* Levels =
			    NewMaterialExpressionFunctionInput(Function, TEXT("levels"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* AbsoluteNoise = NewMaterialExpressionFunctionInput(
			    Function, TEXT("absolute_noise"), EFunctionInputType::FunctionInput_StaticBool, NewMaterialExpressionStaticBool(Function, false));
			UMaterialExpressionFunctionInput* LevelGain =
			    NewMaterialExpressionFunctionInput(Function, TEXT("level_gain"), EFunctionInputType::FunctionInput_Scalar, 0.5f);
			UMaterialExpressionFunctionInput* LevelScale =
			    NewMaterialExpressionFunctionInput(Function, TEXT("level_scale"), EFunctionInputType::FunctionInput_Scalar, 2.0f);
			UMaterialExpressionFunctionInput* LevelProgressiveUScale =
			    NewMaterialExpressionFunctionInput(Function, TEXT("level_progressive_u_scale"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* LevelProgressiveUMotion =
			    NewMaterialExpressionFunctionInput(Function, TEXT("level_progressive_u_motion"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* Normal = NewMaterialExpressionFunctionInput(
			    Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3, NewMaterialExpressionFunctionCall(Function, StateNormal, {}));

			UMaterialExpressionComponentMask* Pos   = NewMaterialExpressionComponentMask(Function, UVWPosition, 3);
			UMaterialExpressionMultiply*      Delta = NewMaterialExpressionMultiply(Function, 0.1f, Size);

			UMaterialExpressionAdd* Result0 = NewMaterialExpressionAdd(
			    Function,
			    NewMaterialExpressionMultiply(
			        Function,
			        NewMaterialExpressionFunctionCall(Function, ImporterSummedFlowNoise,
			                                          {NewMaterialExpressionDivide(Function, Pos, Size), Phase, Levels, AbsoluteNoise, LevelGain,
			                                           LevelScale, LevelProgressiveUScale, LevelProgressiveUMotion}),
			        0.5f),
			    0.5f);
			UMaterialExpressionAdd* Result1 = NewMaterialExpressionAdd(
			    Function,
			    NewMaterialExpressionMultiply(
			        Function,
			        NewMaterialExpressionFunctionCall(
			            Function, ImporterSummedFlowNoise,
			            {NewMaterialExpressionDivide(
			                 Function,
			                 NewMaterialExpressionAdd(Function, Pos, NewMaterialExpressionFunctionCall(Function, MakeFloat2, {Delta, 0.0f})),
			                 Size),
			             Phase, Levels, AbsoluteNoise, LevelGain, LevelScale, LevelProgressiveUScale, LevelProgressiveUMotion}),
			        0.5f),
			    0.5f);
			UMaterialExpressionAdd* Result2 = NewMaterialExpressionAdd(
			    Function,
			    NewMaterialExpressionMultiply(
			        Function,
			        NewMaterialExpressionFunctionCall(
			            Function, ImporterSummedFlowNoise,
			            {NewMaterialExpressionDivide(
			                 Function,
			                 NewMaterialExpressionAdd(Function, Pos, NewMaterialExpressionFunctionCall(Function, MakeFloat2, {0.0f, Delta})),
			                 Size),
			             Phase, Levels, AbsoluteNoise, LevelGain, LevelScale, LevelProgressiveUScale, LevelProgressiveUMotion}),
			        0.5f),
			    0.5f);

			UMaterialExpressionIf* Result = NewMaterialExpressionIfEqual(
			    Function, Factor, 0.0f, Normal,
			    NewMaterialExpressionIfEqual(
			        Function, Size, 0.0f, Normal,
			        NewMaterialExpressionAdd(
			            Function,
			            {Normal,
			             NewMaterialExpressionMultiply(Function, {UVWTangentV, NewMaterialExpressionSubtract(Function, Result2, Result0), Factor}),
			             NewMaterialExpressionMultiply(Function,
			                                           {UVWTangentU, NewMaterialExpressionSubtract(Function, Result1, Result0), Factor})})));

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Result);
		}

		void BaseFlowNoiseTexture(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* ImporterSummedFlowNoise = LoadFunction(TEXT("mdlimporter_summed_flow_noise"));
			UMaterialFunction* MathLuminance           = LoadFunction(TEXT("mdl_math_luminance"));
			UMaterialFunction* StateTextureCoordinate  = LoadFunction(TEXT("mdl_state_texture_coordinate"));
			UMaterialFunction* StateTextureTangentU    = LoadFunction(TEXT("mdl_state_texture_tangent_u"));
			UMaterialFunction* StateTextureTangentV    = LoadFunction(TEXT("mdl_state_texture_tangent_v"));

			Function->Description = TEXT("Color Perlin flow noise.");

			UMaterialExpressionFunctionInput* UVWPosition =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.position"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureCoordinate, {}));
			UMaterialExpressionFunctionInput* UVWTangentU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.tangent_u"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentU, { 0 }));
			UMaterialExpressionFunctionInput* UVWTangentV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.tangent_v"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentV, { 0 }));
			UMaterialExpressionFunctionInput* Color1 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("color1"), EFunctionInputType::FunctionInput_Vector3, {0.0f, 0.0f, 0.0f});
			UMaterialExpressionFunctionInput* Color2 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("color2"), EFunctionInputType::FunctionInput_Vector3, {0.0f, 0.0f, 0.0f});
			UMaterialExpressionFunctionInput* Size =
			    NewMaterialExpressionFunctionInput(Function, TEXT("size"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* Phase =
			    NewMaterialExpressionFunctionInput(Function, TEXT("phase"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* Levels =
			    NewMaterialExpressionFunctionInput(Function, TEXT("levels"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* AbsoluteNoise = NewMaterialExpressionFunctionInput(
			    Function, TEXT("absolute_noise"), EFunctionInputType::FunctionInput_StaticBool, NewMaterialExpressionStaticBool(Function, false));
			UMaterialExpressionFunctionInput* LevelGain =
			    NewMaterialExpressionFunctionInput(Function, TEXT("level_gain"), EFunctionInputType::FunctionInput_Scalar, 0.5f);
			UMaterialExpressionFunctionInput* LevelScale =
			    NewMaterialExpressionFunctionInput(Function, TEXT("level_scale"), EFunctionInputType::FunctionInput_Scalar, 2.0f);
			UMaterialExpressionFunctionInput* LevelProgressiveUScale =
			    NewMaterialExpressionFunctionInput(Function, TEXT("level_progressive_u_scale"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* LevelProgressiveVMotion =
			    NewMaterialExpressionFunctionInput(Function, TEXT("level_progressive_v_motion"), EFunctionInputType::FunctionInput_Scalar, 0.0f);

			UMaterialExpressionMaterialFunctionCall* SummedNoise = NewMaterialExpressionFunctionCall(
			    Function, ImporterSummedFlowNoise,
			    {NewMaterialExpressionDivide(Function, NewMaterialExpressionComponentMask(Function, UVWPosition, 3), Size), Phase, Levels,
			     AbsoluteNoise, LevelGain, LevelScale, LevelProgressiveUScale, LevelProgressiveVMotion});
			UMaterialExpressionIf* Alpha = NewMaterialExpressionIfEqual(
			    Function, Size, 0.0f, 0.0f, NewMaterialExpressionAdd(Function, NewMaterialExpressionMultiply(Function, SummedNoise, 0.5f), 0.5f));
			UMaterialExpressionLinearInterpolate*    Tint = NewMaterialExpressionLinearInterpolate(Function, Color1, Color2, Alpha);
			UMaterialExpressionMaterialFunctionCall* Mono = NewMaterialExpressionFunctionCall(Function, MathLuminance, {Tint});

			NewMaterialExpressionFunctionOutput(Function, TEXT("tint"), Tint);
			NewMaterialExpressionFunctionOutput(Function, TEXT("mono"), Mono);
		}

		/*static*/ void BaseGlossToRough(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialExpressionFunctionInput* Glossiness =
			    NewMaterialExpressionFunctionInput(Function, TEXT("glossiness"), EFunctionInputType::FunctionInput_Scalar);

			UMaterialExpressionOneMinus* Rough = NewMaterialExpressionOneMinus(Function, Glossiness);

			NewMaterialExpressionFunctionOutput(Function, TEXT("rough"), Rough);
		}

		void BasePerlinNoiseBumpTexture(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* ImporterApplyNoiseModifications = LoadFunction(TEXT("mdlimporter_apply_noise_modifications"));
			UMaterialFunction* ImporterSummedPerlinNoise       = LoadFunction(TEXT("mdlimporter_summed_perlin_noise"));
			UMaterialFunction* MakeFloat3 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat3.MakeFloat3"));
			UMaterialFunction* StateNormal            = LoadFunction(TEXT("mdl_state_normal"));
			UMaterialFunction* StateTextureCoordinate = LoadFunction(TEXT("mdl_state_texture_coordinate"));
			UMaterialFunction* StateTextureTangentU   = LoadFunction(TEXT("mdl_state_texture_tangent_u"));
			UMaterialFunction* StateTextureTangentV   = LoadFunction(TEXT("mdl_state_texture_tangent_v"));

			Function->Description = TEXT("Bump-mapping Perlin noise.");

			UMaterialExpressionFunctionInput* UVWPosition =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.position"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureCoordinate, {}));
			UMaterialExpressionFunctionInput* UVWTangentU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.tangent_u"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentU, { 0 }));
			UMaterialExpressionFunctionInput* UVWTangentV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.tangent_v"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentV, { 0 }));
			UMaterialExpressionFunctionInput* Factor =
			    NewMaterialExpressionFunctionInput(Function, TEXT("factor"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* Size =
			    NewMaterialExpressionFunctionInput(Function, TEXT("size"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* ApplyMarble = NewMaterialExpressionFunctionInput(
			    Function, TEXT("apply_marble"), EFunctionInputType::FunctionInput_StaticBool, NewMaterialExpressionStaticBool(Function, false));
			UMaterialExpressionFunctionInput* ApplyDent = NewMaterialExpressionFunctionInput(
			    Function, TEXT("apply_dent"), EFunctionInputType::FunctionInput_StaticBool, NewMaterialExpressionStaticBool(Function, false));
			UMaterialExpressionFunctionInput* NoisePhase =
			    NewMaterialExpressionFunctionInput(Function, TEXT("noise_phase"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* NoiseLevels =
			    NewMaterialExpressionFunctionInput(Function, TEXT("noise_levels"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* AbsoluteNoise = NewMaterialExpressionFunctionInput(
			    Function, TEXT("absolute_noise"), EFunctionInputType::FunctionInput_StaticBool, NewMaterialExpressionStaticBool(Function, false));
			UMaterialExpressionFunctionInput* RidgedNoise = NewMaterialExpressionFunctionInput(
			    Function, TEXT("ridged_noise"), EFunctionInputType::FunctionInput_StaticBool, NewMaterialExpressionStaticBool(Function, false));
			UMaterialExpressionFunctionInput* NoiseDistortion =
			    NewMaterialExpressionFunctionInput(Function, TEXT("noise_distortion"), EFunctionInputType::FunctionInput_Vector3, {0.0f, 0.0f, 0.0f});
			UMaterialExpressionFunctionInput* NoiseThresholdHigh =
			    NewMaterialExpressionFunctionInput(Function, TEXT("noise_threshold_high"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* NoiseThresholdLow =
			    NewMaterialExpressionFunctionInput(Function, TEXT("noise_threshold_low"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* NoiseBands =
			    NewMaterialExpressionFunctionInput(Function, TEXT("noise_bands"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* Normal = NewMaterialExpressionFunctionInput(
			    Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3, NewMaterialExpressionFunctionCall(Function, StateNormal, {}));

			UMaterialExpressionDivide* Delta = NewMaterialExpressionDivide(Function, NewMaterialExpressionMultiply(Function, 0.1f, Size), NoiseBands);

			UMaterialExpressionDivide*               ScaledPosition0 = NewMaterialExpressionDivide(Function, UVWPosition, Size);
			UMaterialExpressionMaterialFunctionCall* Result0         = NewMaterialExpressionFunctionCall(
                Function, ImporterApplyNoiseModifications,
                {NewMaterialExpressionFunctionCall(Function, ImporterSummedPerlinNoise,
                                                   {ScaledPosition0, NoisePhase, NoiseLevels, NoiseDistortion, AbsoluteNoise, RidgedNoise}),
                 NewMaterialExpressionComponentMask(Function, ScaledPosition0, 1), ApplyMarble, ApplyDent, NoiseThresholdHigh, NoiseThresholdLow,
                 NoiseBands});

			UMaterialExpressionDivide* ScaledPosition1 = NewMaterialExpressionDivide(
			    Function,
			    NewMaterialExpressionAdd(Function, UVWPosition, NewMaterialExpressionFunctionCall(Function, MakeFloat3, {Delta, 0.0f, 0.0f})), Size);
			UMaterialExpressionMaterialFunctionCall* Result1 = NewMaterialExpressionFunctionCall(
			    Function, ImporterApplyNoiseModifications,
			    {NewMaterialExpressionFunctionCall(Function, ImporterSummedPerlinNoise,
			                                       {ScaledPosition1, NoisePhase, NoiseLevels, NoiseDistortion, AbsoluteNoise, RidgedNoise}),
			     NewMaterialExpressionComponentMask(Function, ScaledPosition1, 1), ApplyMarble, ApplyDent, NoiseThresholdHigh, NoiseThresholdLow,
			     NoiseBands});

			UMaterialExpressionDivide* ScaledPosition2 = NewMaterialExpressionDivide(
			    Function,
			    NewMaterialExpressionAdd(Function, UVWPosition, NewMaterialExpressionFunctionCall(Function, MakeFloat3, {0.0f, Delta, 0.0f})), Size);
			UMaterialExpressionMaterialFunctionCall* Result2 = NewMaterialExpressionFunctionCall(
			    Function, ImporterApplyNoiseModifications,
			    {NewMaterialExpressionFunctionCall(Function, ImporterSummedPerlinNoise,
			                                       {ScaledPosition2, NoisePhase, NoiseLevels, NoiseDistortion, AbsoluteNoise, RidgedNoise}),
			     NewMaterialExpressionComponentMask(Function, ScaledPosition2, 1), ApplyMarble, ApplyDent, NoiseThresholdHigh, NoiseThresholdLow,
			     NoiseBands});

			UMaterialExpressionDivide* ScaledPosition3 = NewMaterialExpressionDivide(
			    Function,
			    NewMaterialExpressionAdd(Function, UVWPosition, NewMaterialExpressionFunctionCall(Function, MakeFloat3, {0.0f, 0.0f, Delta})), Size);
			UMaterialExpressionMaterialFunctionCall* Result3 = NewMaterialExpressionFunctionCall(
			    Function, ImporterApplyNoiseModifications,
			    {NewMaterialExpressionFunctionCall(Function, ImporterSummedPerlinNoise,
			                                       {ScaledPosition3, NoisePhase, NoiseLevels, NoiseDistortion, AbsoluteNoise, RidgedNoise}),
			     NewMaterialExpressionComponentMask(Function, ScaledPosition3, 1), ApplyMarble, ApplyDent, NoiseThresholdHigh, NoiseThresholdLow,
			     NoiseBands});

			UMaterialExpressionSubtract* BumpFactor = NewMaterialExpressionNegate(Function, Factor);
			UMaterialExpressionIf*       Result     = NewMaterialExpressionIfEqual(
                Function, Factor, 0.0f, Normal,
                NewMaterialExpressionIfEqual(
                    Function, Size, 0.0f, Normal,
                    NewMaterialExpressionNormalize(
                        Function,
                        NewMaterialExpressionAdd(
                            Function,
                            {NewMaterialExpressionMultiply(
                                 Function,
                                 Normal,
                                 NewMaterialExpressionAdd(
                                     Function,
                                     NewMaterialExpressionAbs(
                                         Function, NewMaterialExpressionMultiply(Function, NewMaterialExpressionSubtract(Function, Result3, Result0),
                                                                                 BumpFactor)),
                                     1.0f)),
                             NewMaterialExpressionMultiply(Function,
                                                           {UVWTangentU, NewMaterialExpressionSubtract(Function, Result1, Result0), BumpFactor}),
                             NewMaterialExpressionMultiply(Function,
                                                           {UVWTangentV, NewMaterialExpressionSubtract(Function, Result2, Result0), BumpFactor})}))));

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Result);
		}

		void BasePerlinNoiseTexture(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* ImporterApplyNoiseModifications = LoadFunction(TEXT("mdlimporter_apply_noise_modifications"));
			UMaterialFunction* ImporterSummedPerlinNoise       = LoadFunction(TEXT("mdlimporter_summed_perlin_noise"));
			UMaterialFunction* MathLuminance                   = LoadFunction(TEXT("mdl_math_luminance"));
			UMaterialFunction* StateTextureCoordinate          = LoadFunction(TEXT("mdl_state_texture_coordinate"));
			UMaterialFunction* StateTextureTangentU            = LoadFunction(TEXT("mdl_state_texture_tangent_u"));
			UMaterialFunction* StateTextureTangentV            = LoadFunction(TEXT("mdl_state_texture_tangent_v"));

			Function->Description = TEXT("Color Perlin noise.");

			UMaterialExpressionFunctionInput* UVWPosition =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.position"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureCoordinate, {}));
			UMaterialExpressionFunctionInput* UVWTangentU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.tangent_u"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentU, { 0 }));
			UMaterialExpressionFunctionInput* UVWTangentV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.tangent_v"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentV, { 0 }));
			UMaterialExpressionFunctionInput* Color1 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("color1"), EFunctionInputType::FunctionInput_Vector3, {0.0f, 0.0f, 0.0f});
			UMaterialExpressionFunctionInput* Color2 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("color2"), EFunctionInputType::FunctionInput_Vector3, {0.0f, 0.0f, 0.0f});
			UMaterialExpressionFunctionInput* Size =
			    NewMaterialExpressionFunctionInput(Function, TEXT("size"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* ApplyMarble = NewMaterialExpressionFunctionInput(
			    Function, TEXT("apply_marble"), EFunctionInputType::FunctionInput_StaticBool, NewMaterialExpressionStaticBool(Function, false));
			UMaterialExpressionFunctionInput* ApplyDent = NewMaterialExpressionFunctionInput(
			    Function, TEXT("apply_dent"), EFunctionInputType::FunctionInput_StaticBool, NewMaterialExpressionStaticBool(Function, false));
			UMaterialExpressionFunctionInput* NoisePhase =
			    NewMaterialExpressionFunctionInput(Function, TEXT("noise_phase"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* NoiseLevels =
			    NewMaterialExpressionFunctionInput(Function, TEXT("noise_levels"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* AbsoluteNoise = NewMaterialExpressionFunctionInput(
			    Function, TEXT("absolute_noise"), EFunctionInputType::FunctionInput_StaticBool, NewMaterialExpressionStaticBool(Function, false));
			UMaterialExpressionFunctionInput* RidgedNoise = NewMaterialExpressionFunctionInput(
			    Function, TEXT("ridged_noise"), EFunctionInputType::FunctionInput_StaticBool, NewMaterialExpressionStaticBool(Function, false));
			UMaterialExpressionFunctionInput* NoiseDistortion =
			    NewMaterialExpressionFunctionInput(Function, TEXT("noise_distortion"), EFunctionInputType::FunctionInput_Vector3, {0.0f, 0.0f, 0.0f});
			UMaterialExpressionFunctionInput* NoiseThresholdHigh =
			    NewMaterialExpressionFunctionInput(Function, TEXT("noise_threshold_high"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* NoiseThresholdLow =
			    NewMaterialExpressionFunctionInput(Function, TEXT("noise_threshold_low"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* NoiseBands =
			    NewMaterialExpressionFunctionInput(Function, TEXT("noise_bands"), EFunctionInputType::FunctionInput_Scalar, 1.0f);

			UMaterialExpressionDivide* ScaledPosition = NewMaterialExpressionDivide(Function, UVWPosition, Size);
			UMaterialExpressionIf*     Alpha          = NewMaterialExpressionIfEqual(
                Function, Size, 0.0f, 0.0f,
                NewMaterialExpressionFunctionCall(
                    Function, ImporterApplyNoiseModifications,
                    {NewMaterialExpressionFunctionCall(Function, ImporterSummedPerlinNoise,
                                                       {ScaledPosition, NoisePhase, NoiseLevels, NoiseDistortion, AbsoluteNoise, RidgedNoise}),
                     NewMaterialExpressionComponentMask(Function, ScaledPosition, 1), ApplyMarble, ApplyDent, NoiseThresholdHigh, NoiseThresholdLow,
                     NoiseBands}));
			UMaterialExpressionLinearInterpolate*    Tint = NewMaterialExpressionLinearInterpolate(Function, Color1, Color2, Alpha);
			UMaterialExpressionMaterialFunctionCall* Mono = NewMaterialExpressionFunctionCall(Function, MathLuminance, {Tint});

			NewMaterialExpressionFunctionOutput(Function, TEXT("tint"), Tint);
			NewMaterialExpressionFunctionOutput(Function, TEXT("mono"), Mono);
		}

		void BaseRotationTranslationScale(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* MakeFloat4 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat4.MakeFloat4"));
			UMaterialFunction* MathMultiplyFloat4x4Float4x4 = LoadFunction(TEXT("mdl_math_multiply_float4x4_float4x4"));

			Function->Description = TEXT("Construct transformation matrix from Euler rotation, translation and scale.");

			UMaterialExpressionFunctionInput* Rotation =
			    NewMaterialExpressionFunctionInput(Function, TEXT("rotation"), EFunctionInputType::FunctionInput_Vector3, {0.0f, 0.0f, 0.0f});
			UMaterialExpressionFunctionInput* Translation =
			    NewMaterialExpressionFunctionInput(Function, TEXT("translation"), EFunctionInputType::FunctionInput_Vector3, {0.0f, 0.0f, 0.0f});
			UMaterialExpressionFunctionInput* Scaling =
			    NewMaterialExpressionFunctionInput(Function, TEXT("scaling"), EFunctionInputType::FunctionInput_Vector3, {1.0f, 1.0f, 1.0f});

			// Euler rotation matrix  xyz order
			UMaterialExpressionComponentMask* RX = NewMaterialExpressionComponentMask(Function, Rotation, 1);
			UMaterialExpressionComponentMask* RY = NewMaterialExpressionComponentMask(Function, Rotation, 2);
			UMaterialExpressionComponentMask* RZ = NewMaterialExpressionComponentMask(Function, Rotation, 4);
			UMaterialExpressionSine*          SX = NewMaterialExpressionSine(Function, RX);
			UMaterialExpressionSine*          SY = NewMaterialExpressionSine(Function, RY);
			UMaterialExpressionSine*          SZ = NewMaterialExpressionSine(Function, RZ);
			UMaterialExpressionCosine*        CX = NewMaterialExpressionCosine(Function, RX);
			UMaterialExpressionCosine*        CY = NewMaterialExpressionCosine(Function, RY);
			UMaterialExpressionCosine*        CZ = NewMaterialExpressionCosine(Function, RZ);

			// use the transposed rotation matrix, as we're left-handed here!
			// would need some closer investigations if everything is ordered as expected
			UMaterialExpressionMaterialFunctionCall* Rotate0 =
			    NewMaterialExpressionFunctionCall(Function, MakeFloat4,
			                                      {NewMaterialExpressionMultiply(Function, CY, CZ),
			                                       NewMaterialExpressionSubtract(Function, NewMaterialExpressionMultiply(Function, {SX, SY, CZ}),
			                                                                     NewMaterialExpressionMultiply(Function, CX, SZ)),
			                                       NewMaterialExpressionAdd(Function, NewMaterialExpressionMultiply(Function, {CX, SY, CZ}),
			                                                                NewMaterialExpressionMultiply(Function, SX, SZ)),
			                                       0.0f});
			UMaterialExpressionMaterialFunctionCall* Rotate1 =
			    NewMaterialExpressionFunctionCall(Function, MakeFloat4,
			                                      {NewMaterialExpressionMultiply(Function, CY, SZ),
			                                       NewMaterialExpressionAdd(Function, NewMaterialExpressionMultiply(Function, {SX, SY, SZ}),
			                                                                NewMaterialExpressionMultiply(Function, CX, CZ)),
			                                       NewMaterialExpressionSubtract(Function, NewMaterialExpressionMultiply(Function, {CX, SY, SZ}),
			                                                                     NewMaterialExpressionMultiply(Function, SX, CZ)),
			                                       0.0f});
			UMaterialExpressionMaterialFunctionCall* Rotate2 =
			    NewMaterialExpressionFunctionCall(Function, MakeFloat4,
			                                      {NewMaterialExpressionNegate(Function, SY), NewMaterialExpressionMultiply(Function, SX, CY),
			                                       NewMaterialExpressionMultiply(Function, CX, CY), 0.0f});
			UMaterialExpressionConstant4Vector* Rotate3 = NewMaterialExpressionConstant(Function, 0.5f, 0.5f, 0.5f, 1.0f);

			UMaterialExpressionMaterialFunctionCall* Scale0 =
			    NewMaterialExpressionFunctionCall(Function, MakeFloat4, {NewMaterialExpressionComponentMask(Function, Scaling, 1), 0.0f, 0.0f, 0.0f});
			UMaterialExpressionMaterialFunctionCall* Scale1 =
			    NewMaterialExpressionFunctionCall(Function, MakeFloat4, {0.0f, NewMaterialExpressionComponentMask(Function, Scaling, 2), 0.0f, 0.0f});
			UMaterialExpressionMaterialFunctionCall* Scale2 =
			    NewMaterialExpressionFunctionCall(Function, MakeFloat4, {0.0f, 0.0f, NewMaterialExpressionComponentMask(Function, Scaling, 4), 0.0f});
			UMaterialExpressionMaterialFunctionCall* Scale3 = NewMaterialExpressionFunctionCall(
			    Function, MakeFloat4,
			    {NewMaterialExpressionSubtract(Function, NewMaterialExpressionComponentMask(Function, Translation, 1), 0.5f),
			     NewMaterialExpressionSubtract(Function, NewMaterialExpressionComponentMask(Function, Translation, 2), 0.5f),
			     NewMaterialExpressionSubtract(Function, NewMaterialExpressionComponentMask(Function, Translation, 4), 0.5f), 1.0f});

			UMaterialExpressionMaterialFunctionCall* Result = NewMaterialExpressionFunctionCall(
			    Function, MathMultiplyFloat4x4Float4x4, {Rotate0, Rotate1, Rotate2, Rotate3, Scale0, Scale1, Scale2, Scale3});

			NewMaterialExpressionFunctionOutput(Function, TEXT("result_0"), {Result, 0});
			NewMaterialExpressionFunctionOutput(Function, TEXT("result_1"), {Result, 1});
			NewMaterialExpressionFunctionOutput(Function, TEXT("result_2"), {Result, 2});
			NewMaterialExpressionFunctionOutput(Function, TEXT("result_3"), {Result, 3});
		}

		void BaseSellmeierCoefficientsIOR(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* MakeFloat3 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat3.MakeFloat3"));

			UMaterialExpressionFunctionInput* SellmeierB =
			    NewMaterialExpressionFunctionInput(Function, TEXT("sellmeier_B"), EFunctionInputType::FunctionInput_Vector3, {1.04f, 0.23f, 1.01f});
			UMaterialExpressionFunctionInput* SellmeierC =
			    NewMaterialExpressionFunctionInput(Function, TEXT("sellmeier_C"), EFunctionInputType::FunctionInput_Vector3, {0.006f, 0.2f, 103.56f});

			float                          L2    = 0.5892f * 0.5892f;  // simplified to fixed wavelength of 589.2 nm
			UMaterialExpressionSquareRoot* Color = NewMaterialExpressionSquareRoot(
			    Function,
			    NewMaterialExpressionAdd(
			        Function,
			        {1.0f,
			         NewMaterialExpressionMultiply(
			             Function,
			             NewMaterialExpressionComponentMask(Function, SellmeierB, 1),
			             NewMaterialExpressionDivide(
			                 Function, L2, NewMaterialExpressionSubtract(Function, L2, NewMaterialExpressionComponentMask(Function, SellmeierC, 1)))),
			         NewMaterialExpressionMultiply(
			             Function,
			             NewMaterialExpressionComponentMask(Function, SellmeierB, 2),
			             NewMaterialExpressionDivide(
			                 Function, L2, NewMaterialExpressionSubtract(Function, L2, NewMaterialExpressionComponentMask(Function, SellmeierC, 2)))),
			         NewMaterialExpressionMultiply(
			             Function,
			             NewMaterialExpressionComponentMask(Function, SellmeierB, 4),
			             NewMaterialExpressionDivide(
			                 Function,
			                 L2,
			                 NewMaterialExpressionSubtract(Function, L2, NewMaterialExpressionComponentMask(Function, SellmeierC, 4))))}));

			UMaterialExpressionMaterialFunctionCall* Result = NewMaterialExpressionFunctionCall(Function, MakeFloat3, {Color, Color, Color});

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Result);
		}

		void BaseTangentSpaceNormalTexture(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* StateNormal            = LoadFunction(TEXT("mdl_state_normal"));
			UMaterialFunction* StateTextureCoordinate = LoadFunction(TEXT("mdl_state_texture_coordinate"));
			UMaterialFunction* StateTextureTangentU   = LoadFunction(TEXT("mdl_state_texture_tangent_u"));
			UMaterialFunction* StateTextureTangentV   = LoadFunction(TEXT("mdl_state_texture_tangent_v"));
			UMaterialFunction* TexLookupFloat3        = LoadFunction(TEXT("mdl_tex_lookup_float3"));
#if defined(USE_WORLD_ALIGNED_TEXTURES)
			UMaterialFunction* WorldAlignedTextureFloat3 = LoadFunction(TEXT("mdlimporter_world_aligned_texture_float3"));
#endif

			Function->Description = TEXT("Interprets the color values of a bitmap as a vector in tangent space.");

			UMaterialExpressionFunctionInput* Texture = NewMaterialExpressionFunctionInput(
			    Function, TEXT("texture"), EFunctionInputType::FunctionInput_Texture2D, NewMaterialExpressionTextureObject(Function, nullptr));
			UMaterialExpressionFunctionInput* Factor =
			    NewMaterialExpressionFunctionInput(Function, TEXT("factor"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* FlipTangentU = NewMaterialExpressionFunctionInput(
			    Function, TEXT("flip_tangent_u"), EFunctionInputType::FunctionInput_StaticBool, NewMaterialExpressionStaticBool(Function, false));
			UMaterialExpressionFunctionInput* FlipTangentV = NewMaterialExpressionFunctionInput(
			    Function, TEXT("flip_tangent_v"), EFunctionInputType::FunctionInput_StaticBool, NewMaterialExpressionStaticBool(Function, false));
			UMaterialExpressionFunctionInput* UVWPosition =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.position"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureCoordinate, {}));
			UMaterialExpressionFunctionInput* UVWTangentU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.tangent_u"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentU, { 0 }));
			UMaterialExpressionFunctionInput* UVWTangentV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.tangent_v"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentV, { 0 }));
			UMaterialExpressionFunctionInput* CropU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("crop_u"), EFunctionInputType::FunctionInput_Vector2, {0.0f, 1.0f});
			UMaterialExpressionFunctionInput* CropV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("crop_v"), EFunctionInputType::FunctionInput_Vector2, {0.0f, 1.0f});
			UMaterialExpressionFunctionInput* WrapU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("wrap_u"), EFunctionInputType::FunctionInput_Scalar, wrap_repeat);
			UMaterialExpressionFunctionInput* WrapV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("wrap_v"), EFunctionInputType::FunctionInput_Scalar, wrap_repeat);
			UMaterialExpressionFunctionInput* Clip = NewMaterialExpressionFunctionInput(
			    Function, TEXT("clip"), EFunctionInputType::FunctionInput_StaticBool, NewMaterialExpressionStaticBool(Function, false));
			UMaterialExpressionFunctionInput* Scale =
			    NewMaterialExpressionFunctionInput(Function, TEXT("scale"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* Offset =
			    NewMaterialExpressionFunctionInput(Function, TEXT("offset"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* AnimationStartTime = NewMaterialExpressionFunctionInput(
			    Function, TEXT("animation_start_time"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* AnimationCrop = NewMaterialExpressionFunctionInput(
				Function, TEXT("animation_crop"), EFunctionInputType::FunctionInput_Vector2, {0.0f, 0.0f});
			UMaterialExpressionFunctionInput* AnimationWrap = NewMaterialExpressionFunctionInput(
			    Function, TEXT("animation_wrap"), EFunctionInputType::FunctionInput_Scalar, wrap_repeat); // wrap_mode
			UMaterialExpressionFunctionInput* AnimationFps = NewMaterialExpressionFunctionInput(
			    Function, TEXT("animation_fps"), EFunctionInputType::FunctionInput_Scalar, 30.0f);

#if defined(USE_WORLD_ALIGNED_TEXTURES)
#if defined(USE_WAT_AS_SCALAR)
			UMaterialExpressionFunctionInput* UseWorldAlignedTexture =
			    NewMaterialExpressionFunctionInput(Function, TEXT("use_world_aligned_texture"), EFunctionInputType::FunctionInput_Scalar);
#else
			UMaterialExpressionFunctionInput* UseWorldAlignedTexture =
			    NewMaterialExpressionFunctionInput(Function, TEXT("use_world_aligned_texture"), EFunctionInputType::FunctionInput_StaticBool);
#endif
#endif

			UMaterialExpressionComponentMask* UVWPositionX = NewMaterialExpressionComponentMask(Function, UVWPosition, 1);
			UMaterialExpressionComponentMask* UVWPositionY = NewMaterialExpressionComponentMask(Function, UVWPosition, 2);

			UMaterialExpressionMaterialFunctionCall* DefaultNormal = NewMaterialExpressionFunctionCall(Function, StateNormal, {});

#if 0
			// ignore wrapping for now, as that does not compile without errors!
			// TODO: Needs to be investigated!
			UMaterialExpressionStaticSwitch* NotFlipTangentU =
				NewMaterialExpressionStaticSwitch(Function, FlipTangentU,
					NewMaterialExpressionStaticBool(Function, false),
					NewMaterialExpressionStaticBool(Function, true));
			UMaterialExpressionIf* PositionXEvenCheck =
				NewMaterialExpressionIfEqual(Function,
					NewMaterialExpressionFmod(Function, NewMaterialExpressionCeil(Function, UVWPositionX), 2.0f),
					0.0f,
					NotFlipTangentU,
					FlipTangentU);
			UMaterialExpressionIf* FlipPositionXLessCheck = NewMaterialExpressionIfLess(Function, UVWPositionX, 0.0f, PositionXEvenCheck, FlipTangentU);
			UMaterialExpressionIf* PositionXOddCheck =
				NewMaterialExpressionIfEqual(Function,
					NewMaterialExpressionFmod(Function, NewMaterialExpressionFloor(Function, UVWPositionX), 2.0f),
					1.0f,
					NotFlipTangentU,
					FlipTangentU);
			UMaterialExpressionIf* FlipPositionXGreaterCheck = NewMaterialExpressionIfGreater(Function, UVWPositionX, 0.0f, PositionXOddCheck, FlipPositionXLessCheck);
			UMaterialExpressionIf* PixelFlipU = NewMaterialExpressionIfEqual(Function, WrapU, wrap_mirrored_repeat, FlipPositionXGreaterCheck, FlipTangentU);
			UMaterialExpressionStaticSwitch* TransformedTangentU = NewMaterialExpressionStaticSwitch(Function, PixelFlipU, NewMaterialExpressionNegate(Function, UVWTangentU), UVWTangentU);

			UMaterialExpressionStaticSwitch* NotFlipTangentV =
				NewMaterialExpressionStaticSwitch(Function, FlipTangentV,
					NewMaterialExpressionStaticBool(Function, false),
					NewMaterialExpressionStaticBool(Function, true));
			UMaterialExpressionIf* PositionYEvenCheck =
				NewMaterialExpressionIfEqual(Function,
					NewMaterialExpressionFmod(Function, NewMaterialExpressionCeil(Function, UVWPositionY), 2.0f),
					0.0f,
					NotFlipTangentV,
					FlipTangentV);
			UMaterialExpressionIf* FlipPositionYLessCheck = NewMaterialExpressionIfLess(Function, UVWPositionY, 0.0f, PositionYEvenCheck, FlipTangentV);
			UMaterialExpressionIf* PositionYOddCheck =
				NewMaterialExpressionIfEqual(Function,
					NewMaterialExpressionFmod(Function, NewMaterialExpressionFloor(Function, UVWPositionY), 2.0f),
					1.0f,
					NotFlipTangentV,
					FlipTangentV);
			UMaterialExpressionIf* FlipPositionYGreaterCheck = NewMaterialExpressionIfGreater(Function, UVWPositionY, 0.0f, PositionYOddCheck, FlipPositionYLessCheck);
			UMaterialExpressionIf* PixelFlipV = NewMaterialExpressionIfEqual(Function, WrapV, wrap_mirrored_repeat, FlipPositionYGreaterCheck, FlipTangentV);
			UMaterialExpressionStaticSwitch* TransformedTangentV = NewMaterialExpressionStaticSwitch(Function, PixelFlipV, NewMaterialExpressionNegate(Function, UVWTangentV), UVWTangentV);
#else
			UMaterialExpressionStaticSwitch* TransformedTangentU =
			    NewMaterialExpressionStaticSwitch(Function, FlipTangentU, NewMaterialExpressionNegate(Function, UVWTangentU), UVWTangentU);
			UMaterialExpressionStaticSwitch* TransformedTangentV =
			    NewMaterialExpressionStaticSwitch(Function, FlipTangentV, NewMaterialExpressionNegate(Function, UVWTangentV), UVWTangentV);
#endif

			// Unreal already applies bias to normal after sampling so no need to do it manually
			UMaterialExpressionMultiply* TangentSpaceNormal = NewMaterialExpressionMultiply(
			    Function,
#if defined(USE_WORLD_ALIGNED_TEXTURES)
#if defined(USE_WAT_AS_SCALAR)
			    NewMaterialExpressionIfEqual(Function, UseWorldAlignedTexture, 1.0f,
#else
			    NewMaterialExpressionStaticSwitch(
			        Function, UseWorldAlignedTexture,
#endif
			                                 NewMaterialExpressionFunctionCall(Function, WorldAlignedTextureFloat3, {Texture, UVWPosition}),
			                                 NewMaterialExpressionFunctionCall(Function, TexLookupFloat3,
			                                                                   {Texture, NewMaterialExpressionComponentMask(Function, UVWPosition, 3),
			                                                                    WrapU, WrapV, CropU, CropV})),
#else
			    NewMaterialExpressionFunctionCall(
			        Function, TexLookupFloat3, {Texture, NewMaterialExpressionComponentMask(Function, UVWPosition, 3), WrapU, WrapV, CropU, CropV}),
#endif
			    Factor);
			UMaterialExpressionNormalize* UnclippedNormal = NewMaterialExpressionNormalize(
			    Function,
			    NewMaterialExpressionAdd(
			        Function,
			        {NewMaterialExpressionMultiply(Function, TransformedTangentU,
			                                       NewMaterialExpressionComponentMask(Function, TangentSpaceNormal, 1)),
			         NewMaterialExpressionMultiply(Function, TransformedTangentV,
			                                       NewMaterialExpressionComponentMask(Function, TangentSpaceNormal, 2)),
			         NewMaterialExpressionMultiply(Function,
			                                       DefaultNormal,
			                                       NewMaterialExpressionAdd(Function,
			                                                                NewMaterialExpressionComponentMask(Function, TangentSpaceNormal, 4),
			                                                                NewMaterialExpressionOneMinus(Function, Factor)))}));

			UMaterialExpressionIf* ClipVCheck = NewMaterialExpressionIfEqual(
			    Function, WrapV, wrap_clip,
			    NewMaterialExpressionIfLess(Function, UVWPositionY, 0.0f, DefaultNormal,
			                                NewMaterialExpressionIfGreater(Function, UVWPositionY, 1.0f, DefaultNormal, UnclippedNormal)),
			    UnclippedNormal);
			UMaterialExpressionIf* ClipUCheck = NewMaterialExpressionIfEqual(
			    Function, WrapU, wrap_clip,
			    NewMaterialExpressionIfLess(Function, UVWPositionX, 0.0f, DefaultNormal,
			                                NewMaterialExpressionIfGreater(Function, UVWPositionX, 1.0f, DefaultNormal, ClipVCheck)),
			    ClipVCheck);
			UMaterialExpressionIf* ClampVCheck = NewMaterialExpressionIfEqual(
			    Function, WrapV, wrap_clamp,
			    NewMaterialExpressionIfLess(Function, UVWPositionY, 0.0f, DefaultNormal,
			                                NewMaterialExpressionIfGreater(Function, UVWPositionY, 1.0f, DefaultNormal, ClipUCheck)),
			    ClipUCheck);
			UMaterialExpressionStaticSwitch* Normal = NewMaterialExpressionStaticSwitch(
			    Function, Clip,
			    NewMaterialExpressionIfEqual(
			        Function, WrapU, wrap_clamp,
			        NewMaterialExpressionIfLess(Function, UVWPositionX, 0.0f, DefaultNormal,
			                                    NewMaterialExpressionIfGreater(Function, UVWPositionX, 1.0f, DefaultNormal, ClampVCheck)),
			        ClampVCheck),
			    ClipUCheck);

			NewMaterialExpressionFunctionOutput(Function, TEXT("normal"), Normal);
		}

		void BaseTextureCoordinateInfo(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);

			UMaterialFunction* StateTextureCoordinate = LoadFunction(TEXT("mdl_state_texture_coordinate"));
			UMaterialFunction* StateTextureTangentU   = LoadFunction(TEXT("mdl_state_texture_tangent_u"));
			UMaterialFunction* StateTextureTangentV   = LoadFunction(TEXT("mdl_state_texture_tangent_v"));

			UMaterialExpressionFunctionInput* Position =
			    NewMaterialExpressionFunctionInput(Function, TEXT("position"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureCoordinate, {}));
			UMaterialExpressionFunctionInput* TangentU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tangent_u"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentU, { 0 }));
			UMaterialExpressionFunctionInput* TangentV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tangent_v"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentV, { 0 }));
#if defined(USE_WORLD_ALIGNED_TEXTURES)
#if defined(USE_WAT_AS_SCALAR)
			UMaterialExpressionFunctionInput* UseWorldAlignedTexture =
			    NewMaterialExpressionFunctionInput(Function, TEXT("use_world_aligned_texture"), EFunctionInputType::FunctionInput_Scalar);
#else
			UMaterialExpressionFunctionInput* UseWorldAlignedTexture =
			    NewMaterialExpressionFunctionInput(Function, TEXT("use_world_aligned_texture"), EFunctionInputType::FunctionInput_StaticBool);
#endif

			UMaterialFunction* StatePosition = LoadFunction(TEXT("mdl_state_position"));
#endif

#if defined(USE_WORLD_ALIGNED_TEXTURES)
			UMaterialExpressionMaterialFunctionCall* StatePositionCall = NewMaterialExpressionFunctionCall(Function, StatePosition, {});
#if defined(USE_WAT_AS_SCALAR)
			UMaterialExpressionIf* AdjustedPosition =
			    NewMaterialExpressionIfEqual(Function, UseWorldAlignedTexture, 1.0f,
#else
			UMaterialExpressionStaticSwitch* AdjustedPosition =
			    NewMaterialExpressionStaticSwitch(Function, UseWorldAlignedTexture,
#endif
			                                 NewMaterialExpressionDivide(Function, StatePositionCall, 100.0f), Position);
#else
			UMaterialExpression* AdjustedPosition = Position;
#endif

			NewMaterialExpressionFunctionOutput(Function, TEXT("position"), AdjustedPosition);
			NewMaterialExpressionFunctionOutput(Function, TEXT("tangent_u"), TangentU);
			NewMaterialExpressionFunctionOutput(Function, TEXT("tangent_v"), TangentV);
		}

		void BaseTileBumpTexture(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* ImporterEvalTileFunction = LoadFunction(TEXT("mdlimporter_eval_tile_function"));
			UMaterialFunction* MathLuminance            = LoadFunction(TEXT("mdl_math_luminance"));
			UMaterialFunction* StateTextureCoordinate   = LoadFunction(TEXT("mdl_state_texture_coordinate"));
			UMaterialFunction* StateTextureTangentU     = LoadFunction(TEXT("mdl_state_texture_tangent_u"));
			UMaterialFunction* StateTextureTangentV     = LoadFunction(TEXT("mdl_state_texture_tangent_v"));
			UMaterialFunction* StateNormal              = LoadFunction(TEXT("mdl_state_normal"));

			Function->Description = TEXT("Bump-mapping tiling generator.");

			UMaterialExpressionFunctionInput* UVWPosition =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.position"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureCoordinate, {}));
			UMaterialExpressionFunctionInput* UVWTangentU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.tangent_u"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentU, { 0 }));
			UMaterialExpressionFunctionInput* UVWTangentV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.tangent_v"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentV, { 0 }));
			UMaterialExpressionFunctionInput* Factor =
			    NewMaterialExpressionFunctionInput(Function, TEXT("factor"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* NumberOfRows =
			    NewMaterialExpressionFunctionInput(Function, TEXT("number_of_rows"), EFunctionInputType::FunctionInput_Scalar, 4.0f);
			UMaterialExpressionFunctionInput* NumberOfColumns =
			    NewMaterialExpressionFunctionInput(Function, TEXT("number_of_columns"), EFunctionInputType::FunctionInput_Scalar, 4.0f);
			UMaterialExpressionFunctionInput* GroutWidth =
			    NewMaterialExpressionFunctionInput(Function, TEXT("grout_width"), EFunctionInputType::FunctionInput_Scalar, 0.02f);
			UMaterialExpressionFunctionInput* GroutHeight =
			    NewMaterialExpressionFunctionInput(Function, TEXT("grout_height"), EFunctionInputType::FunctionInput_Scalar, 0.02f);
			UMaterialExpressionFunctionInput* GroutRoughness =
			    NewMaterialExpressionFunctionInput(Function, TEXT("grout_roughness"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* MissingTileAmount =
			    NewMaterialExpressionFunctionInput(Function, TEXT("missing_tile_amount"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* TileBrightnessVariation =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tile_brightness_variation"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* Seed =
			    NewMaterialExpressionFunctionInput(Function, TEXT("seed"), EFunctionInputType::FunctionInput_Scalar, 2.284f);
			UMaterialExpressionFunctionInput* SpecialRowIndex =
			    NewMaterialExpressionFunctionInput(Function, TEXT("special_row_index"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* SpecialRowWidthFactor =
			    NewMaterialExpressionFunctionInput(Function, TEXT("special_row_width_factor"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* SpecialColumnIndex =
			    NewMaterialExpressionFunctionInput(Function, TEXT("special_column_index"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* SpecialColumnHeightFactor =
			    NewMaterialExpressionFunctionInput(Function, TEXT("special_column_height_factor"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* OddRowOffset =
			    NewMaterialExpressionFunctionInput(Function, TEXT("odd_row_offset"), EFunctionInputType::FunctionInput_Scalar, 0.5f);
			UMaterialExpressionFunctionInput* RandomRowOffset =
			    NewMaterialExpressionFunctionInput(Function, TEXT("random_row_offset"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* Normal = NewMaterialExpressionFunctionInput(
			    Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3, NewMaterialExpressionFunctionCall(Function, StateNormal, {}));

			//!! magic, looks good with this value, has to be dependent on the incoming grout spacing as otherwise problems/aliasing with sampling the
			//! bump values
			UMaterialExpressionMultiply* Delta =
			    NewMaterialExpressionMultiply(Function, 0.5f, NewMaterialExpressionAdd(Function, GroutWidth, GroutHeight));

			UMaterialExpressionComponentMask*        BasePosition = NewMaterialExpressionComponentMask(Function, UVWPosition, 3);
			UMaterialExpressionMaterialFunctionCall* Result0      = NewMaterialExpressionFunctionCall(
                Function, ImporterEvalTileFunction,
                {BasePosition, NewMaterialExpressionConstant(Function, 1.0f, 1.0f, 1.0f), NewMaterialExpressionConstant(Function, 1.0f, 1.0f, 1.0f),
                 TileBrightnessVariation, MissingTileAmount, Seed, NumberOfRows, NumberOfColumns, OddRowOffset, RandomRowOffset, GroutWidth,
                 GroutHeight, GroutRoughness, SpecialColumnIndex, SpecialRowIndex, SpecialColumnHeightFactor, SpecialRowWidthFactor});

			UMaterialExpressionMaterialFunctionCall* Result1 = NewMaterialExpressionFunctionCall(
			    Function, ImporterEvalTileFunction,
			    {NewMaterialExpressionAdd(Function, BasePosition, NewMaterialExpressionAppendVector(Function, Delta, 0.0f)),
			     NewMaterialExpressionConstant(Function, 1.0f, 1.0f, 1.0f), NewMaterialExpressionConstant(Function, 1.0f, 1.0f, 1.0f),
			     TileBrightnessVariation, MissingTileAmount, Seed, NumberOfRows, NumberOfColumns, OddRowOffset, RandomRowOffset, GroutWidth,
			     GroutHeight, GroutRoughness, SpecialColumnIndex, SpecialRowIndex, SpecialColumnHeightFactor, SpecialRowWidthFactor});

			UMaterialExpressionMaterialFunctionCall* Result2 = NewMaterialExpressionFunctionCall(
			    Function, ImporterEvalTileFunction,
			    {NewMaterialExpressionAdd(Function, BasePosition, NewMaterialExpressionAppendVector(Function, 0.0f, Delta)),
			     NewMaterialExpressionConstant(Function, 1.0f, 1.0f, 1.0f), NewMaterialExpressionConstant(Function, 1.0f, 1.0f, 1.0f),
			     TileBrightnessVariation, MissingTileAmount, Seed, NumberOfRows, NumberOfColumns, OddRowOffset, RandomRowOffset, GroutWidth,
			     GroutHeight, GroutRoughness, SpecialColumnIndex, SpecialRowIndex, SpecialColumnHeightFactor, SpecialRowWidthFactor});

			UMaterialExpressionMultiply* ContrastFactor = NewMaterialExpressionMultiply(
			    Function, Factor, NewMaterialExpressionAbs(Function, NewMaterialExpressionFunctionCall(Function, MathLuminance, {{Result0, 1}})));

			UMaterialExpressionNormalize* CalculatedNormal = NewMaterialExpressionNormalize(
			    Function,
			    NewMaterialExpressionAdd(
			        Function,
			        {Normal,
			         NewMaterialExpressionMultiply(
			             Function, {UVWTangentU, NewMaterialExpressionSubtract(Function, {Result1, 0}, {Result0, 0}), ContrastFactor}),
			         NewMaterialExpressionMultiply(
			             Function, {UVWTangentV, NewMaterialExpressionSubtract(Function, {Result2, 0}, {Result0, 0}), ContrastFactor})}));
			UMaterialExpressionIf* Result = NewMaterialExpressionIfEqual(Function, Factor, 0.0f, Normal, CalculatedNormal);

			NewMaterialExpressionFunctionOutput(Function, TEXT("normal"), Result);
		}

		void BaseTransformCoordinate(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* MathMultiplyFloat4x4Float4 = LoadFunction(TEXT("mdl_math_multiply_float4x4_float4"));
			UMaterialFunction* StateTextureCoordinate     = LoadFunction(TEXT("mdl_state_texture_coordinate"));
			UMaterialFunction* StateTextureTangentU       = LoadFunction(TEXT("mdl_state_texture_tangent_u"));
			UMaterialFunction* StateTextureTangentV       = LoadFunction(TEXT("mdl_state_texture_tangent_v"));

			Function->Description = TEXT("Transform a texture coordinate by a matrix.");

			UMaterialExpressionFunctionInput* Transform0 = NewMaterialExpressionFunctionInput(
			    Function, TEXT("transform_0"), EFunctionInputType::FunctionInput_Vector4, {1.0f, 0.0f, 0.0f, 0.0f});
			UMaterialExpressionFunctionInput* Transform1 = NewMaterialExpressionFunctionInput(
			    Function, TEXT("transform_1"), EFunctionInputType::FunctionInput_Vector4, {0.0f, 1.0f, 0.0f, 0.0f});
			UMaterialExpressionFunctionInput* Transform2 = NewMaterialExpressionFunctionInput(
			    Function, TEXT("transform_2"), EFunctionInputType::FunctionInput_Vector4, {0.0f, 0.0f, 1.0f, 0.0f});
			UMaterialExpressionFunctionInput* Transform3 = NewMaterialExpressionFunctionInput(
			    Function, TEXT("transform_3"), EFunctionInputType::FunctionInput_Vector4, {0.0f, 0.0f, 0.0f, 1.0f});
			UMaterialExpressionFunctionInput* CoordinatePosition =
			    NewMaterialExpressionFunctionInput(Function, TEXT("coordinate.position"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureCoordinate, {}));
			UMaterialExpressionFunctionInput* CoordinateTangentU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("coordinate.tangent_u"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentU, { 0 }));
			UMaterialExpressionFunctionInput* CoordinateTangentV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("coordinate.tangent_v"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentV, { 0 }));

			UMaterialExpressionAppendVector*  ExtendedPosition = NewMaterialExpressionAppendVector(Function, CoordinatePosition, 1.0f);
			UMaterialExpressionComponentMask* Position         = NewMaterialExpressionComponentMask(
                Function,
                NewMaterialExpressionFunctionCall(Function, MathMultiplyFloat4x4Float4,
                                                  {Transform0, Transform1, Transform2, Transform3, ExtendedPosition}),
                7);
			NewMaterialExpressionFunctionOutput(Function, TEXT("position"), Position);
			NewMaterialExpressionFunctionOutput(Function, TEXT("tangent_u"), CoordinateTangentU);
			NewMaterialExpressionFunctionOutput(Function, TEXT("tangent_v"), CoordinateTangentV);
		}

		void BaseVolumeCoefficient(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* MathLog = LoadFunction(TEXT("mdl_math_log_float3"));

			Function->Description = TEXT("Compute a volume coefficient based on distance and value.");

			UMaterialExpressionFunctionInput* Tint =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tint"), EFunctionInputType::FunctionInput_Vector3, {0.0f, 0.0f, 0.0f});
			UMaterialExpressionFunctionInput* Distance =
			    NewMaterialExpressionFunctionInput(Function, TEXT("distance"), EFunctionInputType::FunctionInput_Scalar, 1.0f);

			UMaterialExpressionIf* Coefficient = NewMaterialExpressionIfGreater(
			    Function, Distance, 0.0f,
			    NewMaterialExpressionNegate(
			        Function, NewMaterialExpressionDivide(Function, NewMaterialExpressionFunctionCall(Function, MathLog, {Tint}), Distance)),
			    {0.0f, 0.0f, 0.0f});

			NewMaterialExpressionFunctionOutput(Function, TEXT("coefficient"), Coefficient);
		}

		void BaseWorleyNoiseBumpTexture(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* ImporterApplyNoiseModifications = LoadFunction(TEXT("mdlimporter_apply_noise_modifications"));
			UMaterialFunction* ImporterWorleyNoiseExt          = LoadFunction(TEXT("mdlimporter_worley_noise_ext"));
			UMaterialFunction* MakeFloat3 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat3.MakeFloat3"));
			UMaterialFunction* StateNormal            = LoadFunction(TEXT("mdl_state_normal"));
			UMaterialFunction* StateTextureCoordinate = LoadFunction(TEXT("mdl_state_texture_coordinate"));
			UMaterialFunction* StateTextureTangentU   = LoadFunction(TEXT("mdl_state_texture_tangent_u"));
			UMaterialFunction* StateTextureTangentV   = LoadFunction(TEXT("mdl_state_texture_tangent_v"));

			Function->Description = TEXT("Bump-mapping Worley noise.");

			UMaterialExpressionFunctionInput* UVWPosition =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.position"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureCoordinate, {}));
			UMaterialExpressionFunctionInput* UVWTangentU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.tangent_u"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentU, { 0 }));
			UMaterialExpressionFunctionInput* UVWTangentV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.tangent_v"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentV, { 0 }));
			UMaterialExpressionFunctionInput* Factor =
			    NewMaterialExpressionFunctionInput(Function, TEXT("factor"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* Size =
			    NewMaterialExpressionFunctionInput(Function, TEXT("size"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* Mode =
			    NewMaterialExpressionFunctionInput(Function, TEXT("mode"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* Metric =
			    NewMaterialExpressionFunctionInput(Function, TEXT("metric"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* ApplyMarble = NewMaterialExpressionFunctionInput(
			    Function, TEXT("apply_marble"), EFunctionInputType::FunctionInput_StaticBool, NewMaterialExpressionStaticBool(Function, false));
			UMaterialExpressionFunctionInput* ApplyDent = NewMaterialExpressionFunctionInput(
			    Function, TEXT("apply_dent"), EFunctionInputType::FunctionInput_StaticBool, NewMaterialExpressionStaticBool(Function, false));
			UMaterialExpressionFunctionInput* NoiseDistortion =
			    NewMaterialExpressionFunctionInput(Function, TEXT("noise_distortion"), EFunctionInputType::FunctionInput_Vector3, {0.0f, 0.0f, 0.0f});
			UMaterialExpressionFunctionInput* NoiseThresholdHigh =
			    NewMaterialExpressionFunctionInput(Function, TEXT("noise_threshold_high"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* NoiseThresholdLow =
			    NewMaterialExpressionFunctionInput(Function, TEXT("noise_threshold_low"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* NoiseBands =
			    NewMaterialExpressionFunctionInput(Function, TEXT("noise_bands"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* StepThreshold =
			    NewMaterialExpressionFunctionInput(Function, TEXT("step_threshold"), EFunctionInputType::FunctionInput_Scalar, 0.2f);
			UMaterialExpressionFunctionInput* Edge =
			    NewMaterialExpressionFunctionInput(Function, TEXT("edge"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* Normal = NewMaterialExpressionFunctionInput(
			    Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3, NewMaterialExpressionFunctionCall(Function, StateNormal, {}));

			UMaterialExpressionDivide* Delta = NewMaterialExpressionDivide(Function, NewMaterialExpressionMultiply(Function, 0.1f, Size), NoiseBands);

			UMaterialExpressionDivide*               ScaledPosition0 = NewMaterialExpressionDivide(Function, UVWPosition, Size);
			UMaterialExpressionMaterialFunctionCall* Result0         = NewMaterialExpressionFunctionCall(
                Function, ImporterApplyNoiseModifications,
                {NewMaterialExpressionFunctionCall(Function, ImporterWorleyNoiseExt,
                                                   {ScaledPosition0, NoiseDistortion, StepThreshold, Mode, Metric, {}}),
                 NewMaterialExpressionComponentMask(Function, ScaledPosition0, 1), ApplyMarble, ApplyDent, NoiseThresholdHigh, NoiseThresholdLow,
                 NoiseBands});

			UMaterialExpressionDivide* ScaledPosition1 = NewMaterialExpressionDivide(
			    Function,
			    NewMaterialExpressionAdd(Function, UVWPosition, NewMaterialExpressionFunctionCall(Function, MakeFloat3, {Delta, 0.0f, 0.0f})), Size);
			UMaterialExpressionMaterialFunctionCall* Result1 = NewMaterialExpressionFunctionCall(
			    Function, ImporterApplyNoiseModifications,
			    {NewMaterialExpressionFunctionCall(Function, ImporterWorleyNoiseExt,
			                                       {ScaledPosition1, NoiseDistortion, StepThreshold, Mode, Metric, {}}),
			     NewMaterialExpressionComponentMask(Function, ScaledPosition1, 1), ApplyMarble, ApplyDent, NoiseThresholdHigh, NoiseThresholdLow,
			     NoiseBands});

			UMaterialExpressionDivide* ScaledPosition2 = NewMaterialExpressionDivide(
			    Function,
			    NewMaterialExpressionAdd(Function, UVWPosition, NewMaterialExpressionFunctionCall(Function, MakeFloat3, {0.0f, Delta, 0.0f})), Size);
			UMaterialExpressionMaterialFunctionCall* Result2 = NewMaterialExpressionFunctionCall(
			    Function, ImporterApplyNoiseModifications,
			    {NewMaterialExpressionFunctionCall(Function, ImporterWorleyNoiseExt,
			                                       {ScaledPosition2, NoiseDistortion, StepThreshold, Mode, Metric, {}}),
			     NewMaterialExpressionComponentMask(Function, ScaledPosition2, 1), ApplyMarble, ApplyDent, NoiseThresholdHigh, NoiseThresholdLow,
			     NoiseBands});

			UMaterialExpressionDivide* ScaledPosition3 = NewMaterialExpressionDivide(
			    Function,
			    NewMaterialExpressionAdd(Function, UVWPosition, NewMaterialExpressionFunctionCall(Function, MakeFloat3, {0.0f, 0.0f, Delta})), Size);
			UMaterialExpressionMaterialFunctionCall* Result3 = NewMaterialExpressionFunctionCall(
			    Function, ImporterApplyNoiseModifications,
			    {NewMaterialExpressionFunctionCall(Function, ImporterWorleyNoiseExt,
			                                       {ScaledPosition3, NoiseDistortion, StepThreshold, Mode, Metric, {}}),
			     NewMaterialExpressionComponentMask(Function, ScaledPosition3, 1), ApplyMarble, ApplyDent, NoiseThresholdHigh, NoiseThresholdLow,
			     NoiseBands});

			UMaterialExpressionSubtract* BumpFactor = NewMaterialExpressionNegate(Function, Factor);
			UMaterialExpressionIf*       Result     = NewMaterialExpressionIfEqual(
                Function, Factor, 0.0f, Normal,
                NewMaterialExpressionIfEqual(
                    Function, Size, 0.0f, Normal,
                    NewMaterialExpressionNormalize(
                        Function,
                        NewMaterialExpressionAdd(
                            Function,
                            {NewMaterialExpressionMultiply(
                                 Function,
                                 Normal,
                                 NewMaterialExpressionAdd(
                                     Function,
                                     NewMaterialExpressionAbs(
                                         Function, NewMaterialExpressionMultiply(Function, NewMaterialExpressionSubtract(Function, Result3, Result0),
                                                                                 BumpFactor)),
                                     1.0f)),
                             NewMaterialExpressionMultiply(Function,
                                                           {UVWTangentU, NewMaterialExpressionSubtract(Function, Result1, Result0), BumpFactor}),
                             NewMaterialExpressionMultiply(Function,
                                                           {UVWTangentV, NewMaterialExpressionSubtract(Function, Result2, Result0), BumpFactor})}))));

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Result);
		}

		void BaseWorleyNoiseTexture(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* ImporterApplyNoiseModifications = LoadFunction(TEXT("mdlimporter_apply_noise_modifications"));
			UMaterialFunction* ImporterWorleyNoiseExt          = LoadFunction(TEXT("mdlimporter_worley_noise_ext"));
			UMaterialFunction* MathLuminance                   = LoadFunction(TEXT("mdl_math_luminance"));
			UMaterialFunction* StateTextureCoordinate          = LoadFunction(TEXT("mdl_state_texture_coordinate"));
			UMaterialFunction* StateTextureTangentU            = LoadFunction(TEXT("mdl_state_texture_tangent_u"));
			UMaterialFunction* StateTextureTangentV            = LoadFunction(TEXT("mdl_state_texture_tangent_v"));

			Function->Description = TEXT("Color Worley noise.");

			UMaterialExpressionFunctionInput* UVWPosition =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.position"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureCoordinate, {}));
			UMaterialExpressionFunctionInput* UVWTangentU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.tangent_u"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentU, { 0 }));
			UMaterialExpressionFunctionInput* UVWTangentV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("uvw.tangent_v"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentV, { 0 }));
			UMaterialExpressionFunctionInput* Color1 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("color1"), EFunctionInputType::FunctionInput_Vector3, {0.0f, 0.0f, 0.0f});
			UMaterialExpressionFunctionInput* Color2 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("color2"), EFunctionInputType::FunctionInput_Vector3, {0.0f, 0.0f, 0.0f});
			UMaterialExpressionFunctionInput* Size =
			    NewMaterialExpressionFunctionInput(Function, TEXT("size"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* Mode =
			    NewMaterialExpressionFunctionInput(Function, TEXT("mode"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* Metric =
			    NewMaterialExpressionFunctionInput(Function, TEXT("metric"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* ApplyMarble = NewMaterialExpressionFunctionInput(
			    Function, TEXT("apply_marble"), EFunctionInputType::FunctionInput_StaticBool, NewMaterialExpressionStaticBool(Function, false));
			UMaterialExpressionFunctionInput* ApplyDent = NewMaterialExpressionFunctionInput(
			    Function, TEXT("apply_dent"), EFunctionInputType::FunctionInput_StaticBool, NewMaterialExpressionStaticBool(Function, false));
			UMaterialExpressionFunctionInput* NoiseDistortion =
			    NewMaterialExpressionFunctionInput(Function, TEXT("noise_distortion"), EFunctionInputType::FunctionInput_Vector3, {0.0f, 0.0f, 0.0f});
			UMaterialExpressionFunctionInput* NoiseThresholdHigh =
			    NewMaterialExpressionFunctionInput(Function, TEXT("noise_threshold_high"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* NoiseThresholdLow =
			    NewMaterialExpressionFunctionInput(Function, TEXT("noise_threshold_low"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* NoiseBands =
			    NewMaterialExpressionFunctionInput(Function, TEXT("noise_bands"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* StepThreshold =
			    NewMaterialExpressionFunctionInput(Function, TEXT("noise_threshold"), EFunctionInputType::FunctionInput_Scalar, 0.2f);
			UMaterialExpressionFunctionInput* Edge =
			    NewMaterialExpressionFunctionInput(Function, TEXT("edge"), EFunctionInputType::FunctionInput_Scalar, 1.0f);

			UMaterialExpressionDivide* ScaledPosition = NewMaterialExpressionDivide(Function, UVWPosition, Size);
			UMaterialExpressionIf*     Alpha          = NewMaterialExpressionIfEqual(
                Function, Size, 0.0f, 0.0f,
                NewMaterialExpressionFunctionCall(
                    Function, ImporterApplyNoiseModifications,
                    {NewMaterialExpressionFunctionCall(Function, ImporterWorleyNoiseExt,
                                                       {ScaledPosition, NoiseDistortion, StepThreshold, Mode, Metric, {}}),
                     NewMaterialExpressionComponentMask(Function, ScaledPosition, 1), ApplyMarble, ApplyDent, NoiseThresholdHigh, NoiseThresholdLow,
                     NoiseBands}));
			UMaterialExpressionLinearInterpolate*    Tint = NewMaterialExpressionLinearInterpolate(Function, Color1, Color2, Alpha);
			UMaterialExpressionMaterialFunctionCall* Mono = NewMaterialExpressionFunctionCall(Function, MathLuminance, {Tint});

			NewMaterialExpressionFunctionOutput(Function, TEXT("tint"), Tint);
			NewMaterialExpressionFunctionOutput(Function, TEXT("mono"), Mono);
		}

		void DFAnisotropicVDF(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* MakeFloat4 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat4.MakeFloat4"));

			Function->Description = TEXT("Volume light distribution with directional bias.");

			UMaterialExpressionFunctionInput* DirectionalBias =
			    NewMaterialExpressionFunctionInput(Function, TEXT("roughness_u"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			// UMaterialExpressionFunctionInput* Normal = NewMaterialExpressionFunctionInput(Function, TEXT("normal"),
			// EFunctionInputType::FunctionInput_Vector3, NewMaterialExpressionFunctionCall(Function, StateNormal, {}));

			// Needs to determine some meaningfull stuff for a VDF !!
			UMaterialExpressionMaterialFunctionCall* VDF =
			    NewMaterialExpressionFunctionCall(Function, MakeFloat4, {DirectionalBias, DirectionalBias, DirectionalBias, DirectionalBias});

			NewMaterialExpressionFunctionOutput(Function, TEXT("vdf"), VDF);
		}

		void DFBackscatteringGlossyReflectionBSDF(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* StateNormal          = LoadFunction(TEXT("mdl_state_normal"));
			UMaterialFunction* StateTextureTangentU = LoadFunction(TEXT("mdl_state_texture_tangent_u"));

			Function->Description = TEXT("Backscattering glossy reflection.");

			UMaterialExpressionFunctionInput* RoughnessU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("roughness_u"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* RoughnessV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("roughness_v"), EFunctionInputType::FunctionInput_Scalar, RoughnessU);
			UMaterialExpressionFunctionInput* Tint =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tint"), EFunctionInputType::FunctionInput_Vector3, {1.0f, 1.0f, 1.0f});
			UMaterialExpressionFunctionInput* TangentU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tangent_u"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentU, { 0 }));
			UMaterialExpressionFunctionInput* Normal = NewMaterialExpressionFunctionInput(
			    Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3, NewMaterialExpressionFunctionCall(Function, StateNormal, {}));

			UMaterialExpressionDivide* Roughness =
			    NewMaterialExpressionDivide(Function, NewMaterialExpressionAdd(Function, RoughnessU, RoughnessV), 2.0f);
			UMaterialExpressionTransform* BackscatteringNormal = NewMaterialExpressionTransform(
			    Function, NewMaterialExpression<UMaterialExpressionCameraVectorWS>(Function), TRANSFORMSOURCE_World, TRANSFORM_Tangent);

			UMaterialExpressionMakeMaterialAttributes* BSDF =
			    NewMaterialExpressionMakeMaterialAttributes(Function, Tint, 1.0f, 0.0f, Roughness, {}, {}, {}, BackscatteringNormal);

			NewMaterialExpressionFunctionOutput(Function, TEXT("bsdf"), BSDF);
		}

		void DFCustomCurveLayer(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* StateNormal            = LoadFunction(TEXT("mdl_state_normal"));
			UMaterialFunction* ImporterBlendClearCoat = LoadFunction(TEXT("mdlimporter_blend_clear_coat"));

			Function->Description = TEXT(
			    "BSDF as a layer on top of another elemental or compound BSDF according to weight and a Schlick-style directional - dependent curve function.The base is weighted with 1 - (weight*curve()).");

			UMaterialExpression* DefaultMaterialAttributes = NewMaterialExpressionMakeMaterialAttributes(Function);

			UMaterialExpressionFunctionInput* NormalReflectivity =
			    NewMaterialExpressionFunctionInput(Function, TEXT("normal_reflectivity"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* GrazingReflectivity =
			    NewMaterialExpressionFunctionInput(Function, TEXT("grazing_reflectivity"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* Exponent =
			    NewMaterialExpressionFunctionInput(Function, TEXT("exponent"), EFunctionInputType::FunctionInput_Scalar, 5.0f);
			UMaterialExpressionFunctionInput* Weight =
			    NewMaterialExpressionFunctionInput(Function, TEXT("weight"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* Layer = NewMaterialExpressionFunctionInput(
			    Function, TEXT("layer"), EFunctionInputType::FunctionInput_MaterialAttributes, DefaultMaterialAttributes);
			UMaterialExpressionFunctionInput* Base = NewMaterialExpressionFunctionInput(
			    Function, TEXT("base"), EFunctionInputType::FunctionInput_MaterialAttributes, DefaultMaterialAttributes);
			UMaterialExpressionFunctionInput* Normal = NewMaterialExpressionFunctionInput(
			    Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3, NewMaterialExpressionFunctionCall(Function, StateNormal, {}));

			UMaterialExpressionClamp* ClampedWeight = NewMaterialExpressionClamp(Function, Weight, 0.0f, 1.0f);

			UMaterialExpressionFresnel* Fresnel = NewMaterialExpressionFresnel(
			    Function, Exponent, 0.0f, NewMaterialExpressionTransform(Function, Normal, TRANSFORMSOURCE_Tangent, TRANSFORM_World));
			UMaterialExpressionLinearInterpolate* Reflectivity =
			    NewMaterialExpressionLinearInterpolate(Function, NormalReflectivity, GrazingReflectivity, Fresnel);
			UMaterialExpressionMultiply* Alpha = NewMaterialExpressionMultiply(Function, ClampedWeight, Reflectivity);

			UMaterialExpressionMaterialFunctionCall* BSDF =
			    NewMaterialExpressionFunctionCall(Function, ImporterBlendClearCoat, {Base, Layer, Alpha, Normal});

			NewMaterialExpressionFunctionOutput(Function, TEXT("bsdf"), BSDF);
		}

		/*static*/ void DFDiffuseEDF(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("Uniform light emission in all directions.");

			UMaterialExpressionMakeMaterialAttributes* EDF =
			    NewMaterialExpressionMakeMaterialAttributes(Function, {}, {}, 0.0f, {}, {1.0f, 1.0f, 1.0f});

			NewMaterialExpressionFunctionOutput(Function, TEXT("edf"), EDF);
		}

		void DFDiffuseReflectionBSDF(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* StateNormal = LoadFunction(TEXT("mdl_state_normal"));

			Function->Description = TEXT("Lambertian reflection extended by the Oren-Nayar microfacet model.");

			UMaterialExpressionFunctionInput* Tint =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tint"), EFunctionInputType::FunctionInput_Vector3, {1.0f, 1.0f, 1.0f});
			UMaterialExpressionFunctionInput* Roughness =
			    NewMaterialExpressionFunctionInput(Function, TEXT("roughness"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* Normal = NewMaterialExpressionFunctionInput(
			    Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3, NewMaterialExpressionFunctionCall(Function, StateNormal, {}));

			UMaterialExpressionMakeMaterialAttributes* BSDF =
			    NewMaterialExpressionMakeMaterialAttributes(Function, Tint, 0.0f, 0.0f, 1.0f, {}, 1.0f, {}, Normal);

			NewMaterialExpressionFunctionOutput(Function, TEXT("bsdf"), BSDF);
		}

		void DFDiffuseTransmissionBSDF(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* StateNormal = LoadFunction(TEXT("mdl_state_normal"));

			Function->Description = TEXT("Pure diffuse transmission of light through a surface.");

			UMaterialExpressionFunctionInput* Tint =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tint"), EFunctionInputType::FunctionInput_Vector3, {1.0f, 1.0f, 1.0f});
			UMaterialExpressionFunctionInput* Normal = NewMaterialExpressionFunctionInput(
			    Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3, NewMaterialExpressionFunctionCall(Function, StateNormal, {}));

			UMaterialExpressionMakeMaterialAttributes* BSDF =
			    NewMaterialExpressionMakeMaterialAttributes(Function, Tint, {}, 0.0f, {}, {}, {}, {}, Normal, {}, Tint);

			NewMaterialExpressionFunctionOutput(Function, TEXT("bsdf"), BSDF);
		}

		void DFDirectionalFactor(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* StateNormal = LoadFunction(TEXT("mdl_state_normal"));

			Function->Description = TEXT("Directional modifier.");

			UMaterialExpressionFunctionInput* NormalTint =
			    NewMaterialExpressionFunctionInput(Function, TEXT("normal_tint"), EFunctionInputType::FunctionInput_Vector3, {1.0f, 1.0f, 1.0f});
			UMaterialExpressionFunctionInput* GrazingTint =
			    NewMaterialExpressionFunctionInput(Function, TEXT("grazing_tint"), EFunctionInputType::FunctionInput_Vector3, {1.0f, 1.0f, 1.0f});
			UMaterialExpressionFunctionInput* Exponent =
			    NewMaterialExpressionFunctionInput(Function, TEXT("exponent"), EFunctionInputType::FunctionInput_Scalar, 5.0f);
			UMaterialExpressionFunctionInput* Base = NewMaterialExpressionFunctionInput(
			    Function, TEXT("base"), EFunctionInputType::FunctionInput_MaterialAttributes, NewMaterialExpressionMakeMaterialAttributes(Function));
			UMaterialExpressionFunctionInput* Normal = NewMaterialExpressionFunctionInput(
			    Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3, NewMaterialExpressionFunctionCall(Function, StateNormal, {}));

			UMaterialExpressionLinearInterpolate* NewBaseColor =
			    NewMaterialExpressionLinearInterpolate(Function, NormalTint, GrazingTint, NewMaterialExpressionFresnel(Function, Exponent, {}, {}));
			UMaterialExpressionMaterialFunctionCall* BSDF = NewMaterialExpressionFunctionCall(
			    Function, LoadFunction(TEXT("/Engine/Functions/MaterialLayerFunctions"), TEXT("MatLayerBlend_MultiplyBaseColor")),
			    {Base, NewBaseColor, {}});

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), BSDF);
		}

		void DFFresnelLayer(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* ImporterBlendClearCoat = LoadFunction(TEXT("mdlimporter_blend_clear_coat"));
			UMaterialFunction* StateNormal            = LoadFunction(TEXT("mdl_state_normal"));

			Function->Description = TEXT(
			    "Add an elemental or compound BSDF as a layer on top of another elemental or compound BSDF according to weight and a Fresnel term using a dedicated index of refraction for the layer");

			UMaterialExpression* DefaultMaterialAttributes = NewMaterialExpressionMakeMaterialAttributes(Function);

			UMaterialExpressionFunctionInput* IOR =
			    NewMaterialExpressionFunctionInput(Function, TEXT("ior"), EFunctionInputType::FunctionInput_Vector3);
			UMaterialExpressionFunctionInput* Weight =
			    NewMaterialExpressionFunctionInput(Function, TEXT("weight"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* Layer = NewMaterialExpressionFunctionInput(
			    Function, TEXT("layer"), EFunctionInputType::FunctionInput_MaterialAttributes, DefaultMaterialAttributes);
			UMaterialExpressionFunctionInput* Base = NewMaterialExpressionFunctionInput(
			    Function, TEXT("base"), EFunctionInputType::FunctionInput_MaterialAttributes, DefaultMaterialAttributes);
			UMaterialExpressionFunctionInput* Normal = NewMaterialExpressionFunctionInput(
			    Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3, NewMaterialExpressionFunctionCall(Function, StateNormal, {}));

			UMaterialExpressionComponentMask* ScalarIOR = NewMaterialExpressionComponentMask(Function, IOR, 2);

			// Use Schlick's approximation for the Exponent: https://en.wikipedia.org/wiki/Schlick%27s_approximation
			UMaterialExpressionMultiply* BaseReflectFraction =
			    NewMaterialExpressionSquare(Function,
			                                NewMaterialExpressionDivide(Function, NewMaterialExpressionOneMinus(Function, ScalarIOR),
			                                                            NewMaterialExpressionAdd(Function, 1.0f, ScalarIOR)));

			UMaterialExpressionMultiply* Alpha = NewMaterialExpressionMultiply(
			    Function,
			    NewMaterialExpressionClamp(Function, Weight, 0.0f, 1.0f),
			    NewMaterialExpressionFresnel(Function, 5.0f, BaseReflectFraction,
			                                 NewMaterialExpressionTransform(Function, Normal, TRANSFORMSOURCE_Tangent, TRANSFORM_World)));

			UMaterialExpressionMaterialFunctionCall* BSDF =
			    NewMaterialExpressionFunctionCall(Function, ImporterBlendClearCoat, {Base, Layer, Alpha, Normal});

			NewMaterialExpressionFunctionOutput(Function, TEXT("bsdf"), BSDF);
		}

		/*static*/ void DFLightProfileMaximum(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description =
			    TEXT("Returns the maximumintensity in this light profile. A lookup on an invalid light profile reference returns zero.");

			// dummy implementation!
			// we just always return 1.0f
			UMaterialExpressionFunctionInput* Profile = NewMaterialExpressionFunctionInput(
			    Function, TEXT("profile"), EFunctionInputType::FunctionInput_Texture2D, NewMaterialExpressionTextureObject(Function, nullptr));

			NewMaterialExpressionFunctionOutput(Function, TEXT("maximum"), 1.0f);
		}

		/*static*/ void DFLightProfilePower(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description =
			    TEXT("Returns the power emitted by this light profile. A lookup on an invalid light profile reference returns zero.");

			// dummy implementation!
			// we just always return 1.0f
			UMaterialExpressionFunctionInput* Profile = NewMaterialExpressionFunctionInput(
			    Function, TEXT("profile"), EFunctionInputType::FunctionInput_Texture2D, NewMaterialExpressionTextureObject(Function, nullptr));

			NewMaterialExpressionFunctionOutput(Function, TEXT("power"), 1.0f);
		}

		void DFMeasuredBSDF(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* StateNormal = LoadFunction(TEXT("mdl_state_normal"));

			Function->Description = TEXT("General isotropic reflection and transmission based on measured data.");

			// a bsdf_measurement essentially is a texture, that needs special accessing functions!
			// 'til its implemented, we simply use a color here, which is totally wrong, but we survive that way
			UMaterialExpressionFunctionInput* Measurement =
			    NewMaterialExpressionFunctionInput(Function, TEXT("measurement"), EFunctionInputType::FunctionInput_Vector3);
			UMaterialExpressionFunctionInput* Multiplier =
			    NewMaterialExpressionFunctionInput(Function, TEXT("multiplier"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* Mode =
			    NewMaterialExpressionFunctionInput(Function, TEXT("mode"), EFunctionInputType::FunctionInput_Scalar, (int)EScatterMode::Reflect);
			UMaterialExpressionFunctionInput* Normal = NewMaterialExpressionFunctionInput(
			    Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3, NewMaterialExpressionFunctionCall(Function, StateNormal, {}));

			UMaterialExpressionMultiply* BaseColor = NewMaterialExpressionMultiply(Function, Measurement, Multiplier);

			UMaterialExpressionFresnel* Fresnel = NewMaterialExpressionFresnel(Function, 5.0f, {}, {});
			UMaterialExpressionIf*      Opacity = NewMaterialExpressionSwitch(Function, Mode, {1.0f, Fresnel, Fresnel});

			UMaterialExpressionMakeMaterialAttributes* BSDF =
			    NewMaterialExpressionMakeMaterialAttributes(Function, BaseColor, {}, {}, {}, {}, Opacity, {}, Normal);

			NewMaterialExpressionFunctionOutput(Function, TEXT("bsdf"), BSDF);
		}

		void DFMeasuredEDF(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* StateTextureTangentU = LoadFunction(TEXT("mdl_state_texture_tangent_u"));

			Function->Description = TEXT("Light distribution defined by a profile.");

			// dummy implementation!
			// we just return a uniform light emission in all directions
			UMaterialExpressionFunctionInput* Profile = NewMaterialExpressionFunctionInput(
			    Function, TEXT("profile"), EFunctionInputType::FunctionInput_Texture2D, NewMaterialExpressionTextureObject(Function, nullptr));
			UMaterialExpressionFunctionInput* Multiplier =
			    NewMaterialExpressionFunctionInput(Function, TEXT("multiplier"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* GlobalDistribution =
			    NewMaterialExpressionFunctionInput(Function, TEXT("global_distribution"), EFunctionInputType::FunctionInput_StaticBool, true);
			UMaterialExpressionFunctionInput* GlobalFrame_0 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("global_frame_0"), EFunctionInputType::FunctionInput_Vector3, {1.0f, 0.0f, 0.0f});
			UMaterialExpressionFunctionInput* GlobalFrame_1 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("global_frame_1"), EFunctionInputType::FunctionInput_Vector3, {0.0f, 1.0f, 0.0f});
			UMaterialExpressionFunctionInput* GlobalFrame_2 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("global_frame_2"), EFunctionInputType::FunctionInput_Vector3, {0.0f, 0.0f, 1.0f});
			UMaterialExpressionFunctionInput* TangentU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tangent_u"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentU, { 0 }));

			UMaterialExpressionMakeMaterialAttributes* EDF =
			    NewMaterialExpressionMakeMaterialAttributes(Function, {}, {}, 0.0f, {}, {1.0f, 1.0f, 1.0f});

			NewMaterialExpressionFunctionOutput(Function, TEXT("edf"), EDF);
		}

		void DFMeasuredCurveFactor(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 < ArraySize);

			UMaterialFunction* StateNormal = LoadFunction(TEXT("mdl_state_normal"));

			Function->Description = TEXT("Modifier weighting a base BSDF based on a measured reflection curve.");

			TArray<UMaterialExpressionFunctionInput*> Colors;
			for (int32 i = 0; i < ArraySize; i++)
			{
				Colors.Add(NewMaterialExpressionFunctionInput(Function, TEXT("color"), EFunctionInputType::FunctionInput_Vector3));
			}
			if (1 < ArraySize)
			{
				for (int32 i = 0; i < ArraySize; i++)
				{
					FString Buf          = Colors[i]->InputName.ToString() + TEXT("_") + FString::FromInt(i);
					Colors[i]->InputName = *Buf;
				}
			}
			UMaterialExpressionFunctionInput* Base = NewMaterialExpressionFunctionInput(
			    Function, TEXT("bsdf"), EFunctionInputType::FunctionInput_MaterialAttributes, NewMaterialExpressionMakeMaterialAttributes(Function));
			UMaterialExpressionFunctionInput* Normal = NewMaterialExpressionFunctionInput(
			    Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3, NewMaterialExpressionFunctionCall(Function, StateNormal, {}));

			UMaterialExpression* Color;
			if (1 == ArraySize)
			{
				Color = Colors[0];
			}
			else
			{
				UMaterialExpressionMultiply* Z = NewMaterialExpressionMultiply(
				    Function,
				    NewMaterialExpressionArccosine(
				        Function,
				        NewMaterialExpressionClamp(Function,
				                                   NewMaterialExpressionDotProduct(
				                                       Function,
				                                       NewMaterialExpressionTransform(Function, Normal, TRANSFORMSOURCE_Tangent, TRANSFORM_World),
				                                       NewMaterialExpression<UMaterialExpressionCameraVectorWS>(Function)),
				                                   0.0f,
				                                   1.0f)),
				    ArraySize * 2.0f / PI);  // multiply with the number of curveValues, to get an index

				TArray<FMaterialExpressionConnection> ColorConnections;
				ColorConnections.Reserve(Colors.Num());
				for (int32 i = 0; i < Colors.Num(); i++)
				{
					ColorConnections.Emplace(Colors[i]);
				}

				Color = NewMaterialExpressionLinearInterpolate(Function, ColorConnections, Z);
			}

			UMaterialExpressionMaterialFunctionCall* BSDF = NewMaterialExpressionFunctionCall(
			    Function, LoadFunction(TEXT("/Engine/Functions/MaterialLayerFunctions"), TEXT("MatLayerBlend_MultiplyBaseColor")), {Base, Color, {}});

			NewMaterialExpressionFunctionOutput(Function, TEXT("bsdf"), BSDF);
		}

		void DFMicrofacetBeckmannSmithBSDF(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* DFSimpleGlossyBSDF   = LoadFunction(TEXT("mdl_df_simple_glossy_bsdf"));
			UMaterialFunction* StateNormal          = LoadFunction(TEXT("mdl_state_normal"));
			UMaterialFunction* StateTextureTangentU = LoadFunction(TEXT("mdl_state_texture_tangent_u"));

			Function->Description = TEXT("Glossy reflection and transmission.");

			UMaterialExpressionFunctionInput* RoughnessU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("roughness_u"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* RoughnessV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("roughness_v"), EFunctionInputType::FunctionInput_Scalar, RoughnessU);
			UMaterialExpressionFunctionInput* Tint =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tint"), EFunctionInputType::FunctionInput_Vector3, {1.0f, 1.0f, 1.0f});
			UMaterialExpressionFunctionInput* TangentU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tangent_u"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentU, { 0 }));
			UMaterialExpressionFunctionInput* Mode =
			    NewMaterialExpressionFunctionInput(Function, TEXT("mode"), EFunctionInputType::FunctionInput_Scalar, (int)EScatterMode::Reflect);
			UMaterialExpressionFunctionInput* Normal = NewMaterialExpressionFunctionInput(
			    Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3, NewMaterialExpressionFunctionCall(Function, StateNormal, {}));

			UMaterialExpressionMaterialFunctionCall* BSDF =
			    NewMaterialExpressionFunctionCall(Function, DFSimpleGlossyBSDF, {RoughnessU, RoughnessV, Tint, TangentU, Mode, Normal});

			NewMaterialExpressionFunctionOutput(Function, TEXT("bsdf"), BSDF);
		}

		void DFMicrofacetBeckmannVCavitiesBSDF(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* DFSimpleGlossyBSDF   = LoadFunction(TEXT("mdl_df_simple_glossy_bsdf"));
			UMaterialFunction* StateNormal          = LoadFunction(TEXT("mdl_state_normal"));
			UMaterialFunction* StateTextureTangentU = LoadFunction(TEXT("mdl_state_texture_tangent_u"));

			Function->Description = TEXT("Glossy reflection and transmission.");

			UMaterialExpressionFunctionInput* RoughnessU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("roughness_u"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* RoughnessV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("roughness_v"), EFunctionInputType::FunctionInput_Scalar, RoughnessU);
			UMaterialExpressionFunctionInput* Tint =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tint"), EFunctionInputType::FunctionInput_Vector3, {1.0f, 1.0f, 1.0f});
			UMaterialExpressionFunctionInput* TangentU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tangent_u"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentU, { 0 }));
			UMaterialExpressionFunctionInput* Mode =
			    NewMaterialExpressionFunctionInput(Function, TEXT("mode"), EFunctionInputType::FunctionInput_Scalar, (int)EScatterMode::Reflect);
			UMaterialExpressionFunctionInput* Normal = NewMaterialExpressionFunctionInput(
			    Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3, NewMaterialExpressionFunctionCall(Function, StateNormal, {}));

			UMaterialExpressionMaterialFunctionCall* BSDF =
			    NewMaterialExpressionFunctionCall(Function, DFSimpleGlossyBSDF, {RoughnessU, RoughnessV, Tint, TangentU, Mode, Normal});

			NewMaterialExpressionFunctionOutput(Function, TEXT("bsdf"), BSDF);
		}

		void DFMicrofacetGGXSmithBSDF(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* DFSimpleGlossyBSDF   = LoadFunction(TEXT("mdl_df_simple_glossy_bsdf"));
			UMaterialFunction* StateNormal          = LoadFunction(TEXT("mdl_state_normal"));
			UMaterialFunction* StateTextureTangentU = LoadFunction(TEXT("mdl_state_texture_tangent_u"));

			Function->Description = TEXT("Glossy reflection and transmission.");

			UMaterialExpressionFunctionInput* RoughnessU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("roughness_u"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* RoughnessV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("roughness_v"), EFunctionInputType::FunctionInput_Scalar, RoughnessU);
			UMaterialExpressionFunctionInput* Tint =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tint"), EFunctionInputType::FunctionInput_Vector3, {1.0f, 1.0f, 1.0f});
			UMaterialExpressionFunctionInput* TangentU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tangent_u"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentU, { 0 }));
			UMaterialExpressionFunctionInput* Mode =
			    NewMaterialExpressionFunctionInput(Function, TEXT("mode"), EFunctionInputType::FunctionInput_Scalar, (int)EScatterMode::Reflect);
			UMaterialExpressionFunctionInput* Normal = NewMaterialExpressionFunctionInput(
			    Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3, NewMaterialExpressionFunctionCall(Function, StateNormal, {}));

			UMaterialExpressionMaterialFunctionCall* BSDF =
			    NewMaterialExpressionFunctionCall(Function, DFSimpleGlossyBSDF, {RoughnessU, RoughnessV, Tint, TangentU, Mode, Normal});

			NewMaterialExpressionFunctionOutput(Function, TEXT("bsdf"), BSDF);
		}

		void DFMicrofacetGGXVCavitiesBSDF(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* DFSimpleGlossyBSDF   = LoadFunction(TEXT("mdl_df_simple_glossy_bsdf"));
			UMaterialFunction* StateNormal          = LoadFunction(TEXT("mdl_state_normal"));
			UMaterialFunction* StateTextureTangentU = LoadFunction(TEXT("mdl_state_texture_tangent_u"));

			Function->Description = TEXT("Glossy reflection and transmission.");

			UMaterialExpressionFunctionInput* RoughnessU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("roughness_u"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* RoughnessV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("roughness_v"), EFunctionInputType::FunctionInput_Scalar, RoughnessU);
			UMaterialExpressionFunctionInput* Tint =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tint"), EFunctionInputType::FunctionInput_Vector3, {1.0f, 1.0f, 1.0f});
			UMaterialExpressionFunctionInput* TangentU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tangent_u"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentU, { 0 }));
			UMaterialExpressionFunctionInput* Mode =
			    NewMaterialExpressionFunctionInput(Function, TEXT("mode"), EFunctionInputType::FunctionInput_Scalar, (int)EScatterMode::Reflect);
			UMaterialExpressionFunctionInput* Normal = NewMaterialExpressionFunctionInput(
			    Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3, NewMaterialExpressionFunctionCall(Function, StateNormal, {}));

			UMaterialExpressionMaterialFunctionCall* BSDF =
			    NewMaterialExpressionFunctionCall(Function, DFSimpleGlossyBSDF, {RoughnessU, RoughnessV, Tint, TangentU, Mode, Normal});

			NewMaterialExpressionFunctionOutput(Function, TEXT("bsdf"), BSDF);
		}

		void DFNormalizedMix(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 < ArraySize);

			UMaterialFunction* MatLayerBlend_Standard =
			    LoadFunction(TEXT("/Engine/Functions/MaterialLayerFunctions"), TEXT("MatLayerBlend_Standard"));
			UMaterialFunction* StateNormal = LoadFunction(TEXT("mdl_state_normal"));

			Function->Description = TEXT(
			    "Mix N elemental or compound distribution functions based on the weights defined in the components. If the sum of the weights exceeds 1.0, they are normalized.");

			TArray<UMaterialExpressionFunctionInput*> Weights, Components;
			for (int32 i = 0; i < ArraySize; i++)
			{
				Weights.Add(NewMaterialExpressionFunctionInput(Function, TEXT("weight"), EFunctionInputType::FunctionInput_Scalar, 0.0f));
				Components.Add(NewMaterialExpressionFunctionInput(Function, TEXT("component"), EFunctionInputType::FunctionInput_MaterialAttributes,
				                                                  NewMaterialExpressionMakeMaterialAttributes(Function)));
			}
			if (1 < ArraySize)
			{
				for (int32 i = 0; i < ArraySize; i++)
				{
					FString Appendix         = TEXT("_") + FString::FromInt(i);
					FString Buf              = Weights[i]->InputName.ToString() + Appendix;
					Weights[i]->InputName    = *Buf;
					Buf                      = Components[i]->InputName.ToString() + Appendix;
					Components[i]->InputName = *Buf;
				}
			}
			UMaterialExpressionFunctionInput* Normal = NewMaterialExpressionFunctionInput(
			    Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3, NewMaterialExpressionFunctionCall(Function, StateNormal, {}));

			TArray<UMaterialExpressionClamp*> ClampedWeights;
			for (int32 i = 0; i < ArraySize; i++)
			{
				ClampedWeights.Add(NewMaterialExpressionClamp(Function, Weights[i], 0.0f, 1.0f));
			}

			TPair<UMaterialExpressionClamp*, UMaterialExpression*> Mix = MixAttributes(Function, MatLayerBlend_Standard, ClampedWeights, Components);

			UMaterialExpressionMakeMaterialAttributes* DefaultBSDF =
			    NewMaterialExpressionMakeMaterialAttributes(Function, {0.0f, 0.0f, 0.0f}, 0.0f, 0.0f, 0.0f);
			UMaterialExpressionMaterialFunctionCall* BSDF =
			    NewMaterialExpressionFunctionCall(Function, MatLayerBlend_Standard, {DefaultBSDF, Mix.Value, Mix.Key});

			NewMaterialExpressionFunctionOutput(Function, TEXT("bsdf"), BSDF);
		}

		void DFSimpleGlossyBSDF(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* StateNormal          = LoadFunction(TEXT("mdl_state_normal"));
			UMaterialFunction* StateTextureTangentU = LoadFunction(TEXT("mdl_state_texture_tangent_u"));

			Function->Description = TEXT("Glossy reflection and transmission.");

			UMaterialExpressionFunctionInput* RoughnessU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("roughness_u"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* RoughnessV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("roughness_v"), EFunctionInputType::FunctionInput_Scalar, RoughnessU);
			UMaterialExpressionFunctionInput* Tint =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tint"), EFunctionInputType::FunctionInput_Vector3, {1.0f, 1.0f, 1.0f});
			UMaterialExpressionFunctionInput* TangentU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tangent_u"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentU, { 0 }));
			UMaterialExpressionFunctionInput* Mode =
			    NewMaterialExpressionFunctionInput(Function, TEXT("mode"), EFunctionInputType::FunctionInput_Scalar, (int)EScatterMode::Reflect);
			UMaterialExpressionFunctionInput* Normal = NewMaterialExpressionFunctionInput(
			    Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3, NewMaterialExpressionFunctionCall(Function, StateNormal, {}));

			// assume material to be metallic, if the mode is (int)EScatterMode::Reflect (opaque!) and the color different from (1,1,1)
			UMaterialExpressionDivide* Roughness =
			    NewMaterialExpressionDivide(Function, NewMaterialExpressionAdd(Function, RoughnessU, RoughnessV), 2.0f);

			UMaterialExpressionFresnel* Fresnel = NewMaterialExpressionFresnel(Function, 5.0f, {}, {});
			UMaterialExpressionIf*      Opacity = NewMaterialExpressionSwitch(Function, Mode, {1.0f, Fresnel, Fresnel});

			UMaterialExpressionMakeMaterialAttributes* BSDF =
			    NewMaterialExpressionMakeMaterialAttributes(Function, Tint, 1.0f, 0.0f, Roughness, {}, Opacity, {}, Normal);

			NewMaterialExpressionFunctionOutput(Function, TEXT("bsdf"), BSDF);
		}

		void DFSpecularBSDF(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* StateNormal = LoadFunction(TEXT("mdl_state_normal"));

			Function->Description = TEXT("Specular reflections and transmissions.");

			UMaterialExpressionFunctionInput* Tint =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tint"), EFunctionInputType::FunctionInput_Vector3, {1.0f, 1.0f, 1.0f});
			UMaterialExpressionFunctionInput* Mode =
			    NewMaterialExpressionFunctionInput(Function, TEXT("mode"), EFunctionInputType::FunctionInput_Scalar, (int)EScatterMode::Reflect);
			UMaterialExpressionFunctionInput* Normal = NewMaterialExpressionFunctionInput(
			    Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3, NewMaterialExpressionFunctionCall(Function, StateNormal, {}));

			// This, using Material->IOR, instead of IOR would be right for BaseReflectionFraction in Fresnel !
			// UMaterialExpressionMultiply* BaseReflectFraction =
			//  NewMaterialExpressionSquare(Function,
			//      NewMaterialExpressionDivide(Function, NewMaterialExpressionOneMinus(Function, IOR), NewMaterialExpressionAdd(Function, 1.0f,
			//      IOR)));

			UMaterialExpressionFresnel* Fresnel = NewMaterialExpressionFresnel(Function, 5.0f, {}, {});
			UMaterialExpressionIf*      Opacity = NewMaterialExpressionSwitch(Function, Mode, {1.0f, Fresnel, Fresnel});

			UMaterialExpressionMakeMaterialAttributes* BSDF =
			    NewMaterialExpressionMakeMaterialAttributes(Function, Tint, 1.0f, 0.0f, 0.0f, {}, Opacity, {}, Normal);

			NewMaterialExpressionFunctionOutput(Function, TEXT("bsdf"), BSDF);
		}

		/*static*/ void DFSpotEDF(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("Exponentiated cosine weighting for spotlight. The spot light is oriented along the positive z-axis.");

			// dummy implementation!
			// we just return a uniform light emission in all directions
			UMaterialExpressionFunctionInput* Exponent =
			    NewMaterialExpressionFunctionInput(Function, TEXT("exponent"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* Spread =
			    NewMaterialExpressionFunctionInput(Function, TEXT("spread"), EFunctionInputType::FunctionInput_Scalar, PI);
			UMaterialExpressionFunctionInput* GlobalDistribution =
			    NewMaterialExpressionFunctionInput(Function, TEXT("global_distribution"), EFunctionInputType::FunctionInput_StaticBool, true);
			UMaterialExpressionFunctionInput* GlobalFrame_0 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("global_frame_0"), EFunctionInputType::FunctionInput_Vector3, {1.0f, 0.0f, 0.0f});
			UMaterialExpressionFunctionInput* GlobalFrame_1 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("global_frame_1"), EFunctionInputType::FunctionInput_Vector3, {0.0f, 1.0f, 0.0f});
			UMaterialExpressionFunctionInput* GlobalFrame_2 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("global_frame_2"), EFunctionInputType::FunctionInput_Vector3, {0.0f, 0.0f, 1.0f});

			UMaterialExpressionMakeMaterialAttributes* EDF =
			    NewMaterialExpressionMakeMaterialAttributes(Function, {}, {}, 0.0f, {}, {1.0f, 1.0f, 1.0f});

			NewMaterialExpressionFunctionOutput(Function, TEXT("edf"), EDF);
		}

		void DFThinFilm(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);

			UMaterialFunction* MakeFloat3 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat3.MakeFloat3"));
			UMaterialFunction* MathSine3  = LoadFunction(TEXT("mdl_math_sin_float3"));
			UMaterialFunction* Refract    = LoadFunction(TEXT("mdlimporter_refract"));
			UMaterialFunction* StateNormal = LoadFunction(TEXT("mdl_state_normal"));

			UMaterialExpressionFunctionInput* Thickness =
			    NewMaterialExpressionFunctionInput(Function, TEXT("thickness"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* IOR =
			    NewMaterialExpressionFunctionInput(Function, TEXT("ior"), EFunctionInputType::FunctionInput_Vector3);
			UMaterialExpressionFunctionInput* Base =
			    NewMaterialExpressionFunctionInput(Function, TEXT("base"), EFunctionInputType::FunctionInput_MaterialAttributes);
			UMaterialExpressionFunctionInput* Normal = NewMaterialExpressionFunctionInput(
			    Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3, NewMaterialExpressionFunctionCall(Function, StateNormal, {}));

			// get the refraction vector T, and check for total internal reflection
			UMaterialExpressionCameraVectorWS*       ViewDir = NewMaterialExpression<UMaterialExpressionCameraVectorWS>(Function);
			UMaterialExpressionMaterialFunctionCall* T0 =
			    NewMaterialExpressionFunctionCall(Function, Refract, {ViewDir, Normal, NewMaterialExpressionComponentMask(Function, IOR, 1)});
			UMaterialExpressionMaterialFunctionCall* T1 =
			    NewMaterialExpressionFunctionCall(Function, Refract, {ViewDir, Normal, NewMaterialExpressionComponentMask(Function, IOR, 2)});
			UMaterialExpressionMaterialFunctionCall* T2 =
			    NewMaterialExpressionFunctionCall(Function, Refract, {ViewDir, Normal, NewMaterialExpressionComponentMask(Function, IOR, 4)});

			// angle between negative stateNormal and refraction vector T
			UMaterialExpressionSubtract*             NegativeNormal = NewMaterialExpressionNegate(Function, Normal);
			UMaterialExpressionMaterialFunctionCall* CosTheta       = NewMaterialExpressionFunctionCall(
                Function, MakeFloat3,
                {NewMaterialExpressionDotProduct(Function, NegativeNormal, T0), NewMaterialExpressionDotProduct(Function, NegativeNormal, T1),
                 NewMaterialExpressionDotProduct(Function, NegativeNormal, T2)});

			// wavelength of rgb found at http://en.wikipedia.org/wiki/Visible_spectrum
			UMaterialExpressionConstant3Vector* OneOverLambda = NewMaterialExpressionConstant(Function, 1.0f / 685.0f, 1.0f / 533.0f, 1.0f / 473.0f);

			// optical path difference, as found at https://en.wikipedia.org/wiki/Thin-film_interference in multiples of wavelengths
			UMaterialExpressionMultiply* OPD = NewMaterialExpressionMultiply(Function, {2.0f, IOR, Thickness, CosTheta, OneOverLambda});

			// adjust for total reflection
			UMaterialExpressionComponentMask*        OPDx = NewMaterialExpressionComponentMask(Function, OPD, 1);
			UMaterialExpressionComponentMask*        OPDy = NewMaterialExpressionComponentMask(Function, OPD, 2);
			UMaterialExpressionComponentMask*        OPDz = NewMaterialExpressionComponentMask(Function, OPD, 4);
			UMaterialExpressionMaterialFunctionCall* M    = NewMaterialExpressionFunctionCall(
                Function, MakeFloat3,
                {NewMaterialExpressionIfEqual(Function, OPDx, 0.0f, 0.5f, OPDx), NewMaterialExpressionIfEqual(Function, OPDy, 0.0f, 0.5f, OPDy),
                 NewMaterialExpressionIfEqual(Function, OPDz, 0.0f, 0.5f, OPDz)});

			// with 1 < ior, we have a face shift of 180 degree at the upper boundary of the film, then,
			// with fract(m) == 0.0, we have destructive interference
			// with fract(m) == 0.5, we have constructive interference
			UMaterialExpressionFrac*                 PD       = NewMaterialExpressionFrac(Function, M);  // range [0.0, 1.0)
			UMaterialExpressionMaterialFunctionCall* Modulate = NewMaterialExpressionFunctionCall(
			    Function, MathSine3, {NewMaterialExpressionMultiply(Function, PI, PD)});  // range [0.0, 1.0), with maximum and PD == 0.5

			UMaterialExpressionMaterialFunctionCall* BSDF = NewMaterialExpressionFunctionCall(
			    Function, LoadFunction(TEXT("/Engine/Functions/MaterialLayerFunctions"), TEXT("MatLayerBlend_Tint")), {Base, Modulate, {}});

			NewMaterialExpressionFunctionOutput(Function, TEXT("bsdf"), BSDF);
		}

		void DFTint(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* StateNormal = LoadFunction(TEXT("mdl_state_normal"));

			Function->Description = TEXT("Tint the result of an elemental or compound distribution function with an additional color.");

			UMaterialExpressionFunctionInput* Color =
			    NewMaterialExpressionFunctionInput(Function, TEXT("color"), EFunctionInputType::FunctionInput_Vector3);
			UMaterialExpressionFunctionInput* Base =
			    NewMaterialExpressionFunctionInput(Function, TEXT("base"), EFunctionInputType::FunctionInput_MaterialAttributes);
			UMaterialExpressionFunctionInput* Normal = NewMaterialExpressionFunctionInput(
			    Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3, NewMaterialExpressionFunctionCall(Function, StateNormal, {}));

			UMaterialExpressionMaterialFunctionCall* BSDF = NewMaterialExpressionFunctionCall(
			    Function, LoadFunction(TEXT("/Engine/Functions/MaterialLayerFunctions"), TEXT("MatLayerBlend_Tint")), {Base, Color, {}});

			NewMaterialExpressionFunctionOutput(Function, TEXT("bsdf"), BSDF);
		}

		void DFWardGeislerMoroderBSDF(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* DFSimpleGlossyBSDF   = LoadFunction(TEXT("mdl_df_simple_glossy_bsdf"));
			UMaterialFunction* StateNormal          = LoadFunction(TEXT("mdl_state_normal"));
			UMaterialFunction* StateTextureTangentU = LoadFunction(TEXT("mdl_state_texture_tangent_u"));

			Function->Description = TEXT("Glossy reflection and transmission.");

			UMaterialExpressionFunctionInput* RoughnessU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("roughness_u"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* RoughnessV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("roughness_v"), EFunctionInputType::FunctionInput_Scalar, RoughnessU);
			UMaterialExpressionFunctionInput* Tint =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tint"), EFunctionInputType::FunctionInput_Vector3, {1.0f, 1.0f, 1.0f});
			UMaterialExpressionFunctionInput* TangentU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tangent_u"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpressionFunctionCall(Function, StateTextureTangentU, { 0 }));
			UMaterialExpressionFunctionInput* Normal = NewMaterialExpressionFunctionInput(
			    Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3, NewMaterialExpressionFunctionCall(Function, StateNormal, {}));

			UMaterialExpressionMaterialFunctionCall* BSDF = NewMaterialExpressionFunctionCall(
			    Function, DFSimpleGlossyBSDF, {RoughnessU, RoughnessV, Tint, TangentU, (int)EScatterMode::Reflect, Normal});

			NewMaterialExpressionFunctionOutput(Function, TEXT("bsdf"), BSDF);
		}

		void DFWeightedLayer(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* StateNormal = LoadFunction(TEXT("mdl_state_normal"));

			Function->Description =
			    TEXT("Add an elemental or compound BSDF as a layer on top of another elemental or compound BSDF according to weight.");

			UMaterialExpression* DefaultMaterialAttributes = NewMaterialExpressionMakeMaterialAttributes(Function);

			UMaterialExpressionFunctionInput* Weight =
			    NewMaterialExpressionFunctionInput(Function, TEXT("weight"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* Layer = NewMaterialExpressionFunctionInput(
			    Function, TEXT("layer"), EFunctionInputType::FunctionInput_MaterialAttributes, DefaultMaterialAttributes);
			UMaterialExpressionFunctionInput* Base = NewMaterialExpressionFunctionInput(
			    Function, TEXT("base"), EFunctionInputType::FunctionInput_MaterialAttributes, DefaultMaterialAttributes);
			UMaterialExpressionFunctionInput* Normal = NewMaterialExpressionFunctionInput(
			    Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3, NewMaterialExpressionFunctionCall(Function, StateNormal, {}));

			UMaterialExpressionClamp* ClampedWeight = NewMaterialExpressionClamp(Function, Weight, 0.0f, 1.0f);

			UMaterialExpressionMaterialFunctionCall* AdjustedLayer = NewMaterialExpressionFunctionCall(
			    Function, LoadFunction(TEXT("/Engine/Functions/MaterialLayerFunctions"), TEXT("MatLayerBlend_ReplaceNormals")), {Layer, Normal});
			UMaterialExpressionMaterialFunctionCall* BSDF = NewMaterialExpressionFunctionCall(
			    Function, LoadFunction(TEXT("/Engine/Functions/MaterialLayerFunctions"), TEXT("MatLayerBlend_Standard")),
			    {Base, AdjustedLayer, ClampedWeight});

			NewMaterialExpressionFunctionOutput(Function, TEXT("bsdf"), BSDF);
		}

		/*static*/ void MathCosFloat(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("Computes the cosine of an angle in radian.");

			UMaterialExpressionFunctionInput* A = NewMaterialExpressionFunctionInput(Function, TEXT("a"), EFunctionInputType::FunctionInput_Scalar);

			UMaterialExpressionCosine* Cosine = NewMaterialExpressionCosine(Function, A);

			NewMaterialExpressionFunctionOutput(Function, TEXT("cos"), Cosine);
		}

		void MathCosFloat3(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* MakeFloat3 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat3.MakeFloat3"));

			Function->Description = TEXT("Computes the cosine of three angles in radian.");

			UMaterialExpressionFunctionInput* A = NewMaterialExpressionFunctionInput(Function, TEXT("a"), EFunctionInputType::FunctionInput_Vector3);

			UMaterialExpressionMaterialFunctionCall* Cos =
			    NewMaterialExpressionFunctionCall(Function, MakeFloat3,
			                                      {NewMaterialExpressionCosine(Function, NewMaterialExpressionComponentMask(Function, A, 1)),
			                                       NewMaterialExpressionCosine(Function, NewMaterialExpressionComponentMask(Function, A, 2)),
			                                       NewMaterialExpressionCosine(Function, NewMaterialExpressionComponentMask(Function, A, 4))});

			NewMaterialExpressionFunctionOutput(Function, TEXT("cos"), Cos);
		}

		/*static*/ void MathLogFloat(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("Computes the natural logarithm.");

			UMaterialExpressionFunctionInput* A = NewMaterialExpressionFunctionInput(Function, TEXT("a"), EFunctionInputType::FunctionInput_Scalar);

			UMaterialExpressionMultiply* Ln = NewMaterialExpressionMultiply(Function, NewMaterialExpressionLogarithm2(Function, A), log(2.0f));

			NewMaterialExpressionFunctionOutput(Function, TEXT("ln"), Ln);
		}

		void MathLogFloat3(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* MakeFloat3 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat3.MakeFloat3"));

			Function->Description = TEXT("Computes the natural logarithm.");

			UMaterialExpressionFunctionInput* A = NewMaterialExpressionFunctionInput(Function, TEXT("a"), EFunctionInputType::FunctionInput_Vector3);

			UMaterialExpressionMultiply* Ln = NewMaterialExpressionMultiply(
			    Function,
			    NewMaterialExpressionFunctionCall(Function, MakeFloat3,
			                                      {NewMaterialExpressionLogarithm2(Function, NewMaterialExpressionComponentMask(Function, A, 1)),
			                                       NewMaterialExpressionLogarithm2(Function, NewMaterialExpressionComponentMask(Function, A, 2)),
			                                       NewMaterialExpressionLogarithm2(Function, NewMaterialExpressionComponentMask(Function, A, 4))}),
			    log(2.0f));

			NewMaterialExpressionFunctionOutput(Function, TEXT("ln"), Ln);
		}

		/*static*/ void MathLog10Float(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("Computes the logarithm to the base 10.");

			UMaterialExpressionFunctionInput* A = NewMaterialExpressionFunctionInput(Function, TEXT("a"), EFunctionInputType::FunctionInput_Scalar);

			UMaterialExpressionMultiply* Log10 = NewMaterialExpressionMultiply(Function, NewMaterialExpressionLogarithm2(Function, A), log(10.0f));

			NewMaterialExpressionFunctionOutput(Function, TEXT("log10"), Log10);
		}

		void MathLog10Float3(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* MakeFloat3 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat3.MakeFloat3"));

			Function->Description = TEXT("Computes the logarithm to the base 10.");

			UMaterialExpressionFunctionInput* A = NewMaterialExpressionFunctionInput(Function, TEXT("a"), EFunctionInputType::FunctionInput_Vector3);

			UMaterialExpressionMultiply* Log10 = NewMaterialExpressionMultiply(
			    Function,
			    NewMaterialExpressionFunctionCall(Function, MakeFloat3,
			                                      {NewMaterialExpressionLogarithm2(Function, NewMaterialExpressionComponentMask(Function, A, 1)),
			                                       NewMaterialExpressionLogarithm2(Function, NewMaterialExpressionComponentMask(Function, A, 2)),
			                                       NewMaterialExpressionLogarithm2(Function, NewMaterialExpressionComponentMask(Function, A, 4))}),
			    log(10.0f));

			NewMaterialExpressionFunctionOutput(Function, TEXT("log10"), Log10);
		}

		/*static*/ void MathLog2Float(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("Computes the logarithm to the base 2.");

			UMaterialExpressionFunctionInput* A = NewMaterialExpressionFunctionInput(Function, TEXT("a"), EFunctionInputType::FunctionInput_Scalar);

			UMaterialExpressionLogarithm2* Log2 = NewMaterialExpressionLogarithm2(Function, A);

			NewMaterialExpressionFunctionOutput(Function, TEXT("log2"), Log2);
		}

		void MathLog2Float3(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* MakeFloat3 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat3.MakeFloat3"));

			Function->Description = TEXT("Computes the logarithm to the base 2.");

			UMaterialExpressionFunctionInput* A = NewMaterialExpressionFunctionInput(Function, TEXT("a"), EFunctionInputType::FunctionInput_Vector3);

			UMaterialExpressionMaterialFunctionCall* Log2 =
			    NewMaterialExpressionFunctionCall(Function, MakeFloat3,
			                                      {NewMaterialExpressionLogarithm2(Function, NewMaterialExpressionComponentMask(Function, A, 1)),
			                                       NewMaterialExpressionLogarithm2(Function, NewMaterialExpressionComponentMask(Function, A, 2)),
			                                       NewMaterialExpressionLogarithm2(Function, NewMaterialExpressionComponentMask(Function, A, 4))});

			NewMaterialExpressionFunctionOutput(Function, TEXT("log2"), Log2);
		}

		/*static*/ void MathLuminance(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("the luminance is equal to 0.212671 * a.x + 0.715160 * a.y + 0.072169 * a.z.");

			UMaterialExpressionFunctionInput* A = NewMaterialExpressionFunctionInput(Function, TEXT("a"), EFunctionInputType::FunctionInput_Vector3);

			UMaterialExpressionDotProduct* Luminance = NewMaterialExpressionDotProduct(Function, A, {0.212677f, 0.715160f, 0.072129f});

			NewMaterialExpressionFunctionOutput(Function, TEXT("luminance"), Luminance);
		}

		/*static*/ void MathMaxValue(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("Returns the largest value of a.");

			UMaterialExpressionFunctionInput* A = NewMaterialExpressionFunctionInput(Function, TEXT("a"), EFunctionInputType::FunctionInput_Vector3);

			UMaterialExpressionMax* MaxValue = NewMaterialExpressionMax(Function,
			                                                            NewMaterialExpressionComponentMask(Function, A, 1),
			                                                            NewMaterialExpressionComponentMask(Function, A, 2),
			                                                            NewMaterialExpressionComponentMask(Function, A, 4));

			NewMaterialExpressionFunctionOutput(Function, TEXT("max_value"), MaxValue);
		}

		/*static*/ void MathMinValue(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("Returns the smallest value of a.");

			UMaterialExpressionFunctionInput* A = NewMaterialExpressionFunctionInput(Function, TEXT("a"), EFunctionInputType::FunctionInput_Vector3);

			UMaterialExpressionMin* MinValue = NewMaterialExpressionMin(Function,
			                                                            NewMaterialExpressionComponentMask(Function, A, 1),
			                                                            NewMaterialExpressionComponentMask(Function, A, 2),
			                                                            NewMaterialExpressionComponentMask(Function, A, 4));

			NewMaterialExpressionFunctionOutput(Function, TEXT("min_value"), MinValue);
		}

		void MathMultiplyFloat4x4Float4(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* MakeFloat4 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat4.MakeFloat4"));

			Function->Description = TEXT("Multiplies 4x4-matrix with 4-vector.");

			UMaterialExpressionFunctionInput* Column0 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("column0"), EFunctionInputType::FunctionInput_Vector4);
			UMaterialExpressionFunctionInput* Column1 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("column1"), EFunctionInputType::FunctionInput_Vector4);
			UMaterialExpressionFunctionInput* Column2 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("column2"), EFunctionInputType::FunctionInput_Vector4);
			UMaterialExpressionFunctionInput* Column3 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("column3"), EFunctionInputType::FunctionInput_Vector4);
			UMaterialExpressionFunctionInput* Vector =
			    NewMaterialExpressionFunctionInput(Function, TEXT("vector"), EFunctionInputType::FunctionInput_Vector4);

			UMaterialExpressionMaterialFunctionCall* Row0 = NewMaterialExpressionFunctionCall(
			    Function, MakeFloat4,
			    {NewMaterialExpressionComponentMask(Function, Column0, 1), NewMaterialExpressionComponentMask(Function, Column1, 1),
			     NewMaterialExpressionComponentMask(Function, Column2, 1), NewMaterialExpressionComponentMask(Function, Column3, 1)});
			UMaterialExpressionMaterialFunctionCall* Row1 = NewMaterialExpressionFunctionCall(
			    Function, MakeFloat4,
			    {NewMaterialExpressionComponentMask(Function, Column0, 2), NewMaterialExpressionComponentMask(Function, Column1, 2),
			     NewMaterialExpressionComponentMask(Function, Column2, 2), NewMaterialExpressionComponentMask(Function, Column3, 2)});
			UMaterialExpressionMaterialFunctionCall* Row2 = NewMaterialExpressionFunctionCall(
			    Function, MakeFloat4,
			    {NewMaterialExpressionComponentMask(Function, Column0, 4), NewMaterialExpressionComponentMask(Function, Column1, 4),
			     NewMaterialExpressionComponentMask(Function, Column2, 4), NewMaterialExpressionComponentMask(Function, Column3, 4)});
			UMaterialExpressionMaterialFunctionCall* Row3 = NewMaterialExpressionFunctionCall(
			    Function, MakeFloat4,
			    {NewMaterialExpressionComponentMask(Function, Column0, 8), NewMaterialExpressionComponentMask(Function, Column1, 8),
			     NewMaterialExpressionComponentMask(Function, Column2, 8), NewMaterialExpressionComponentMask(Function, Column3, 8)});
			UMaterialExpressionMaterialFunctionCall* Result = NewMaterialExpressionFunctionCall(
			    Function, MakeFloat4,
			    {NewMaterialExpressionDotProduct(Function, Row0, Vector), NewMaterialExpressionDotProduct(Function, Row1, Vector),
			     NewMaterialExpressionDotProduct(Function, Row2, Vector), NewMaterialExpressionDotProduct(Function, Row3, Vector)});

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Result);
		}

		void MathMultiplyFloat4x4Float4x4(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* MakeFloat4 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat4.MakeFloat4"));

			Function->Description = TEXT("Multiplies two 4x4 matrices.");

			UMaterialExpressionFunctionInput* Matrix0Col0 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("matrix0_col0"), EFunctionInputType::FunctionInput_Vector4);
			UMaterialExpressionFunctionInput* Matrix0Col1 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("matrix0_col1"), EFunctionInputType::FunctionInput_Vector4);
			UMaterialExpressionFunctionInput* Matrix0Col2 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("matrix0_col2"), EFunctionInputType::FunctionInput_Vector4);
			UMaterialExpressionFunctionInput* Matrix0Col3 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("matrix0_col3"), EFunctionInputType::FunctionInput_Vector4);
			UMaterialExpressionFunctionInput* Matrix1Col0 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("matrix1_col0"), EFunctionInputType::FunctionInput_Vector4);
			UMaterialExpressionFunctionInput* Matrix1Col1 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("matrix1_col1"), EFunctionInputType::FunctionInput_Vector4);
			UMaterialExpressionFunctionInput* Matrix1Col2 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("matrix1_col2"), EFunctionInputType::FunctionInput_Vector4);
			UMaterialExpressionFunctionInput* Matrix1Col3 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("matrix1_col3"), EFunctionInputType::FunctionInput_Vector4);

			UMaterialExpression* Row0 = NewMaterialExpressionFunctionCall(
			    Function, MakeFloat4,
			    {NewMaterialExpressionComponentMask(Function, Matrix0Col0, 1), NewMaterialExpressionComponentMask(Function, Matrix0Col1, 1),
			     NewMaterialExpressionComponentMask(Function, Matrix0Col2, 1), NewMaterialExpressionComponentMask(Function, Matrix0Col3, 1)});
			UMaterialExpression* Row1 = NewMaterialExpressionFunctionCall(
			    Function, MakeFloat4,
			    {NewMaterialExpressionComponentMask(Function, Matrix0Col0, 2), NewMaterialExpressionComponentMask(Function, Matrix0Col1, 2),
			     NewMaterialExpressionComponentMask(Function, Matrix0Col2, 2), NewMaterialExpressionComponentMask(Function, Matrix0Col3, 2)});
			UMaterialExpression* Row2 = NewMaterialExpressionFunctionCall(
			    Function, MakeFloat4,
			    {NewMaterialExpressionComponentMask(Function, Matrix0Col0, 4), NewMaterialExpressionComponentMask(Function, Matrix0Col1, 4),
			     NewMaterialExpressionComponentMask(Function, Matrix0Col2, 4), NewMaterialExpressionComponentMask(Function, Matrix0Col3, 4)});
			UMaterialExpression* Row3 = NewMaterialExpressionFunctionCall(
			    Function, MakeFloat4,
			    {NewMaterialExpressionComponentMask(Function, Matrix0Col0, 8), NewMaterialExpressionComponentMask(Function, Matrix0Col1, 8),
			     NewMaterialExpressionComponentMask(Function, Matrix0Col2, 8), NewMaterialExpressionComponentMask(Function, Matrix0Col3, 8)});

			UMaterialExpressionMaterialFunctionCall* Result0 = NewMaterialExpressionFunctionCall(
			    Function, MakeFloat4,
			    {NewMaterialExpressionDotProduct(Function, Row0, Matrix1Col0), NewMaterialExpressionDotProduct(Function, Row1, Matrix1Col0),
			     NewMaterialExpressionDotProduct(Function, Row2, Matrix1Col0), NewMaterialExpressionDotProduct(Function, Row3, Matrix1Col0)});
			UMaterialExpressionMaterialFunctionCall* Result1 = NewMaterialExpressionFunctionCall(
			    Function, MakeFloat4,
			    {NewMaterialExpressionDotProduct(Function, Row0, Matrix1Col1), NewMaterialExpressionDotProduct(Function, Row1, Matrix1Col1),
			     NewMaterialExpressionDotProduct(Function, Row2, Matrix1Col1), NewMaterialExpressionDotProduct(Function, Row3, Matrix1Col1)});
			UMaterialExpressionMaterialFunctionCall* Result2 = NewMaterialExpressionFunctionCall(
			    Function, MakeFloat4,
			    {NewMaterialExpressionDotProduct(Function, Row0, Matrix1Col2), NewMaterialExpressionDotProduct(Function, Row1, Matrix1Col2),
			     NewMaterialExpressionDotProduct(Function, Row2, Matrix1Col2), NewMaterialExpressionDotProduct(Function, Row3, Matrix1Col2)});
			UMaterialExpressionMaterialFunctionCall* Result3 = NewMaterialExpressionFunctionCall(
			    Function, MakeFloat4,
			    {NewMaterialExpressionDotProduct(Function, Row0, Matrix1Col3), NewMaterialExpressionDotProduct(Function, Row1, Matrix1Col3),
			     NewMaterialExpressionDotProduct(Function, Row2, Matrix1Col3), NewMaterialExpressionDotProduct(Function, Row3, Matrix1Col3)});

			NewMaterialExpressionFunctionOutput(Function, TEXT("result_0"), Result0);
			NewMaterialExpressionFunctionOutput(Function, TEXT("result_1"), Result1);
			NewMaterialExpressionFunctionOutput(Function, TEXT("result_2"), Result2);
			NewMaterialExpressionFunctionOutput(Function, TEXT("result_3"), Result3);
		}

		/*static*/ void MathSinFloat(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("Computes the sine of an angle in radian.");

			UMaterialExpressionFunctionInput* A = NewMaterialExpressionFunctionInput(Function, TEXT("a"), EFunctionInputType::FunctionInput_Scalar);

			UMaterialExpressionSine* Sine = NewMaterialExpressionSine(Function, A);

			NewMaterialExpressionFunctionOutput(Function, TEXT("sin"), Sine);
		}

		void MathSinFloat3(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* MakeFloat3 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat3.MakeFloat3"));

			Function->Description = TEXT("Computes the sine of three angles in radian.");

			UMaterialExpressionFunctionInput* A = NewMaterialExpressionFunctionInput(Function, TEXT("a"), EFunctionInputType::FunctionInput_Vector3);

			UMaterialExpressionMaterialFunctionCall* Sin =
			    NewMaterialExpressionFunctionCall(Function, MakeFloat3,
			                                      {NewMaterialExpressionSine(Function, NewMaterialExpressionComponentMask(Function, A, 1)),
			                                       NewMaterialExpressionSine(Function, NewMaterialExpressionComponentMask(Function, A, 2)),
			                                       NewMaterialExpressionSine(Function, NewMaterialExpressionComponentMask(Function, A, 4))});

			NewMaterialExpressionFunctionOutput(Function, TEXT("sin"), Sin);
		}

		/*static*/ void MathSum(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("Returns the sum of the elements of a.");

			UMaterialExpressionFunctionInput* A = NewMaterialExpressionFunctionInput(Function, TEXT("a"), EFunctionInputType::FunctionInput_Vector3);

			UMaterialExpressionAdd* Sum =
			    NewMaterialExpressionAdd(Function,
			                             {NewMaterialExpressionComponentMask(Function, A, 1), NewMaterialExpressionComponentMask(Function, A, 2),
			                              NewMaterialExpressionComponentMask(Function, A, 4)});

			NewMaterialExpressionFunctionOutput(Function, TEXT("sum"), Sum);
		}

		/*static*/ void StateAnimationTime(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("The time of the current sample in seconds, including the time within a shutter interval.");

			UMaterialExpression* Time = NewMaterialExpression<UMaterialExpressionTime>(Function);

			NewMaterialExpressionFunctionOutput(Function, TEXT("animation_time"), Time);
		}

		/*static*/ void StateDirection(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("Lookup direction in the context of an environment lookup and float3(0.0) in all other contexts.");

			UMaterialExpressionFunctionInput* CustomWorldNormal =
			    NewMaterialExpressionFunctionInput(Function, TEXT("CustomWorldNormal"), EFunctionInputType::FunctionInput_Vector3,
			                                       NewMaterialExpression<UMaterialExpressionPixelNormalWS>(Function));

			UMaterialExpressionReflectionVectorWS* Direction = NewMaterialExpressionReflectionVectorWS(Function, CustomWorldNormal);

			NewMaterialExpressionFunctionOutput(Function, TEXT("direction"), Direction);
		}

		/*static*/ void StateGeometryNormal(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("The true geometric surface normal for the current geometry as a unit - length vector.");

			NewMaterialExpressionFunctionOutput(Function, TEXT("geometry_normal"), {0.0f, 0.0f, 1.0f});
		}

		/*static*/ void StateGeometryTangentU(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("Array of geometry tangents.");

			NewMaterialExpressionFunctionOutput(Function, TEXT("geometry_tangent_u"), {1.0f, 0.0f, 0.0f});
		}

		/*static*/ void StateGeometryTangentV(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("Array of geometry bitangents.");

			// As Unreal is left handed, the v-tangent is (0, -1, 0) !
			NewMaterialExpressionFunctionOutput(Function, TEXT("geometry_tangent_v"), {0.0f, -1.0f, 0.0f});
		}

		/*static*/ void StateMetersPerSceneUnit(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("Returns the distance of one scene unit in meters.");

			NewMaterialExpressionFunctionOutput(Function, TEXT("meters_per_scene_unit"), 0.01f);
		}

		/*static*/ void StateNormal(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("The shading surface normal as a unit-length vector.");

			// DAR HACK Need to evaluate the state normal in fragment space to get the clear coat bottom normal working.
			UTexture2D* Texture = LoadObject<UTexture2D>(nullptr, TEXT("/Engine/EngineMaterials/FlatNormal.FlatNormal"));
			check(Texture);
			UMaterialExpressionTextureSample* NormalSampler = NewMaterialExpressionTextureSample(Function, Texture, {});
			NormalSampler->SamplerType                      = SAMPLERTYPE_Normal;

			UMaterialExpressionMultiply* Normal = NewMaterialExpressionMultiply(Function, NormalSampler, NewMaterialExpressionTwoSidedSign(Function));

			NewMaterialExpressionFunctionOutput(Function, TEXT("normal"), {Normal});
		}

		/*static*/ void StateObjectId(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("Returns the object ID provided in a scene, and zero if none was given or for the environment.");

			NewMaterialExpressionFunctionOutput(Function, TEXT("object_id"), 0.0f);
		}

		/*static*/ void StatePosition(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("The intersection point on the surface or the sample point in the volume.");

			UMaterialExpressionWorldPosition* Position = NewMaterialExpression<UMaterialExpressionWorldPosition>(Function);

			NewMaterialExpressionFunctionOutput(Function, TEXT("position"), Position);
		}

		/*static*/ void StateSceneUnitsPerMeter(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("Returns the distance of one meter in scene units.");

			NewMaterialExpressionFunctionOutput(Function, TEXT("scene_units_per_meter"), 100.0f);
		}

		void StateTangentSpace(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* StateNormal          = LoadFunction(TEXT("mdl_state_normal"));
			UMaterialFunction* StateTextureTangentU = LoadFunction(TEXT("mdl_state_texture_tangent_u"));
			UMaterialFunction* StateTextureTangentV = LoadFunction(TEXT("mdl_state_texture_tangent_v"));

			Function->Description = TEXT("The array of tangent space matrices for each texture space.");

			UMaterialExpressionFunctionInput* Index =
			    NewMaterialExpressionFunctionInput(Function, TEXT("index"), EFunctionInputType::FunctionInput_Scalar);

			UMaterialExpressionMaterialFunctionCall* X = NewMaterialExpressionFunctionCall(Function, StateTextureTangentU, { Index });
			UMaterialExpressionMaterialFunctionCall* Y = NewMaterialExpressionFunctionCall(Function, StateTextureTangentV, { Index });
			UMaterialExpressionMaterialFunctionCall* Z = NewMaterialExpressionFunctionCall(Function, StateNormal, {});

			NewMaterialExpressionFunctionOutput(Function, TEXT("texture_tangent_u"), X);
			NewMaterialExpressionFunctionOutput(Function, TEXT("texture_tangent_v"), Y);
			NewMaterialExpressionFunctionOutput(Function, TEXT("normal"), Z);
		}

		void StateTextureCoordinate(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("The texture space at the given index.");

			UMaterialFunction* BreakFloat2 =
			    LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("BreakFloat2Components.BreakFloat2Components"));

			UMaterialExpressionFunctionInput* TextureCoordinateInput =
			    NewMaterialExpressionFunctionInput(Function, TEXT("position_uv"), EFunctionInputType::FunctionInput_Vector2);
			TextureCoordinateInput->Preview.Expression        = NewMaterialExpressionTextureCoordinate(Function, 0);
			TextureCoordinateInput->bUsePreviewValueAsDefault = true;

			// we need to OneMinus the y-component of the texture coordinate to get into the same orientation as mdl expects it
			// -> later on, in the TexLookupN functions, the y-component is OneMinus'ed back again, as UE turns the textures upside down as well
			UMaterialExpressionMaterialFunctionCall* TextureCoordinatesBreak =
			    NewMaterialExpressionFunctionCall(Function, BreakFloat2, {TextureCoordinateInput});
			UMaterialExpressionAppendVector* TextureCoordinate = NewMaterialExpressionAppendVector(
			    Function, {TextureCoordinatesBreak, 0}, NewMaterialExpressionOneMinus(Function, {TextureCoordinatesBreak, 1}), 0.0f);

			NewMaterialExpressionFunctionOutput(Function, TEXT("texture_coordinate"), TextureCoordinate);
		}

		/*static*/ void StateTextureSpaceMax(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("The maximal number of texture spaces available.");

			NewMaterialExpressionFunctionOutput(Function, TEXT("texture_space_max"), 1.0f);
		}

		void StateTextureTangentU(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("The array of tangent vectors for each texture space.");

			UMaterialExpressionFunctionInput* Index = NewMaterialExpressionFunctionInput(Function, TEXT("index"), EFunctionInputType::FunctionInput_Scalar);

			NewMaterialExpressionFunctionOutput(Function, TEXT("texture_tangent_u"), { 1.0f, 0.0f, 0.0f });
		}

		void StateTextureTangentV(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("The array of bitangent vectors for each texture space.");

			UMaterialExpressionFunctionInput* Index = NewMaterialExpressionFunctionInput(Function, TEXT("index"), EFunctionInputType::FunctionInput_Scalar);

			NewMaterialExpressionFunctionOutput(Function, TEXT("texture_tangent_v"), { 0.0f, 1.0f, 0.0f });
		}

		/*static*/ void StateTransformPoint(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			// for points, internal space is world space !
			Function->Description = TEXT("Transforms a point from one coordinate system to another.");

			UMaterialExpressionFunctionInput* From =
			    NewMaterialExpressionFunctionInput(Function, TEXT("from"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* To = NewMaterialExpressionFunctionInput(Function, TEXT("to"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* Point =
			    NewMaterialExpressionFunctionInput(Function, TEXT("point"), EFunctionInputType::FunctionInput_Vector3);

			UMaterialExpressionTransformPosition* ObjectToWorld =
			    NewMaterialExpressionTransformPosition(Function, Point, TRANSFORMPOSSOURCE_Local, TRANSFORMPOSSOURCE_World);
			UMaterialExpressionTransformPosition* WorldToObject =
			    NewMaterialExpressionTransformPosition(Function, Point, TRANSFORMPOSSOURCE_World, TRANSFORMPOSSOURCE_Local);

			// for points, internal space equals world space
			UMaterialExpressionIf* ToInternal = NewMaterialExpressionIf(Function, From, (int)ECoordinateSpace::Object, Point, ObjectToWorld, Point);
			UMaterialExpressionIf* ToObject =
			    NewMaterialExpressionIf(Function, From, (int)ECoordinateSpace::Object, WorldToObject, Point, WorldToObject);
			UMaterialExpressionIf* ToWorld = NewMaterialExpressionIf(Function, From, (int)ECoordinateSpace::Object, Point, ObjectToWorld, Point);

			UMaterialExpressionIf* Transformed = NewMaterialExpressionIf(Function, To, (int)ECoordinateSpace::Object, ToInternal, ToObject, ToWorld);

			NewMaterialExpressionFunctionOutput(Function, TEXT("transformed"), Transformed);
		}

		/*static*/ void StateTransformVector(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("Transforms a vector from one coordinate system to another.");

			UMaterialExpressionFunctionInput* From =
			    NewMaterialExpressionFunctionInput(Function, TEXT("from"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* To = NewMaterialExpressionFunctionInput(Function, TEXT("to"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* Vector =
			    NewMaterialExpressionFunctionInput(Function, TEXT("vector"), EFunctionInputType::FunctionInput_Vector3);

			UMaterialExpressionTransform* ObjectToInternal =
			    NewMaterialExpressionTransform(Function, Vector, TRANSFORMSOURCE_Local, TRANSFORM_Tangent);
			UMaterialExpressionTransform* WorldToInternal =
			    NewMaterialExpressionTransform(Function, Vector, TRANSFORMSOURCE_World, TRANSFORM_Tangent);
			UMaterialExpressionTransform* InternalToObject =
			    NewMaterialExpressionTransform(Function, Vector, TRANSFORMSOURCE_Tangent, TRANSFORM_Local);
			UMaterialExpressionTransform* WorldToObject = NewMaterialExpressionTransform(Function, Vector, TRANSFORMSOURCE_World, TRANSFORM_Local);
			UMaterialExpressionTransform* InternalToWorld =
			    NewMaterialExpressionTransform(Function, Vector, TRANSFORMSOURCE_Tangent, TRANSFORM_World);
			UMaterialExpressionTransform* ObjectToWorld = NewMaterialExpressionTransform(Function, Vector, TRANSFORMSOURCE_Local, TRANSFORM_World);

			// for vectors, internal space equals tangent space
			UMaterialExpressionIf* ToInternal =
			    NewMaterialExpressionIf(Function, From, (int)ECoordinateSpace::Object, Vector, ObjectToInternal, WorldToInternal);
			UMaterialExpressionIf* ToObject =
			    NewMaterialExpressionIf(Function, From, (int)ECoordinateSpace::Object, InternalToObject, Vector, WorldToObject);
			UMaterialExpressionIf* ToWorld =
			    NewMaterialExpressionIf(Function, From, (int)ECoordinateSpace::Object, InternalToWorld, ObjectToWorld, Vector);

			UMaterialExpressionIf* Transformed = NewMaterialExpressionIf(Function, To, (int)ECoordinateSpace::Object, ToInternal, ToObject, ToWorld);

			NewMaterialExpressionFunctionOutput(Function, TEXT("transformed"), Transformed);
		}

		void TexLookupFloat(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* ImporterTextureSample = LoadFunction(TEXT("mdlimporter_texture_sample"));

			Function->Description = TEXT(
			    "Returns the sampled texture value for the twodimensional coordinates coord given in normalized texture space in the range [0, 1)², where the wrap"
			    "modes define the behavior for coordinates outside of that range. The crop parameters further define a sub - range on the texture that is actually"
			    "used and that defines the normalized texture space in the range [0, 1)². The crop parameter defaults float2(0.0, 1.0) corresponds to the whole texture"
			    "in the corresponding axis. A lookup on an invalid texture reference returns zero.");

			UMaterialExpressionFunctionInput* Tex = NewMaterialExpressionFunctionInput(
			    Function, TEXT("tex"), EFunctionInputType::FunctionInput_Texture2D, NewMaterialExpressionTextureObject(Function, nullptr));
			UMaterialExpressionFunctionInput* Coord = NewMaterialExpressionFunctionInput(
			    Function, TEXT("coord"), EFunctionInputType::FunctionInput_Vector2, NewMaterialExpressionTextureCoordinate(Function, 0));
			UMaterialExpressionFunctionInput* WrapU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("wrap_u"), EFunctionInputType::FunctionInput_Scalar, wrap_repeat);
			UMaterialExpressionFunctionInput* WrapV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("wrap_v"), EFunctionInputType::FunctionInput_Scalar, wrap_repeat);
			UMaterialExpressionFunctionInput* CropU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("crop_u"), EFunctionInputType::FunctionInput_Vector2, {0.0f, 1.0f});
			UMaterialExpressionFunctionInput* CropV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("crop_v"), EFunctionInputType::FunctionInput_Vector2, {0.0f, 1.0f});
			UMaterialExpressionFunctionInput* Frame =
			    NewMaterialExpressionFunctionInput(Function, TEXT("frame"), EFunctionInputType::FunctionInput_Scalar, 0);

			UMaterialExpressionMaterialFunctionCall* Sample =
			    NewMaterialExpressionFunctionCall(Function, ImporterTextureSample, {Tex, Coord, WrapU, WrapV, CropU, CropV});

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), {Sample, 1});// r in output index 1
		}

		void TexLookupFloat2(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* ImporterTextureSample = LoadFunction(TEXT("mdlimporter_texture_sample"));

			Function->Description = TEXT(
			    "Returns the sampled texture value for the twodimensional coordinates coord given in normalized texture space in the range [0, 1)², where the wrap"
			    "modes define the behavior for coordinates outside of that range. The crop parameters further define a sub - range on the texture that is actually"
			    "used and that defines the normalized texture space in the range [0, 1)². The crop parameter defaults float2(0.0, 1.0) corresponds to the whole texture"
			    "in the corresponding axis. A lookup on an invalid texture reference returns zero.");

			UMaterialExpressionFunctionInput* Tex = NewMaterialExpressionFunctionInput(
			    Function, TEXT("tex"), EFunctionInputType::FunctionInput_Texture2D, NewMaterialExpressionTextureObject(Function, nullptr));
			UMaterialExpressionFunctionInput* Coord = NewMaterialExpressionFunctionInput(
			    Function, TEXT("coord"), EFunctionInputType::FunctionInput_Vector2, NewMaterialExpressionTextureCoordinate(Function, 0));
			UMaterialExpressionFunctionInput* WrapU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("wrap_u"), EFunctionInputType::FunctionInput_Scalar, wrap_repeat);
			UMaterialExpressionFunctionInput* WrapV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("wrap_v"), EFunctionInputType::FunctionInput_Scalar, wrap_repeat);
			UMaterialExpressionFunctionInput* CropU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("crop_u"), EFunctionInputType::FunctionInput_Vector2, {0.0f, 1.0f});
			UMaterialExpressionFunctionInput* CropV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("crop_v"), EFunctionInputType::FunctionInput_Vector2, {0.0f, 1.0f});
			UMaterialExpressionFunctionInput* Frame =
			    NewMaterialExpressionFunctionInput(Function, TEXT("frame"), EFunctionInputType::FunctionInput_Scalar, 0);

			UMaterialExpressionMaterialFunctionCall* Sample =
			    NewMaterialExpressionFunctionCall(Function, ImporterTextureSample, {Tex, Coord, WrapU, WrapV, CropU, CropV});

			UMaterialFunction* MakeFloat2 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat2.MakeFloat2"));
			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), NewMaterialExpressionFunctionCall(Function, MakeFloat2, {{Sample, 1}, {Sample, 2}})); // (r, b)
		}

		void TexLookupFloat3(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* ImporterTextureSample = LoadFunction(TEXT("mdlimporter_texture_sample"));

			Function->Description = TEXT(
			    "Returns the sampled texture value for the twodimensional coordinates coord given in normalized texture space in the range [0, 1)², where the wrap"
			    "modes define the behavior for coordinates outside of that range. The crop parameters further define a sub - range on the texture that is actually"
			    "used and that defines the normalized texture space in the range [0, 1)². The crop parameter defaults float2(0.0, 1.0) corresponds to the whole texture"
			    "in the corresponding axis. A lookup on an invalid texture reference returns zero.");

			UMaterialExpressionFunctionInput* Tex = NewMaterialExpressionFunctionInput(
			    Function, TEXT("tex"), EFunctionInputType::FunctionInput_Texture2D, NewMaterialExpressionTextureObject(Function, nullptr));
			UMaterialExpressionFunctionInput* Coord = NewMaterialExpressionFunctionInput(
			    Function, TEXT("coord"), EFunctionInputType::FunctionInput_Vector2, NewMaterialExpressionTextureCoordinate(Function, 0));
			UMaterialExpressionFunctionInput* WrapU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("wrap_u"), EFunctionInputType::FunctionInput_Scalar, wrap_repeat);
			UMaterialExpressionFunctionInput* WrapV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("wrap_v"), EFunctionInputType::FunctionInput_Scalar, wrap_repeat);
			UMaterialExpressionFunctionInput* CropU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("crop_u"), EFunctionInputType::FunctionInput_Vector2, {0.0f, 1.0f});
			UMaterialExpressionFunctionInput* CropV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("crop_v"), EFunctionInputType::FunctionInput_Vector2, {0.0f, 1.0f});
			UMaterialExpressionFunctionInput* Frame =
			    NewMaterialExpressionFunctionInput(Function, TEXT("frame"), EFunctionInputType::FunctionInput_Scalar, 0);

			UMaterialExpressionMaterialFunctionCall* Sample =
			    NewMaterialExpressionFunctionCall(Function, ImporterTextureSample, {Tex, Coord, WrapU, WrapV, CropU, CropV});

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Sample); // rgb in output index 0
		}

		void TexLookupFloat4(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* ImporterTextureSample = LoadFunction(TEXT("mdlimporter_texture_sample"));

			Function->Description = TEXT(
			    "Returns the sampled texture value for the twodimensional coordinates coord given in normalized texture space in the range [0, 1)², where the wrap"
			    "modes define the behavior for coordinates outside of that range. The crop parameters further define a sub - range on the texture that is actually"
			    "used and that defines the normalized texture space in the range [0, 1)². The crop parameter defaults float2(0.0, 1.0) corresponds to the whole texture"
			    "in the corresponding axis. A lookup on an invalid texture reference returns zero.");

			UMaterialExpressionFunctionInput* Tex = NewMaterialExpressionFunctionInput(
			    Function, TEXT("tex"), EFunctionInputType::FunctionInput_Texture2D, NewMaterialExpressionTextureObject(Function, nullptr));
			UMaterialExpressionFunctionInput* Coord = NewMaterialExpressionFunctionInput(
			    Function, TEXT("coord"), EFunctionInputType::FunctionInput_Vector2, NewMaterialExpressionTextureCoordinate(Function, 0));
			UMaterialExpressionFunctionInput* WrapU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("wrap_u"), EFunctionInputType::FunctionInput_Scalar, wrap_repeat);
			UMaterialExpressionFunctionInput* WrapV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("wrap_v"), EFunctionInputType::FunctionInput_Scalar, wrap_repeat);
			UMaterialExpressionFunctionInput* CropU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("crop_u"), EFunctionInputType::FunctionInput_Vector2, {0.0f, 1.0f});
			UMaterialExpressionFunctionInput* CropV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("crop_v"), EFunctionInputType::FunctionInput_Vector2, {0.0f, 1.0f});
			UMaterialExpressionFunctionInput* Frame =
			    NewMaterialExpressionFunctionInput(Function, TEXT("frame"), EFunctionInputType::FunctionInput_Scalar, 0);

			UMaterialExpressionMaterialFunctionCall* Sample =
			    NewMaterialExpressionFunctionCall(Function, ImporterTextureSample, {Tex, Coord, WrapU, WrapV, CropU, CropV});

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), NewMaterialExpressionAppendVector(Function, {Sample, 0}, {Sample, 4}));// Append(rba, a)
		}

		/*static*/ void ImporterAddDetailNormal(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialExpressionFunctionInput* Normal =
			    NewMaterialExpressionFunctionInput(Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3);
			UMaterialExpressionFunctionInput* Detail =
			    NewMaterialExpressionFunctionInput(Function, TEXT("detail"), EFunctionInputType::FunctionInput_Vector3);

			UMaterialExpressionAdd* TangentSpaceNormal =
			    NewMaterialExpressionAdd(Function,
			                             NewMaterialExpressionTransform(Function, Normal, TRANSFORMSOURCE_Local, TRANSFORM_Tangent),
			                             NewMaterialExpressionConstant(Function, 0.0f, 0.0f, 1.0f));
			UMaterialExpressionMultiply* TangentSpaceDetail =
			    NewMaterialExpressionMultiply(Function,
			                                  NewMaterialExpressionTransform(Function, Detail, TRANSFORMSOURCE_Local, TRANSFORM_Tangent),
			                                  NewMaterialExpressionConstant(Function, -1.0f, -1.0f, 1.0f));
			UMaterialExpressionNormalize* DetailNormal = NewMaterialExpressionNormalize(
			    Function,
			    NewMaterialExpressionTransform(
			        Function,
			        NewMaterialExpressionSubtract(
			            Function,
			            NewMaterialExpressionMultiply(
			                Function,
			                TangentSpaceNormal,
			                NewMaterialExpressionDivide(Function,
			                                            NewMaterialExpressionDotProduct(Function, TangentSpaceNormal, TangentSpaceDetail),
			                                            NewMaterialExpressionComponentMask(Function, TangentSpaceNormal, 4))),
			            TangentSpaceDetail),
			        TRANSFORMSOURCE_Tangent, TRANSFORM_Local));

			NewMaterialExpressionFunctionOutput(Function, TEXT("detail_normal"), DetailNormal);
		}

		/*static*/ void ImporterApplyNoiseModifications(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialExpressionFunctionInput* Value =
			    NewMaterialExpressionFunctionInput(Function, TEXT("value"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* Position =
			    NewMaterialExpressionFunctionInput(Function, TEXT("position"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* ApplyMarble = NewMaterialExpressionFunctionInput(
			    Function, TEXT("apply_marble"), EFunctionInputType::FunctionInput_StaticBool, NewMaterialExpressionStaticBool(Function, false));
			UMaterialExpressionFunctionInput* ApplyDent = NewMaterialExpressionFunctionInput(
			    Function, TEXT("apply_dent"), EFunctionInputType::FunctionInput_StaticBool, NewMaterialExpressionStaticBool(Function, false));
			UMaterialExpressionFunctionInput* NoiseThresholdHigh =
			    NewMaterialExpressionFunctionInput(Function, TEXT("noise_threshold_high"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* NoiseThresholdLow =
			    NewMaterialExpressionFunctionInput(Function, TEXT("noise_threshold_low"), EFunctionInputType::FunctionInput_Scalar, 0.0f);
			UMaterialExpressionFunctionInput* NoiseBands =
			    NewMaterialExpressionFunctionInput(Function, TEXT("noise_bands"), EFunctionInputType::FunctionInput_Scalar, 1.0f);

			UMaterialExpressionStaticSwitch* ApplyMarbleCheck = NewMaterialExpressionStaticSwitch(
			    Function, ApplyMarble,
			    NewMaterialExpressionCosine(Function,
			                                NewMaterialExpressionAdd(Function, Position, NewMaterialExpressionMultiply(Function, Value, 5.0f))),
			    Value);

			UMaterialExpressionStaticSwitch* ApplyDentCheck = NewMaterialExpressionStaticSwitch(
			    Function, ApplyDent, NewMaterialExpressionMultiply(Function, {ApplyMarbleCheck, ApplyMarbleCheck, ApplyMarbleCheck}),
			    ApplyMarbleCheck);

			UMaterialExpressionPower* NoiseBandsResult = NewMaterialExpressionPower(
			    Function,
			    NewMaterialExpressionOneMinus(
			        Function, NewMaterialExpressionFrac(Function, NewMaterialExpressionMultiply(Function, ApplyDentCheck, NoiseBands))),
			    20.0f);
			UMaterialExpressionIf* NoiseBandsCheck = NewMaterialExpressionIfEqual(Function, NoiseBands, 1.0f, ApplyDentCheck, NoiseBandsResult);

			UMaterialExpressionIf* Result = NewMaterialExpressionIfGreater(
			    Function, NoiseThresholdHigh, NoiseThresholdLow,
			    NewMaterialExpressionClamp(
			        Function,
			        NewMaterialExpressionDivide(Function,
			                                    NewMaterialExpressionSubtract(Function, NoiseBandsCheck, NoiseThresholdLow),
			                                    NewMaterialExpressionSubtract(Function, NoiseThresholdHigh, NoiseThresholdLow)),
			        0.0f,
			        1.0f),
			    NoiseBandsCheck);

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Result);
		}

		void ImporterBlendClearCoat(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);

			UMaterialExpressionFunctionInput* Base =
			    NewMaterialExpressionFunctionInput(Function, TEXT("base"), EFunctionInputType::FunctionInput_MaterialAttributes);
			UMaterialExpressionFunctionInput* Layer =
			    NewMaterialExpressionFunctionInput(Function, TEXT("layer"), EFunctionInputType::FunctionInput_MaterialAttributes);
			UMaterialExpressionFunctionInput* Alpha =
			    NewMaterialExpressionFunctionInput(Function, TEXT("weight"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* LayerNormal =
			    NewMaterialExpressionFunctionInput(Function, TEXT("layer_normal"), EFunctionInputType::FunctionInput_Vector3);

			UMaterialExpressionMaterialFunctionCall* AdjustedLayer = NewMaterialExpressionFunctionCall(
			    Function, LoadFunction(TEXT("/Engine/Functions/MaterialLayerFunctions"), TEXT("MatLayerBlend_ReplaceNormals")), {Layer, LayerNormal});
			UMaterialExpressionMaterialFunctionCall* Blend = NewMaterialExpressionFunctionCall(
			    Function, LoadFunction(TEXT("/Engine/Functions/MaterialLayerFunctions"), TEXT("MatLayerBlend_Standard")),
			    {Base, AdjustedLayer, Alpha});

			UMaterialExpressionBreakMaterialAttributes* BaseBreak  = NewMaterialExpressionBreakMaterialAttributes(Function, Base);
			UMaterialExpressionBreakMaterialAttributes* LayerBreak = NewMaterialExpressionBreakMaterialAttributes(Function, AdjustedLayer);
			UMaterialExpressionBreakMaterialAttributes* BlendBreak = NewMaterialExpressionBreakMaterialAttributes(Function, Blend);

			// use ClearCoat as some kind of flag: if it's set, functions getting this as input know, there has been a "clearcoatable" layer
			UMaterialExpressionConstant* ClearCoat = NewMaterialExpressionConstant(Function, 1.0f);

			// blend the ClearCoatRoughness out of the Roughness and the ClearCoatRoughness of the Layer, using its ClearCoat as the weight
			UMaterialExpressionLinearInterpolate* ClearCoatRoughness =
			    NewMaterialExpressionLinearInterpolate(Function, {LayerBreak, 3}, {LayerBreak, 13}, {LayerBreak, 12});

			UMaterialExpressionMakeMaterialAttributes* BSDF =
			    NewMaterialExpressionMakeMaterialAttributes(Function,
			                                                {BaseBreak, 0},      // BaseColor purely from the base
			                                                {BlendBreak, 1},     // Metallic
			                                                {BlendBreak, 2},     // Specular
			                                                {LayerBreak, 3},     // Roughness purely from the layer
			                                                {BlendBreak, 4},     // EmissiveColor
			                                                {BlendBreak, 5},     // Opacity,
			                                                {BlendBreak, 6},     // OpacityMask
			                                                {LayerBreak, 7},     // Normal purely from the layer
			                                                {BlendBreak, 8},     // WorldPositionOffset,
			                                                {BlendBreak, 9},    // SubsurfaceColor,
			                                                ClearCoat,           // ClearCoat
			                                                ClearCoatRoughness,  // ClearCoatRoughness
			                                                {BlendBreak, 12},    // AmbientOcclusion,
			                                                {BlendBreak, 13},    // Refraction
			                                                {BlendBreak, 14},    // CustomizedUVs0
			                                                {BlendBreak, 15},    // CustomizedUVs1
			                                                {BlendBreak, 16},    // CustomizedUVs2
			                                                {BlendBreak, 17},    // CustomizedUVs3
			                                                {BlendBreak, 18},    // CustomizedUVs4
			                                                {BlendBreak, 19},    // CustomizedUVs5
			                                                {BlendBreak, 20},    // CustomizedUVs6
			                                                {BlendBreak, 21},    // CustomizedUVs7
			                                                {BlendBreak, 22});   // PixelDepthOffset

			NewMaterialExpressionFunctionOutput(Function, TEXT("bsdf"), BSDF);
		}

		void ImporterBlendColors(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* MakeFloat3 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat3.MakeFloat3"));

			Function->Description = TEXT("Helper function for texture blending");

			UMaterialFunction* ImporterCalculateHue        = LoadFunction(TEXT("mdlimporter_calculate_hue"));
			UMaterialFunction* ImporterCalculateSaturation = LoadFunction(TEXT("mdlimporter_calculate_saturation"));
			UMaterialFunction* ImporterHSVToRGB            = LoadFunction(TEXT("mdlimporter_hsv_to_rgb"));

			UMaterialExpressionFunctionInput* Top =
			    NewMaterialExpressionFunctionInput(Function, TEXT("top"), EFunctionInputType::FunctionInput_Vector3);
			UMaterialExpressionFunctionInput* Bottom =
			    NewMaterialExpressionFunctionInput(Function, TEXT("bottom"), EFunctionInputType::FunctionInput_Vector3);
			UMaterialExpressionFunctionInput* Weight =
			    NewMaterialExpressionFunctionInput(Function, TEXT("weight"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* Mode =
			    NewMaterialExpressionFunctionInput(Function, TEXT("mode"), EFunctionInputType::FunctionInput_Scalar);

			UMaterialExpressionComponentMask* TopX    = NewMaterialExpressionComponentMask(Function, Top, 1);
			UMaterialExpressionComponentMask* TopY    = NewMaterialExpressionComponentMask(Function, Top, 2);
			UMaterialExpressionComponentMask* TopZ    = NewMaterialExpressionComponentMask(Function, Top, 4);
			UMaterialExpressionComponentMask* BottomX = NewMaterialExpressionComponentMask(Function, Bottom, 1);
			UMaterialExpressionComponentMask* BottomY = NewMaterialExpressionComponentMask(Function, Bottom, 2);
			UMaterialExpressionComponentMask* BottomZ = NewMaterialExpressionComponentMask(Function, Bottom, 4);

			UMaterialExpression*                     Blend    = Top;
			UMaterialExpressionAdd*                  Add      = NewMaterialExpressionAdd(Function, Top, Bottom);
			UMaterialExpressionMultiply*             Multiply = NewMaterialExpressionMultiply(Function, Top, Bottom);
			UMaterialExpressionMaterialFunctionCall* Screen   = NewMaterialExpressionFunctionCall(
                Function, LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends"), TEXT("Blend_Screen")), {Top, Bottom});
			UMaterialExpressionMaterialFunctionCall* Overlay = NewMaterialExpressionFunctionCall(
			    Function, LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends"), TEXT("Blend_Overlay")), {Top, Bottom});
			UMaterialExpressionMax* BrightnessTop    = NewMaterialExpressionMax(Function, TopX, TopY, TopZ);
			UMaterialExpressionMax* BrightnessBottom = NewMaterialExpressionMax(Function, BottomX, BottomY, BottomZ);
			UMaterialExpressionIf*  Brightness       = NewMaterialExpressionIfEqual(
                Function, BrightnessBottom, 0.0f,
                NewMaterialExpressionFunctionCall(Function, MakeFloat3, {BrightnessTop, BrightnessTop, BrightnessTop}),
                NewMaterialExpressionMultiply(Function, Bottom, NewMaterialExpressionDivide(Function, BrightnessTop, BrightnessBottom)));
			UMaterialExpressionIf* Color = NewMaterialExpressionIfEqual(
			    Function, BrightnessTop, 0.0f,
			    NewMaterialExpressionFunctionCall(Function, MakeFloat3, {BrightnessBottom, BrightnessBottom, BrightnessBottom}),
			    NewMaterialExpressionMultiply(Function, Top, NewMaterialExpressionDivide(Function, BrightnessBottom, BrightnessTop)));
			UMaterialExpressionMaterialFunctionCall* Exclusion = NewMaterialExpressionFunctionCall(
			    Function, LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends"), TEXT("Blend_Exclusion")), {Top, Bottom});
			UMaterialExpressionMultiply*             Average = NewMaterialExpressionMultiply(Function, Add, 0.5f);
			UMaterialExpressionMaterialFunctionCall* Lighten = NewMaterialExpressionFunctionCall(
			    Function, LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends"), TEXT("Blend_Lighten")), {Top, Bottom});
			UMaterialExpressionMaterialFunctionCall* Darken = NewMaterialExpressionFunctionCall(
			    Function, LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends"), TEXT("Blend_Darken")), {Top, Bottom});
			UMaterialExpressionSubtract* Sub = NewMaterialExpressionSubtract(Function, Add, 1.0f);
			UMaterialExpressionOneMinus* Negation =
			    NewMaterialExpressionOneMinus(Function, NewMaterialExpressionAbs(Function, NewMaterialExpressionOneMinus(Function, Add)));
			UMaterialExpressionMaterialFunctionCall* Difference = NewMaterialExpressionFunctionCall(
			    Function, LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends"), TEXT("Blend_Difference")), {Top, Bottom});

			// Note: the build-in Function /Engine/Functions/Engine_MaterialFunctions03/Blends/Blend_SoftLight make something different!!
			UMaterialExpressionMultiply* LessThanPointFive = NewMaterialExpressionMultiply(
			    Function,
			    2.0f,
			    NewMaterialExpressionAdd(
			        Function,
			        NewMaterialExpressionMultiply(Function, Top, Bottom),
			        NewMaterialExpressionMultiply(Function, {Bottom, Bottom, NewMaterialExpressionSubtract(Function, 0.5f, Top)})));
			UMaterialExpressionMaterialFunctionCall* SquareRootBottom = NewMaterialExpressionFunctionCall(
			    Function, MakeFloat3,
			    {NewMaterialExpressionSquareRoot(Function, NewMaterialExpressionComponentMask(Function, Bottom, 1)),
			     NewMaterialExpressionSquareRoot(Function, NewMaterialExpressionComponentMask(Function, Bottom, 2)),
			     NewMaterialExpressionSquareRoot(Function, NewMaterialExpressionComponentMask(Function, Bottom, 4))});
			UMaterialExpressionMultiply* GreaterThanPointFive = NewMaterialExpressionMultiply(
			    Function,
			    2.0f,
			    NewMaterialExpressionSubtract(
			        Function,
			        NewMaterialExpressionAdd(
			            Function, NewMaterialExpressionMultiply(Function, SquareRootBottom, NewMaterialExpressionSubtract(Function, Top, 0.5f)),
			            Bottom),
			        NewMaterialExpressionMultiply(Function, Top, Bottom)));
			UMaterialExpressionMaterialFunctionCall* SoftLight = NewMaterialExpressionFunctionCall(
			    Function, MakeFloat3,
			    {NewMaterialExpressionIfLess(Function, TopX, 0.5f, NewMaterialExpressionComponentMask(Function, LessThanPointFive, 1),
			                                 NewMaterialExpressionComponentMask(Function, GreaterThanPointFive, 1)),
			     NewMaterialExpressionIfLess(Function, TopY, 0.5f, NewMaterialExpressionComponentMask(Function, LessThanPointFive, 2),
			                                 NewMaterialExpressionComponentMask(Function, GreaterThanPointFive, 2)),
			     NewMaterialExpressionIfLess(Function, TopZ, 0.5f, NewMaterialExpressionComponentMask(Function, LessThanPointFive, 4),
			                                 NewMaterialExpressionComponentMask(Function, GreaterThanPointFive, 4))});

			UMaterialExpressionMaterialFunctionCall* ColorDodge = NewMaterialExpressionFunctionCall(
			    Function, LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends"), TEXT("Blend_ColorDodge")), {Top, Bottom});
			UMaterialExpressionMin* ReflectTemp = NewMaterialExpressionMin(
			    Function,
			    NewMaterialExpressionMultiply(Function, Bottom,
			                                  NewMaterialExpressionDivide(Function, Bottom, NewMaterialExpressionOneMinus(Function, Top))),
			    1.0f);
			UMaterialExpression* Reflect = NewMaterialExpressionFunctionCall(
			    Function, MakeFloat3,
			    {NewMaterialExpressionIfEqual(Function, TopX, 1.0f, 1.0f, NewMaterialExpressionComponentMask(Function, ReflectTemp, 1)),
			     NewMaterialExpressionIfEqual(Function, TopY, 1.0f, 1.0f, NewMaterialExpressionComponentMask(Function, ReflectTemp, 2)),
			     NewMaterialExpressionIfEqual(Function, TopZ, 1.0f, 1.0f, NewMaterialExpressionComponentMask(Function, ReflectTemp, 4))});
			UMaterialExpressionMaterialFunctionCall* ColorBurn = NewMaterialExpressionFunctionCall(
			    Function, LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends"), TEXT("Blend_ColorBurn")), {Top, Bottom});
			UMaterialExpressionAdd* Phoenix =
			    NewMaterialExpressionAdd(Function,
			                             NewMaterialExpressionMin(Function, Top, Bottom),
			                             NewMaterialExpressionOneMinus(Function, NewMaterialExpressionMax(Function, Top, Bottom)));
			UMaterialExpressionMaterialFunctionCall* HardLight = NewMaterialExpressionFunctionCall(
			    Function, LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends"), TEXT("Blend_HardLight")), {Top, Bottom});
			UMaterialExpressionMaterialFunctionCall* PinLight = NewMaterialExpressionFunctionCall(
			    Function, LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends"), TEXT("Blend_PinLight")), {Top, Bottom});
			UMaterialExpression* HardMix = NewMaterialExpressionFunctionCall(
			    Function, MakeFloat3,
			    {NewMaterialExpressionIfGreater(Function, NewMaterialExpressionComponentMask(Function, Add, 1), 1.0f, 1.0f, 0.0f),
			     NewMaterialExpressionIfGreater(Function, NewMaterialExpressionComponentMask(Function, Add, 2), 1.0f, 1.0f, 0.0f),
			     NewMaterialExpressionIfGreater(Function, NewMaterialExpressionComponentMask(Function, Add, 4), 1.0f, 1.0f, 0.0f)});
			UMaterialExpressionMaterialFunctionCall* LinearDodge = NewMaterialExpressionFunctionCall(
			    Function, LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends"), TEXT("Blend_LinearDodge")), {Top, Bottom});
			UMaterialExpressionMaterialFunctionCall* LinearBurn = NewMaterialExpressionFunctionCall(
			    Function, LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions03/Blends"), TEXT("Blend_LinearBurn")), {Top, Bottom});
			UMaterialExpressionMultiply* SpotLight      = NewMaterialExpressionMultiply(Function, Multiply, 2.0f);
			UMaterialExpressionAdd*      SpotLightBlend = NewMaterialExpressionAdd(Function, Multiply, Bottom);
			UMaterialExpression*         HueArgument =
			    NewMaterialExpressionFunctionCall(Function, MakeFloat3,
			                                      {NewMaterialExpressionFunctionCall(Function, ImporterCalculateHue, {Top}),
			                                       NewMaterialExpressionFunctionCall(Function, ImporterCalculateSaturation, {Bottom}),
			                                       NewMaterialExpressionMax(Function, BottomX, BottomY, BottomZ)});
			UMaterialExpressionMaterialFunctionCall* Hue = NewMaterialExpressionFunctionCall(Function, ImporterHSVToRGB, {HueArgument});
			UMaterialExpression*                     SaturationArgument =
			    NewMaterialExpressionFunctionCall(Function, MakeFloat3,
			                                      {NewMaterialExpressionFunctionCall(Function, ImporterCalculateHue, {Bottom}),
			                                       NewMaterialExpressionFunctionCall(Function, ImporterCalculateSaturation, {Top}),
			                                       NewMaterialExpressionMax(Function, BottomX, BottomY, BottomZ)});
			UMaterialExpressionMaterialFunctionCall* Saturation = NewMaterialExpressionFunctionCall(Function, ImporterHSVToRGB, {SaturationArgument});
			UMaterialExpressionIf*                   ColorBlend = NewMaterialExpressionSwitch(
                Function, Mode, {Blend,    Add,     Multiply,    Screen,     Overlay,   Brightness,     Color,   Exclusion, Average, Lighten,
                                 Darken,   Sub,     Negation,    Difference, SoftLight, ColorDodge,     Reflect, ColorBurn, Phoenix, HardLight,
                                 PinLight, HardMix, LinearDodge, LinearBurn, SpotLight, SpotLightBlend, Hue,     Saturation});

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"),
			                                    NewMaterialExpressionLinearInterpolate(Function, Bottom, ColorBlend, Weight));
		}

		void ImporterCalculateHue(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* MathMaxValue = LoadFunction(TEXT("mdl_math_max_value"));
			UMaterialFunction* MathMinValue = LoadFunction(TEXT("mdl_math_min_value"));

			UMaterialExpressionFunctionInput* RGB =
			    NewMaterialExpressionFunctionInput(Function, TEXT("rgb"), EFunctionInputType::FunctionInput_Vector3);

			UMaterialExpressionComponentMask* RGBX = NewMaterialExpressionComponentMask(Function, RGB, 1);
			UMaterialExpressionComponentMask* RGBY = NewMaterialExpressionComponentMask(Function, RGB, 2);
			UMaterialExpressionComponentMask* RGBZ = NewMaterialExpressionComponentMask(Function, RGB, 4);

			UMaterialExpressionMaterialFunctionCall* Max   = NewMaterialExpressionFunctionCall(Function, MathMaxValue, {RGB});
			UMaterialExpressionMaterialFunctionCall* Min   = NewMaterialExpressionFunctionCall(Function, MathMinValue, {RGB});
			UMaterialExpressionSubtract*             Range = NewMaterialExpressionSubtract(Function, Max, Min);

			UMaterialExpressionIf* Hue = NewMaterialExpressionIfEqual(
			    Function, Range, 0.0f, 0.0f,
			    NewMaterialExpressionMultiply(
			        Function,
			        1.0f / 6.0f,
			        NewMaterialExpressionIfEqual(
			            Function, Max, RGBX, NewMaterialExpressionDivide(Function, NewMaterialExpressionSubtract(Function, RGBY, RGBZ), Range),
			            NewMaterialExpressionIfEqual(
			                Function, Max, RGBY,
			                NewMaterialExpressionAdd(
			                    Function, 2.0f, NewMaterialExpressionDivide(Function, NewMaterialExpressionSubtract(Function, RGBZ, RGBX), Range)),
			                NewMaterialExpressionAdd(
			                    Function,
			                    4.0f,
			                    NewMaterialExpressionDivide(Function, NewMaterialExpressionSubtract(Function, RGBX, RGBY), Range))))));

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"),
			                                    NewMaterialExpressionIfLess(Function, Hue, 0.0f, NewMaterialExpressionAdd(Function, Hue, 1.0f), Hue));
		}

		void ImporterCalculateSaturation(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* MathMaxValue = LoadFunction(TEXT("mdl_math_max_value"));
			UMaterialFunction* MathMinValue = LoadFunction(TEXT("mdl_math_min_value"));

			UMaterialExpressionFunctionInput* RGB =
			    NewMaterialExpressionFunctionInput(Function, TEXT("rgb"), EFunctionInputType::FunctionInput_Vector3);

			UMaterialExpressionMaterialFunctionCall* Max = NewMaterialExpressionFunctionCall(Function, MathMaxValue, {RGB});
			UMaterialExpressionMaterialFunctionCall* Min = NewMaterialExpressionFunctionCall(Function, MathMinValue, {RGB});
			UMaterialExpressionOneMinus* Saturation      = NewMaterialExpressionOneMinus(Function, NewMaterialExpressionDivide(Function, Min, Max));

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), NewMaterialExpressionIfEqual(Function, Max, 0.0f, 0.0f, Saturation));
		}

		void ImporterComputeCubicTransform(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* MathMultiplyFloat4x4Float4 = LoadFunction(TEXT("mdl_math_multiply_float4x4_float4"));

			UMaterialExpressionFunctionInput* ProjectionTransform0 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("projection_transform_0"), EFunctionInputType::FunctionInput_Vector4);
			UMaterialExpressionFunctionInput* ProjectionTransform1 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("projection_transform_1"), EFunctionInputType::FunctionInput_Vector4);
			UMaterialExpressionFunctionInput* ProjectionTransform2 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("projection_transform_2"), EFunctionInputType::FunctionInput_Vector4);
			UMaterialExpressionFunctionInput* ProjectionTransform3 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("projection_transform_3"), EFunctionInputType::FunctionInput_Vector4);
			UMaterialExpressionFunctionInput* Normal =
			    NewMaterialExpressionFunctionInput(Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3);
			UMaterialExpressionFunctionInput* Position =
			    NewMaterialExpressionFunctionInput(Function, TEXT("pos"), EFunctionInputType::FunctionInput_Vector3);

			UMaterialExpressionMaterialFunctionCall* TransformedPosition =
			    NewMaterialExpressionFunctionCall(Function, MathMultiplyFloat4x4Float4,
			                                      {ProjectionTransform0, ProjectionTransform1, ProjectionTransform2, ProjectionTransform3,
			                                       NewMaterialExpressionAppendVector(Function, Position, 1.0f)});
			UMaterialExpressionMaterialFunctionCall* TempNormal =
			    NewMaterialExpressionFunctionCall(Function, MathMultiplyFloat4x4Float4,
			                                      {ProjectionTransform0, ProjectionTransform1, ProjectionTransform2, ProjectionTransform3,
			                                       NewMaterialExpressionAppendVector(Function, Normal, 0.0f)});
			UMaterialExpressionIf* TransformedNormal = NewMaterialExpressionIfLess(
			    Function,
			    NewMaterialExpressionDotProduct(Function, NewMaterialExpressionComponentMask(Function, TransformedPosition, 7),
			                                    NewMaterialExpressionComponentMask(Function, TempNormal, 7)),
			    0.0f, NewMaterialExpressionNegate(Function, TempNormal), TempNormal);
			UMaterialExpressionComponentMask* NX    = NewMaterialExpressionComponentMask(Function, TransformedNormal, 1);
			UMaterialExpressionComponentMask* NY    = NewMaterialExpressionComponentMask(Function, TransformedNormal, 2);
			UMaterialExpressionComponentMask* NZ    = NewMaterialExpressionComponentMask(Function, TransformedNormal, 4);
			UMaterialExpressionAbs*           AbsNX = NewMaterialExpressionAbs(Function, NX);
			UMaterialExpressionAbs*           AbsNY = NewMaterialExpressionAbs(Function, NY);
			UMaterialExpressionAbs*           AbsNZ = NewMaterialExpressionAbs(Function, NZ);

			UMaterialExpressionSubtract* NegativeProjectionTransform0 = NewMaterialExpressionNegate(Function, ProjectionTransform0);
			UMaterialExpressionSubtract* NegativeProjectionTransform1 = NewMaterialExpressionNegate(Function, ProjectionTransform1);

			// Find out on which cube face is the intersection point
			// transform is then planar, but based on a rotated/flipped coordsys
			UMaterialExpressionIf* ZCheck0 = NewMaterialExpressionIfLess(
			    Function, AbsNX, AbsNZ,
			    NewMaterialExpressionIfLess(Function, AbsNY, AbsNZ,
			                                NewMaterialExpressionIfLess(Function, NZ, 0.0f, NegativeProjectionTransform0, ProjectionTransform0),
			                                ProjectionTransform0),
			    ProjectionTransform0);
			UMaterialExpressionIf* YCheck0 = NewMaterialExpressionIfLess(
			    Function, AbsNX, AbsNY,
			    NewMaterialExpressionIfLess(Function, AbsNZ, AbsNY,
			                                NewMaterialExpressionIfGreater(Function, NY, 0.0f, NegativeProjectionTransform0, ProjectionTransform0),
			                                ZCheck0),
			    ZCheck0);
			UMaterialExpressionIf* Result0 = NewMaterialExpressionIfLess(
			    Function, AbsNY, AbsNX,
			    NewMaterialExpressionIfLess(Function, AbsNZ, AbsNX,
			                                NewMaterialExpressionIfGreater(Function, NX, 0.0f, ProjectionTransform1, NegativeProjectionTransform1),
			                                YCheck0),
			    YCheck0);

			UMaterialExpressionIf* YCheck1 = NewMaterialExpressionIfLess(
			    Function, AbsNX, AbsNY, NewMaterialExpressionIfLess(Function, AbsNZ, AbsNY, ProjectionTransform2, ProjectionTransform1),
			    ProjectionTransform1);
			UMaterialExpressionIf* Result1 = NewMaterialExpressionIfLess(
			    Function, AbsNY, AbsNX, NewMaterialExpressionIfLess(Function, AbsNZ, AbsNX, ProjectionTransform2, YCheck1), YCheck1);

			UMaterialExpressionIf* YCheck2 = NewMaterialExpressionIfLess(
			    Function, AbsNX, AbsNY, NewMaterialExpressionIfLess(Function, AbsNZ, AbsNY, ProjectionTransform1, ProjectionTransform2),
			    ProjectionTransform2);
			UMaterialExpressionIf* Result2 = NewMaterialExpressionIfLess(
			    Function, AbsNY, AbsNX, NewMaterialExpressionIfLess(Function, AbsNZ, AbsNX, ProjectionTransform0, YCheck2), YCheck2);

			UMaterialExpression* Result3 = ProjectionTransform3;

			NewMaterialExpressionFunctionOutput(Function, TEXT("result_0"), Result0);
			NewMaterialExpressionFunctionOutput(Function, TEXT("result_1"), Result1);
			NewMaterialExpressionFunctionOutput(Function, TEXT("result_2"), Result2);
			NewMaterialExpressionFunctionOutput(Function, TEXT("result_3"), Result3);
		}

		void ImporterComputeCylindricTransform(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* MakeFloat3 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat3.MakeFloat3"));
			UMaterialFunction* MakeFloat4 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat4.MakeFloat4"));
			UMaterialFunction* MathMultiplyFloat4x4Float4   = LoadFunction(TEXT("mdl_math_multiply_float4x4_float4"));
			UMaterialFunction* MathMultiplyFloat4x4Float4x4 = LoadFunction(TEXT("mdl_math_multiply_float4x4_float4x4"));

			UMaterialExpressionFunctionInput* ProjectionTransform0 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("projection_transform_0"), EFunctionInputType::FunctionInput_Vector4);
			UMaterialExpressionFunctionInput* ProjectionTransform1 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("projection_transform_1"), EFunctionInputType::FunctionInput_Vector4);
			UMaterialExpressionFunctionInput* ProjectionTransform2 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("projection_transform_2"), EFunctionInputType::FunctionInput_Vector4);
			UMaterialExpressionFunctionInput* ProjectionTransform3 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("projection_transform_3"), EFunctionInputType::FunctionInput_Vector4);
			UMaterialExpressionFunctionInput* Normal =
			    NewMaterialExpressionFunctionInput(Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3);
			UMaterialExpressionFunctionInput* Position =
			    NewMaterialExpressionFunctionInput(Function, TEXT("pos"), EFunctionInputType::FunctionInput_Vector3);
			UMaterialExpressionFunctionInput* Infinite = NewMaterialExpressionFunctionInput(
			    Function, TEXT("infinite"), EFunctionInputType::FunctionInput_StaticBool, NewMaterialExpressionStaticBool(Function, false));
			UMaterialExpressionFunctionInput* Normalized = NewMaterialExpressionFunctionInput(
			    Function, TEXT("normalized"), EFunctionInputType::FunctionInput_StaticBool, NewMaterialExpressionStaticBool(Function, false));

			UMaterialExpressionMaterialFunctionCall* TransformedPosition =
			    NewMaterialExpressionFunctionCall(Function, MathMultiplyFloat4x4Float4,
			                                      {ProjectionTransform0, ProjectionTransform1, ProjectionTransform2, ProjectionTransform3,
			                                       NewMaterialExpressionAppendVector(Function, Position, 1.0f)});
			UMaterialExpressionComponentMask* TransformedPositionX = NewMaterialExpressionComponentMask(Function, TransformedPosition, 1);
			UMaterialExpressionComponentMask* TransformedPositionY = NewMaterialExpressionComponentMask(Function, TransformedPosition, 2);
			UMaterialExpressionComponentMask* TransformedPositionZ = NewMaterialExpressionComponentMask(Function, TransformedPosition, 4);

			UMaterialExpressionMaterialFunctionCall* TempNormal =
			    NewMaterialExpressionFunctionCall(Function, MathMultiplyFloat4x4Float4,
			                                      {ProjectionTransform0, ProjectionTransform1, ProjectionTransform2, ProjectionTransform3,
			                                       NewMaterialExpressionAppendVector(Function, Normal, 0.0f)});
			UMaterialExpressionIf* TransformedNormal =
			    NewMaterialExpressionIfLess(Function, NewMaterialExpressionDotProduct(Function, TransformedPosition, TempNormal), 0.0f,
			                                NewMaterialExpressionNegate(Function, TempNormal), TempNormal);
			UMaterialExpressionComponentMask* TransformedNormalX = NewMaterialExpressionComponentMask(Function, TransformedNormal, 1);
			UMaterialExpressionComponentMask* TransformedNormalY = NewMaterialExpressionComponentMask(Function, TransformedNormal, 2);
			UMaterialExpressionComponentMask* TransformedNormalZ = NewMaterialExpressionComponentMask(Function, TransformedNormal, 4);
			UMaterialExpressionAbs*           AbsNX              = NewMaterialExpressionAbs(Function, TransformedNormalX);
			UMaterialExpressionAbs*           AbsNY              = NewMaterialExpressionAbs(Function, TransformedNormalY);
			UMaterialExpressionAbs*           AbsNZ              = NewMaterialExpressionAbs(Function, TransformedNormalZ);

			UMaterialExpressionSquareRoot* Length = NewMaterialExpressionSquareRoot(
			    Function, NewMaterialExpressionAdd(Function, NewMaterialExpressionSquare(Function, TransformedPositionX),
			                                       NewMaterialExpressionSquare(Function, TransformedPositionY)));
			UMaterialExpressionStaticSwitch* OrgDist = NewMaterialExpressionStaticSwitch(Function, Normalized, 1.0f / PI, Length);
			UMaterialExpressionNormalize*    BaseZ   = NewMaterialExpressionNormalize(
                Function, NewMaterialExpressionFunctionCall(Function, MakeFloat3, {TransformedPositionX, TransformedPositionY, 0.0f}));
			UMaterialExpressionConstant3Vector* BaseY = NewMaterialExpressionConstant(Function, 0.0f, 0.0f, 1.0f);
			UMaterialExpressionNormalize* BaseX = NewMaterialExpressionNormalize(Function, NewMaterialExpressionCrossProduct(Function, BaseY, BaseZ));

			UMaterialExpressionMaterialFunctionCall* Offset2TimesBase = NewMaterialExpressionFunctionCall(
			    Function, MathMultiplyFloat4x4Float4x4,
			    {NewMaterialExpressionFunctionCall(
			         Function, MakeFloat4,
			         {1.0f, 0.0f, 0.0f,
			          NewMaterialExpressionMultiply(Function, OrgDist,
			                                        NewMaterialExpressionArctangent2(Function, TransformedPositionY, TransformedPositionX))}),
			     NewMaterialExpressionFunctionCall(Function, MakeFloat4, {0.0f, 1.0f, 0.0f, TransformedPositionZ}),
			     {0.0f, 0.0f, 1.0f, 0.0f},
			     {0.0f, 0.0f, 0.0f, 1.0f},
			     NewMaterialExpressionAppendVector(Function, BaseX, 0.0f),
			     NewMaterialExpressionAppendVector(Function, BaseY, 0.0f),
			     NewMaterialExpressionAppendVector(Function, BaseZ, 0.0f),
			     {0.0f, 0.0f, 0.0f, 1.0f}});
			UMaterialExpressionMaterialFunctionCall* Result = NewMaterialExpressionFunctionCall(
			    Function, MathMultiplyFloat4x4Float4x4,
			    {
			        {Offset2TimesBase, 0},
			        {Offset2TimesBase, 1},
			        {Offset2TimesBase, 2},
			        {Offset2TimesBase, 3},
			        NewMaterialExpressionFunctionCall(
			            Function, MakeFloat4,
			            {1.0f, 0.0f, 0.0f, NewMaterialExpressionNegate(Function, NewMaterialExpressionComponentMask(Function, Position, 1))}),
			        NewMaterialExpressionFunctionCall(
			            Function, MakeFloat4,
			            {0.0f, 1.0f, 0.0f, NewMaterialExpressionNegate(Function, NewMaterialExpressionComponentMask(Function, Position, 2))}),
			        NewMaterialExpressionFunctionCall(
			            Function, MakeFloat4,
			            {0.0f, 0.0f, 1.0f, NewMaterialExpressionNegate(Function, NewMaterialExpressionComponentMask(Function, Position, 4))}),
			        {0.0f, 0.0f, 0.0f, 1.0f},
			    });

			UMaterialExpressionStaticSwitch* Result0 = NewMaterialExpressionStaticSwitch(
			    Function, Infinite, {Result, 0},
			    NewMaterialExpressionIfLess(
			        Function, AbsNX, AbsNZ,
			        NewMaterialExpressionIfLess(
			            Function, AbsNY, AbsNZ,
			            NewMaterialExpressionIfLess(Function, TransformedNormalZ, 0.0f, NewMaterialExpressionNegate(Function, ProjectionTransform0),
			                                        ProjectionTransform0),
			            {Result, 0}),
			        {Result, 0}));

			UMaterialExpressionStaticSwitch* Result1 = NewMaterialExpressionStaticSwitch(
			    Function, Infinite, {Result, 1},
			    NewMaterialExpressionIfLess(Function, AbsNX, AbsNZ,
			                                NewMaterialExpressionIfLess(Function, AbsNY, AbsNZ, ProjectionTransform1, {Result, 1}), {Result, 1}));

			UMaterialExpressionStaticSwitch* Result2 = NewMaterialExpressionStaticSwitch(
			    Function, Infinite, {Result, 2},
			    NewMaterialExpressionIfLess(Function, AbsNX, AbsNZ,
			                                NewMaterialExpressionIfLess(Function, AbsNY, AbsNZ, ProjectionTransform2, {Result, 2}), {Result, 2}));

			UMaterialExpressionStaticSwitch* Result3 = NewMaterialExpressionStaticSwitch(
			    Function, Infinite, {Result, 3},
			    NewMaterialExpressionIfLess(Function, AbsNX, AbsNZ,
			                                NewMaterialExpressionIfLess(Function, AbsNY, AbsNZ, ProjectionTransform3, {Result, 3}), {Result, 3}));

			NewMaterialExpressionFunctionOutput(Function, TEXT("result_0"), Result0);
			NewMaterialExpressionFunctionOutput(Function, TEXT("result_1"), Result1);
			NewMaterialExpressionFunctionOutput(Function, TEXT("result_2"), Result2);
			NewMaterialExpressionFunctionOutput(Function, TEXT("result_3"), Result3);
		}

		void ImporterComputeSphericProjection(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* MakeFloat3 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat3.MakeFloat3"));

			UMaterialExpressionFunctionInput* Dir =
			    NewMaterialExpressionFunctionInput(Function, TEXT("dir"), EFunctionInputType::FunctionInput_Vector4);
			UMaterialExpressionFunctionInput* Normalized = NewMaterialExpressionFunctionInput(
			    Function, TEXT("normalized"), EFunctionInputType::FunctionInput_StaticBool, NewMaterialExpressionStaticBool(Function, false));

			UMaterialExpressionComponentMask* DirX = NewMaterialExpressionComponentMask(Function, Dir, 1);
			UMaterialExpressionComponentMask* DirY = NewMaterialExpressionComponentMask(Function, Dir, 2);
			UMaterialExpressionComponentMask* DirZ = NewMaterialExpressionComponentMask(Function, Dir, 4);

			UMaterialExpressionAdd* XY =
			    NewMaterialExpressionAdd(Function, NewMaterialExpressionSquare(Function, DirX), NewMaterialExpressionSquare(Function, DirY));
			UMaterialExpressionStaticSwitch* OrgDist = NewMaterialExpressionStaticSwitch(
			    Function, Normalized, 1.0f / PI,
			    NewMaterialExpressionSquareRoot(Function, NewMaterialExpressionAdd(Function, XY, NewMaterialExpressionSquare(Function, DirZ))));

			UMaterialExpressionMaterialFunctionCall* UVCoordXY = NewMaterialExpressionFunctionCall(
			    Function, MakeFloat3,
			    {NewMaterialExpressionMultiply(Function, OrgDist, NewMaterialExpressionArctangent2(Function, DirY, DirZ)),
			     NewMaterialExpressionMultiply(Function, OrgDist,
			                                   NewMaterialExpressionArctangent2(Function, DirZ, NewMaterialExpressionSquareRoot(Function, XY))),
			     OrgDist});
			UMaterialExpressionMaterialFunctionCall* UVCoordZ = NewMaterialExpressionFunctionCall(
			    Function, MakeFloat3,
			    {0.0f,
			     NewMaterialExpressionMultiply(Function, OrgDist,
			                                   NewMaterialExpressionArctangent2(Function, DirZ, NewMaterialExpressionSquareRoot(Function, XY))),
			     OrgDist});

			UMaterialExpressionIf* UVCoord = NewMaterialExpressionIfGreater(
			    Function, NewMaterialExpressionAbs(Function, DirX), 0.0f, UVCoordXY,
			    NewMaterialExpressionIfGreater(Function, NewMaterialExpressionAbs(Function, DirY), 0.0f, UVCoordXY,
			                                   NewMaterialExpressionIfGreater(Function, NewMaterialExpressionAbs(Function, DirZ), 0.0f, UVCoordZ,
			                                                                  NewMaterialExpressionConstant(Function, 0.0f, 0.0f, 0.0f))));

			NewMaterialExpressionFunctionOutput(Function, TEXT("uv_coord"), UVCoord);
		}

		void ImporterComputeSphericTransform(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* ImporterComputeSphericProjection = LoadFunction(TEXT("mdlimporter_compute_spheric_projection"));
			UMaterialFunction* MakeFloat4 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat4.MakeFloat4"));
			UMaterialFunction* MathMultiplyFloat4x4Float4   = LoadFunction(TEXT("mdl_math_multiply_float4x4_float4"));
			UMaterialFunction* MathMultiplyFloat4x4Float4x4 = LoadFunction(TEXT("mdl_math_multiply_float4x4_float4x4"));

			UMaterialExpressionFunctionInput* ProjectionTransform0 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("projection_transform_0"), EFunctionInputType::FunctionInput_Vector4);
			UMaterialExpressionFunctionInput* ProjectionTransform1 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("projection_transform_1"), EFunctionInputType::FunctionInput_Vector4);
			UMaterialExpressionFunctionInput* ProjectionTransform2 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("projection_transform_2"), EFunctionInputType::FunctionInput_Vector4);
			UMaterialExpressionFunctionInput* ProjectionTransform3 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("projection_transform_3"), EFunctionInputType::FunctionInput_Vector4);
			UMaterialExpressionFunctionInput* Position =
			    NewMaterialExpressionFunctionInput(Function, TEXT("pos"), EFunctionInputType::FunctionInput_Vector3);
			UMaterialExpressionFunctionInput* Normalized = NewMaterialExpressionFunctionInput(
			    Function, TEXT("normalized"), EFunctionInputType::FunctionInput_StaticBool, NewMaterialExpressionStaticBool(Function, false));

			UMaterialExpressionMaterialFunctionCall* TransformedPosition =
			    NewMaterialExpressionFunctionCall(Function, MathMultiplyFloat4x4Float4,
			                                      {ProjectionTransform0, ProjectionTransform1, ProjectionTransform2, ProjectionTransform3,
			                                       NewMaterialExpressionAppendVector(Function, Position, 1.0f)});
			UMaterialExpressionComponentMask* TransformedPositionX = NewMaterialExpressionComponentMask(Function, TransformedPosition, 1);
			UMaterialExpressionComponentMask* TransformedPositionY = NewMaterialExpressionComponentMask(Function, TransformedPosition, 2);

			UMaterialExpressionMaterialFunctionCall* UVCoord =
			    NewMaterialExpressionFunctionCall(Function, ImporterComputeSphericProjection, {TransformedPosition, Normalized});
			UMaterialExpressionNormalize* BaseZ =
			    NewMaterialExpressionNormalize(Function, NewMaterialExpressionComponentMask(Function, TransformedPosition, 7));
			UMaterialExpressionNormalize* BaseX =
			    NewMaterialExpressionNormalize(Function, NewMaterialExpressionCrossProduct(Function, {0.0f, 0.0f, 1.0f}, BaseZ));
			UMaterialExpressionNormalize* BaseY = NewMaterialExpressionNormalize(Function, NewMaterialExpressionCrossProduct(Function, BaseZ, BaseX));

			UMaterialExpressionMaterialFunctionCall* Offset2TimesBase = NewMaterialExpressionFunctionCall(
			    Function, MathMultiplyFloat4x4Float4x4,
			    {NewMaterialExpressionFunctionCall(Function, MakeFloat4,
			                                       {1.0f, 0.0f, 0.0f, NewMaterialExpressionComponentMask(Function, UVCoord, 1)}),
			     NewMaterialExpressionFunctionCall(Function, MakeFloat4,
			                                       {0.0f, 1.0f, 0.0f, NewMaterialExpressionComponentMask(Function, UVCoord, 2)}),
			     {0.0f, 0.0f, 1.0f, 0.0f},
			     {0.0f, 0.0f, 0.0f, 1.0f},
			     NewMaterialExpressionAppendVector(Function, BaseX, 0.0f),
			     NewMaterialExpressionAppendVector(Function, BaseY, 0.0f),
			     NewMaterialExpressionAppendVector(Function, BaseZ, 0.0f),
			     {0.0f, 0.0f, 0.0f, 1.0f}});
			UMaterialExpressionMaterialFunctionCall* Result = NewMaterialExpressionFunctionCall(
			    Function, MathMultiplyFloat4x4Float4x4,
			    {
			        {Offset2TimesBase, 0},
			        {Offset2TimesBase, 1},
			        {Offset2TimesBase, 2},
			        {Offset2TimesBase, 3},
			        NewMaterialExpressionFunctionCall(
			            Function, MakeFloat4,
			            {1.0f, 0.0f, 0.0f, NewMaterialExpressionNegate(Function, NewMaterialExpressionComponentMask(Function, Position, 1))}),
			        NewMaterialExpressionFunctionCall(
			            Function, MakeFloat4,
			            {0.0f, 1.0f, 0.0f, NewMaterialExpressionNegate(Function, NewMaterialExpressionComponentMask(Function, Position, 2))}),
			        NewMaterialExpressionFunctionCall(
			            Function, MakeFloat4,
			            {0.0f, 0.0f, 1.0f, NewMaterialExpressionNegate(Function, NewMaterialExpressionComponentMask(Function, Position, 4))}),
			        {0.0f, 0.0f, 0.0f, 1.0f},
			    });

			NewMaterialExpressionFunctionOutput(Function, TEXT("result_0"), {Result, 0});
			NewMaterialExpressionFunctionOutput(Function, TEXT("result_1"), {Result, 1});
			NewMaterialExpressionFunctionOutput(Function, TEXT("result_2"), {Result, 2});
			NewMaterialExpressionFunctionOutput(Function, TEXT("result_3"), {Result, 3});
		}

		void ImporterComputeTangents(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* MakeFloat3 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat3.MakeFloat3"));
			UMaterialFunction* StateNormal            = LoadFunction(TEXT("mdl_state_normal"));
			UMaterialFunction* StateTextureCoordinate = LoadFunction(TEXT("mdl_state_texture_coordinate"));
			UMaterialFunction* StateTransformVector   = LoadFunction(TEXT("mdl_state_transform_vector"));

			UMaterialExpressionFunctionInput* Space =
			    NewMaterialExpressionFunctionInput(Function, TEXT("space"), EFunctionInputType::FunctionInput_Scalar);

			UMaterialExpressionMaterialFunctionCall* Position = NewMaterialExpressionFunctionCall(Function, StateTextureCoordinate, {});

			UMaterialExpressionMaterialFunctionCall* Normal = NewMaterialExpressionFunctionCall(Function, StateNormal, {});
			UMaterialExpressionNormalize*            UDir   = NewMaterialExpressionNormalize(
                Function, NewMaterialExpressionFunctionCall(Function, StateTransformVector, {Space, 0.0f, {1.0f, 0.0f, 0.0f}}));
			UMaterialExpressionNormalize* VDir = NewMaterialExpressionNormalize(
			    Function, NewMaterialExpressionFunctionCall(Function, StateTransformVector, {Space, 0.0f, {0.0f, 1.0f, 0.0f}}));
			UMaterialExpressionAbs* NormalDotUDir = NewMaterialExpressionAbs(Function, NewMaterialExpressionDotProduct(Function, Normal, UDir));
			UMaterialExpressionAbs* NormalDotVDir = NewMaterialExpressionAbs(Function, NewMaterialExpressionDotProduct(Function, Normal, VDir));

			UMaterialExpressionMaterialFunctionCall* ShadingNormal =
			    NewMaterialExpressionFunctionCall(Function, StateTransformVector, {0.0f, Space, Normal});
			UMaterialExpressionComponentMask* ShadingNormalX = NewMaterialExpressionComponentMask(Function, ShadingNormal, 1);
			UMaterialExpressionComponentMask* ShadingNormalY = NewMaterialExpressionComponentMask(Function, ShadingNormal, 2);
			UMaterialExpressionComponentMask* ShadingNormalZ = NewMaterialExpressionComponentMask(Function, ShadingNormal, 4);
			UMaterialExpressionSubtract*      YZ =
			    NewMaterialExpressionNegate(Function, NewMaterialExpressionMultiply(Function, ShadingNormalY, ShadingNormalZ));
			UMaterialExpressionIf* TangentVPreliminary = NewMaterialExpressionIfGreater(
			    Function, NewMaterialExpressionAbs(Function, ShadingNormalZ), 0.99999f,
			    NewMaterialExpressionNormalize(
			        Function,
			        NewMaterialExpressionFunctionCall(
			            Function, MakeFloat3,
			            {NewMaterialExpressionNegate(Function, NewMaterialExpressionMultiply(Function, ShadingNormalX, ShadingNormalY)),
			             NewMaterialExpressionOneMinus(Function, NewMaterialExpressionSquare(Function, ShadingNormalY)), YZ})),
			    NewMaterialExpressionNormalize(
			        Function,
			        NewMaterialExpressionFunctionCall(
			            Function, MakeFloat3,
			            {NewMaterialExpressionNegate(Function, NewMaterialExpressionMultiply(Function, ShadingNormalX, ShadingNormalZ)), YZ,
			             NewMaterialExpressionOneMinus(Function, NewMaterialExpressionSquare(Function, ShadingNormalZ))})));
			UMaterialExpressionNormalize* TangentUComplexPreliminary =
			    NewMaterialExpressionNormalize(Function, NewMaterialExpressionCrossProduct(Function, TangentVPreliminary, ShadingNormal));
			UMaterialExpressionNormalize* TangentUComplex = NewMaterialExpressionNormalize(
			    Function, NewMaterialExpressionFunctionCall(Function, StateTransformVector, {Space, 0.0f, TangentUComplexPreliminary}));
			UMaterialExpressionNormalize* TangentUSimplePreliminary =
			    NewMaterialExpressionNormalize(Function, NewMaterialExpressionCrossProduct(Function, VDir, Normal));
			UMaterialExpressionIf* TangentUSimple =
			    NewMaterialExpressionIfLess(Function, NewMaterialExpressionDotProduct(Function, TangentUSimplePreliminary, UDir), 0.0f,
			                                NewMaterialExpressionNegate(Function, TangentUSimplePreliminary), TangentUSimplePreliminary);
			UMaterialExpressionIf* TangentUUDirCheck =
			    NewMaterialExpressionIfGreater(Function, NormalDotUDir, 0.999f, TangentUComplex, TangentUSimple);
			UMaterialExpressionIf* TangentU = NewMaterialExpressionIfGreater(Function, NormalDotVDir, 0.999f, TangentUComplex, TangentUUDirCheck);

			UMaterialExpressionNormalize* TangentVComplex = NewMaterialExpressionNormalize(
			    Function, NewMaterialExpressionFunctionCall(Function, StateTransformVector, {Space, 0.0f, TangentVPreliminary}));
			UMaterialExpressionNormalize* TangentVSimple =
			    NewMaterialExpressionNormalize(Function, NewMaterialExpressionCrossProduct(Function, Normal, TangentUSimplePreliminary));
			UMaterialExpressionIf* TangentVUDirCheck =
			    NewMaterialExpressionIfGreater(Function, NormalDotUDir, 0.999f, TangentVComplex, TangentVSimple);
			// As Unreal is left-handed, negate the resulting v-tangent !
			UMaterialExpressionSubtract* TangentV = NewMaterialExpressionNegate(
			    Function, NewMaterialExpressionIfGreater(Function, NormalDotVDir, 0.999f, TangentVComplex, TangentVUDirCheck));

			NewMaterialExpressionFunctionOutput(Function, TEXT("position"), Position);
			NewMaterialExpressionFunctionOutput(Function, TEXT("tangent_u"), TangentU);
			NewMaterialExpressionFunctionOutput(Function, TEXT("tangent_v"), TangentV);
		}

		void ImporterComputeTangentsTransformed(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* MakeFloat3 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat3.MakeFloat3"));
			UMaterialFunction* StateNormal            = LoadFunction(TEXT("mdl_state_normal"));
			UMaterialFunction* StateTextureCoordinate = LoadFunction(TEXT("mdl_state_texture_coordinate"));
			UMaterialFunction* StateTransformVector   = LoadFunction(TEXT("mdl_state_transform_vector"));

			UMaterialExpressionFunctionInput* CoordinateSystem =
			    NewMaterialExpressionFunctionInput(Function, TEXT("coordinate_system"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* Transform0 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("transform_0"), EFunctionInputType::FunctionInput_Vector3);
			UMaterialExpressionFunctionInput* Transform1 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("transform_1"), EFunctionInputType::FunctionInput_Vector3);
			UMaterialExpressionFunctionInput* Transform2 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("transform_2"), EFunctionInputType::FunctionInput_Vector3);

			UMaterialExpressionMaterialFunctionCall* Position = NewMaterialExpressionFunctionCall(Function, StateTextureCoordinate, {});

			UMaterialExpressionIf*        Space = NewMaterialExpressionIfEqual(Function, CoordinateSystem, (int)ETextureCoordinateSystem::World,
                                                                        (int)ECoordinateSpace::World, (int)ECoordinateSpace::Object);
			UMaterialExpressionNormalize* UDir  = NewMaterialExpressionNormalize(
                Function,
                NewMaterialExpressionFunctionCall(
                    Function, StateTransformVector,
                    {Space, (int)ECoordinateSpace::Internal,
                     NewMaterialExpressionFunctionCall(
                         Function, MakeFloat3,
                         {NewMaterialExpressionComponentMask(Function, Transform0, 1), NewMaterialExpressionComponentMask(Function, Transform1, 1),
                          NewMaterialExpressionComponentMask(Function, Transform2, 1)})}));
			UMaterialExpressionNormalize* VDir = NewMaterialExpressionNormalize(
			    Function,
			    NewMaterialExpressionFunctionCall(
			        Function, StateTransformVector,
			        {Space, (int)ECoordinateSpace::Internal,
			         NewMaterialExpressionFunctionCall(
			             Function, MakeFloat3,
			             {NewMaterialExpressionComponentMask(Function, Transform0, 2), NewMaterialExpressionComponentMask(Function, Transform1, 2),
			              NewMaterialExpressionComponentMask(Function, Transform2, 2)})}));

			UMaterialExpressionMaterialFunctionCall* Normal = NewMaterialExpressionFunctionCall(Function, StateNormal, {});
			UMaterialExpressionAbs* NormalDotUDir = NewMaterialExpressionAbs(Function, NewMaterialExpressionDotProduct(Function, Normal, UDir));
			UMaterialExpressionAbs* NormalDotVDir = NewMaterialExpressionAbs(Function, NewMaterialExpressionDotProduct(Function, Normal, VDir));

			UMaterialExpressionMaterialFunctionCall* ShadingNormal =
			    NewMaterialExpressionFunctionCall(Function, StateTransformVector, {0.0f, Space, Normal});
			UMaterialExpressionComponentMask* ShadingNormalX = NewMaterialExpressionComponentMask(Function, ShadingNormal, 1);
			UMaterialExpressionComponentMask* ShadingNormalY = NewMaterialExpressionComponentMask(Function, ShadingNormal, 2);
			UMaterialExpressionComponentMask* ShadingNormalZ = NewMaterialExpressionComponentMask(Function, ShadingNormal, 4);
			UMaterialExpressionSubtract*      YZ =
			    NewMaterialExpressionNegate(Function, NewMaterialExpressionMultiply(Function, ShadingNormalY, ShadingNormalZ));
			UMaterialExpressionIf* TangentVPreliminary = NewMaterialExpressionIfGreater(
			    Function, NewMaterialExpressionAbs(Function, ShadingNormalZ), 0.99999f,
			    NewMaterialExpressionNormalize(
			        Function,
			        NewMaterialExpressionFunctionCall(
			            Function, MakeFloat3,
			            {NewMaterialExpressionNegate(Function, NewMaterialExpressionMultiply(Function, ShadingNormalX, ShadingNormalY)),
			             NewMaterialExpressionOneMinus(Function, NewMaterialExpressionSquare(Function, ShadingNormalY)), YZ})),
			    NewMaterialExpressionNormalize(
			        Function,
			        NewMaterialExpressionFunctionCall(
			            Function, MakeFloat3,
			            {NewMaterialExpressionNegate(Function, NewMaterialExpressionMultiply(Function, ShadingNormalX, ShadingNormalZ)), YZ,
			             NewMaterialExpressionOneMinus(Function, NewMaterialExpressionSquare(Function, ShadingNormalZ))})));
			UMaterialExpressionNormalize* TangentUComplexPreliminary =
			    NewMaterialExpressionNormalize(Function, NewMaterialExpressionCrossProduct(Function, TangentVPreliminary, ShadingNormal));
			UMaterialExpressionNormalize* TangentUComplex = NewMaterialExpressionNormalize(
			    Function, NewMaterialExpressionFunctionCall(Function, StateTransformVector, {Space, 0.0f, TangentUComplexPreliminary}));
			UMaterialExpressionNormalize* TangentUSimplePreliminary =
			    NewMaterialExpressionNormalize(Function, NewMaterialExpressionCrossProduct(Function, VDir, Normal));
			UMaterialExpressionIf* TangentUSimple =
			    NewMaterialExpressionIfLess(Function, NewMaterialExpressionDotProduct(Function, TangentUSimplePreliminary, UDir), 0.0f,
			                                NewMaterialExpressionNegate(Function, TangentUSimplePreliminary), TangentUSimplePreliminary);
			UMaterialExpressionIf* TangentUUDirCheck =
			    NewMaterialExpressionIfGreater(Function, NormalDotUDir, 0.999f, TangentUComplex, TangentUSimple);
			UMaterialExpressionIf* TangentU = NewMaterialExpressionIfGreater(Function, NormalDotVDir, 0.999f, TangentUComplex, TangentUUDirCheck);

			UMaterialExpressionNormalize* TangentVComplex = NewMaterialExpressionNormalize(
			    Function, NewMaterialExpressionFunctionCall(Function, StateTransformVector, {Space, 0.0f, TangentVPreliminary}));
			UMaterialExpressionNormalize* TangentVSimplePreliminary =
			    NewMaterialExpressionNormalize(Function, NewMaterialExpressionCrossProduct(Function, Normal, TangentUSimplePreliminary));
			UMaterialExpressionIf* TangentVSimple =
			    NewMaterialExpressionIfLess(Function, NewMaterialExpressionDotProduct(Function, TangentVSimplePreliminary, VDir), 0.0f,
			                                NewMaterialExpressionNegate(Function, TangentVSimplePreliminary), TangentVSimplePreliminary);
			UMaterialExpressionIf* TangentVUDirCheck =
			    NewMaterialExpressionIfGreater(Function, NormalDotUDir, 0.999f, TangentVComplex, TangentVSimple);
			// As Unreal is left-handed, negate the resulting v-tangent !
			UMaterialExpressionSubtract* TangentV = NewMaterialExpressionNegate(
			    Function, NewMaterialExpressionIfGreater(Function, NormalDotVDir, 0.999f, TangentVComplex, TangentVUDirCheck));

			NewMaterialExpressionFunctionOutput(Function, TEXT("position"), Position);
			NewMaterialExpressionFunctionOutput(Function, TEXT("tangent_u"), TangentU);
			NewMaterialExpressionFunctionOutput(Function, TEXT("tangent_v"), TangentV);
		}

		void ImporterEvalChecker(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* MakeFloat3 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat3.MakeFloat3"));

			UMaterialExpressionFunctionInput* Position =
			    NewMaterialExpressionFunctionInput(Function, TEXT("position"), EFunctionInputType::FunctionInput_Vector3);
			UMaterialExpressionFunctionInput* CheckerPosition =
			    NewMaterialExpressionFunctionInput(Function, TEXT("checker_position"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* Blur =
			    NewMaterialExpressionFunctionInput(Function, TEXT("blur"), EFunctionInputType::FunctionInput_Scalar);

			UMaterialExpressionFrac* Tex = NewMaterialExpressionFrac(
			    Function, NewMaterialExpressionAdd(
			                  Function,
			                  NewMaterialExpressionSubtract(Function, Position, 0.25f),
			                  NewMaterialExpressionMultiply(
			                      Function,
			                      NewMaterialExpressionMin(Function, CheckerPosition, NewMaterialExpressionOneMinus(Function, CheckerPosition)),
			                      0.5f)));

			UMaterialExpressionFmod* InColor1 = NewMaterialExpressionFmod(
			    Function,
			    NewMaterialExpressionAdd(
			        Function,
			        {NewMaterialExpressionIfLess(Function, NewMaterialExpressionComponentMask(Function, Tex, 1), CheckerPosition, 1.0f, 0.0f),
			         NewMaterialExpressionIfLess(Function, NewMaterialExpressionComponentMask(Function, Tex, 2), CheckerPosition, 1.0f, 0.0f),
			         NewMaterialExpressionIfLess(Function, NewMaterialExpressionComponentMask(Function, Tex, 4), CheckerPosition, 1.0f, 0.0f)}),
			    2.0f);

			UMaterialExpressionMin* EdgeDist = NewMaterialExpressionMin(
			    Function,
			    Tex,
			    NewMaterialExpressionMin(
			        Function,
			        NewMaterialExpressionOneMinus(Function, Tex),
			        NewMaterialExpressionAbs(
			            Function,
			            NewMaterialExpressionSubtract(
			                Function,
			                Tex,
			                NewMaterialExpressionFunctionCall(Function, MakeFloat3, {CheckerPosition, CheckerPosition, CheckerPosition})))));
			UMaterialExpressionComponentMask* EdgeDistX = NewMaterialExpressionComponentMask(Function, EdgeDist, 1);
			UMaterialExpressionComponentMask* EdgeDistY = NewMaterialExpressionComponentMask(Function, EdgeDist, 2);
			UMaterialExpressionComponentMask* EdgeDistZ = NewMaterialExpressionComponentMask(Function, EdgeDist, 2);

			UMaterialExpressionIf* BlendAmountX = NewMaterialExpressionIfGreater(
			    Function, Blur, EdgeDistX, NewMaterialExpressionMultiply(Function, NewMaterialExpressionDivide(Function, EdgeDistX, Blur), 0.5f),
			    0.5f);
			UMaterialExpressionIf* BlendAmountY = NewMaterialExpressionIfGreater(
			    Function, Blur, EdgeDistY,
			    NewMaterialExpressionMultiply(Function, BlendAmountX, NewMaterialExpressionDivide(Function, EdgeDistY, Blur)), BlendAmountX);
			UMaterialExpressionIf* BlendAmount = NewMaterialExpressionIfGreater(
			    Function, Blur, EdgeDistZ,
			    NewMaterialExpressionMultiply(Function, BlendAmountY, NewMaterialExpressionDivide(Function, EdgeDistZ, Blur)), BlendAmountY);

			UMaterialExpressionIf* Result =
			    NewMaterialExpressionIfEqual(Function, InColor1, 1.0f, NewMaterialExpressionAdd(Function, 0.5f, BlendAmount),
			                                 NewMaterialExpressionSubtract(Function, 0.5f, BlendAmount));

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Result);
		}

		void ImporterEvalTileFunction(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* MakeFloat2 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat2.MakeFloat2"));
			UMaterialFunction* MakeFloat4 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat4.MakeFloat4"));
			UMaterialFunction* PerlinNoise = LoadFunction(TEXT("mdlimporter_perlin_noise"));

			UMaterialExpressionFunctionInput* Tex =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tex"), EFunctionInputType::FunctionInput_Vector2);
			UMaterialExpressionFunctionInput* TileColor =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tile_color"), EFunctionInputType::FunctionInput_Vector3);
			UMaterialExpressionFunctionInput* GroutColor =
			    NewMaterialExpressionFunctionInput(Function, TEXT("grout_color"), EFunctionInputType::FunctionInput_Vector3);
			UMaterialExpressionFunctionInput* TileBrightnessVariation =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tile_brightness_variation"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* TileHoleAmount =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tile_hole_amount"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* Seed =
			    NewMaterialExpressionFunctionInput(Function, TEXT("seed"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* NumberOfRows =
			    NewMaterialExpressionFunctionInput(Function, TEXT("number_of_rows"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* NumberOfColumns =
			    NewMaterialExpressionFunctionInput(Function, TEXT("number_of_columns"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* OddRowOffset =
			    NewMaterialExpressionFunctionInput(Function, TEXT("odd_row_offset"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* RandomRowOffset =
			    NewMaterialExpressionFunctionInput(Function, TEXT("random_row_offset"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* TileGroutWidth =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tile_grout_width"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* TileGroutHeight =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tile_grout_height"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* TileGroutRoughness =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tile_grout_roughness"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* SpecialColumnIndex =
			    NewMaterialExpressionFunctionInput(Function, TEXT("special_column_index"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* SpecialRowIndex =
			    NewMaterialExpressionFunctionInput(Function, TEXT("special_row_index"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* SpecialColumnHeightFactor =
			    NewMaterialExpressionFunctionInput(Function, TEXT("special_column_height_factor"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* SpecialRowWidthFactor =
			    NewMaterialExpressionFunctionInput(Function, TEXT("special_row_width_factor"), EFunctionInputType::FunctionInput_Scalar);

			UMaterialExpressionComponentMask*        TexX = NewMaterialExpressionComponentMask(Function, Tex, 1);
			UMaterialExpressionComponentMask*        TexY = NewMaterialExpressionComponentMask(Function, Tex, 2);
			UMaterialExpressionMaterialFunctionCall* XYCR = NewMaterialExpressionFunctionCall(
			    Function, MakeFloat4,
			    {NewMaterialExpressionMultiply(Function, NewMaterialExpressionSubtract(Function, TexX, NewMaterialExpressionFloor(Function, TexX)),
			                                   NumberOfColumns),
			     NewMaterialExpressionMultiply(Function, NewMaterialExpressionSubtract(Function, TexY, NewMaterialExpressionFloor(Function, TexY)),
			                                   NumberOfRows),
			     NumberOfColumns, NumberOfRows});

			UMaterialExpressionIf* ColIndexCheck = NewMaterialExpressionIfEqual(
			    Function,
			    NewMaterialExpressionFmod(
			        Function,
			        NewMaterialExpressionAdd(Function, NewMaterialExpressionFloor(Function, NewMaterialExpressionComponentMask(Function, XYCR, 1)),
			                                 1.0f),
			        SpecialColumnIndex),
			    0.0f,
			    NewMaterialExpressionMultiply(
			        Function, XYCR,
			        NewMaterialExpressionFunctionCall(
			            Function, MakeFloat4, {SpecialColumnHeightFactor, SpecialRowWidthFactor, SpecialColumnHeightFactor, SpecialRowWidthFactor})),
			    XYCR);
			UMaterialExpressionIf* RowIndexCheck = NewMaterialExpressionIfEqual(
			    Function,
			    NewMaterialExpressionFmod(
			        Function,
			        NewMaterialExpressionAdd(Function, NewMaterialExpressionFloor(Function, NewMaterialExpressionComponentMask(Function, XYCR, 2)),
			                                 1.0f),
			        SpecialRowIndex),
			    0.0f,
			    ColIndexCheck,
			    XYCR);
			UMaterialExpressionComponentMask* X0       = NewMaterialExpressionComponentMask(Function, RowIndexCheck, 1);
			UMaterialExpressionComponentMask* Y0       = NewMaterialExpressionComponentMask(Function, RowIndexCheck, 2);
			UMaterialExpressionFloor*         RowIndex = NewMaterialExpressionFloor(Function, Y0);
			UMaterialExpressionIf* X1 = NewMaterialExpressionIfEqual(Function, NewMaterialExpressionFmod(Function, RowIndex, 2.0f), 0.0f, X0,
			                                                         NewMaterialExpressionAdd(Function, X0, OddRowOffset));
			UMaterialExpressionIf* X2 = NewMaterialExpressionIfGreater(
			    Function, RandomRowOffset, 0.0f,
			    NewMaterialExpressionAdd(
			        Function,
			        X1,
			        NewMaterialExpressionMultiply(
			            Function, RandomRowOffset,
			            NewMaterialExpressionFunctionCall(Function, PerlinNoise, {NewMaterialExpressionMultiply(Function, RowIndex, Seed)}))),
			    X1);
			UMaterialExpressionFloor*    ColIndex = NewMaterialExpressionFloor(Function, X2);
			UMaterialExpressionSubtract* X        = NewMaterialExpressionSubtract(Function, X2, ColIndex);
			UMaterialExpressionSubtract* Y        = NewMaterialExpressionSubtract(Function, Y0, RowIndex);

			UMaterialExpressionComponentMask* NumColumns = NewMaterialExpressionComponentMask(Function, RowIndexCheck, 4);
			UMaterialExpressionComponentMask* NumRows    = NewMaterialExpressionComponentMask(Function, RowIndexCheck, 8);
			UMaterialExpressionMultiply*      NoiseScaleY =
			    NewMaterialExpressionMultiply(Function, 10.0f, NewMaterialExpressionAdd(Function, NumRows, NumColumns));
			UMaterialExpressionMultiply* NoiseScaleX =
			    NewMaterialExpressionMultiply(Function, NewMaterialExpressionDivide(Function, NumColumns, NumRows), NoiseScaleY);

			UMaterialExpressionMultiply* NoisePos0 =
			    NewMaterialExpressionMultiply(Function, NewMaterialExpressionFunctionCall(Function, MakeFloat2, {ColIndex, RowIndex}), Seed);
			UMaterialExpressionMultiply* NoisePos1 =
			    NewMaterialExpressionMultiply(Function, Tex, NewMaterialExpressionFunctionCall(Function, MakeFloat2, {NoiseScaleX, NoiseScaleY}));
			UMaterialExpressionMultiply* NoisePos2 =
			    NewMaterialExpressionMultiply(Function, Tex, NewMaterialExpressionFunctionCall(Function, MakeFloat2, {NoiseScaleY, NoiseScaleX}));
			UMaterialExpressionMultiply* NoisePos3 = NewMaterialExpressionMultiply(Function, NoisePos0, 0.5f);

			UMaterialExpressionMaterialFunctionCall* Noise0 =
			    NewMaterialExpressionFunctionCall(Function, PerlinNoise, {NewMaterialExpressionAppendVector(Function, NoisePos0, 0.0f)});
			UMaterialExpressionMaterialFunctionCall* Noise1 =
			    NewMaterialExpressionFunctionCall(Function, PerlinNoise, {NewMaterialExpressionAppendVector(Function, NoisePos1, 0.0f)});
			UMaterialExpressionMaterialFunctionCall* Noise2 =
			    NewMaterialExpressionFunctionCall(Function, PerlinNoise, {NewMaterialExpressionAppendVector(Function, NoisePos2, 0.0f)});
			UMaterialExpressionMaterialFunctionCall* Noise3 =
			    NewMaterialExpressionFunctionCall(Function, PerlinNoise, {NewMaterialExpressionAppendVector(Function, NoisePos3, 0.0f)});

			UMaterialExpressionMultiply* GroutWidthBase      = NewMaterialExpressionMultiply(Function, TileGroutWidth, NumColumns);
			UMaterialExpressionMultiply* GroutHeightBase     = NewMaterialExpressionMultiply(Function, TileGroutHeight, NumRows);
			UMaterialExpressionIf*       GroutRoughnessCheck = NewMaterialExpressionIfGreater(
                Function, TileGroutRoughness, 0.0f,
                NewMaterialExpressionFunctionCall(
                    Function, MakeFloat2,
                    {NewMaterialExpressionAdd(
                         Function,
                         GroutWidthBase,
                         NewMaterialExpressionMultiply(
                             Function, {NewMaterialExpressionDivide(Function, NumColumns, NumberOfColumns), TileGroutRoughness, Noise1})),
                     NewMaterialExpressionAdd(Function,
                                              GroutHeightBase,
                                              NewMaterialExpressionMultiply(
                                                  Function,
                                                  {NewMaterialExpressionDivide(Function, NewMaterialExpressionSquare(Function, NumRows),
                                                                               NewMaterialExpressionMultiply(Function, NumColumns, NumberOfRows)),
                                                   TileGroutRoughness, Noise2}))}),
                NewMaterialExpressionFunctionCall(Function, MakeFloat2, {GroutWidthBase, GroutHeightBase}));
			UMaterialExpressionComponentMask* GroutWidth  = NewMaterialExpressionComponentMask(Function, GroutRoughnessCheck, 1);
			UMaterialExpressionComponentMask* GroutHeight = NewMaterialExpressionComponentMask(Function, GroutRoughnessCheck, 2);

			UMaterialExpressionAppendVector* GroutReturn = NewMaterialExpressionAppendVector(Function, GroutColor, 0.0f);

			UMaterialExpressionIf* TileBrightnessVariationCheck = NewMaterialExpressionIfGreater(
			    Function, TileBrightnessVariation, 0.0f,
			    NewMaterialExpressionSaturate(
			        Function,
			        NewMaterialExpressionAdd(Function, TileColor,
			                                 NewMaterialExpressionMultiply(Function, {TileColor, TileBrightnessVariation, Noise3}))),
			    TileColor);
			UMaterialExpressionAppendVector* TileReturn = NewMaterialExpressionAppendVector(Function, TileBrightnessVariationCheck, 1.0f);

			UMaterialExpressionIf* InTileCheck4 =
			    NewMaterialExpressionIfGreater(Function, Y, NewMaterialExpressionOneMinus(Function, GroutHeight), GroutReturn, TileReturn);
			UMaterialExpressionIf* InTileCheck3 = NewMaterialExpressionIfLess(Function, Y, GroutHeight, GroutReturn, InTileCheck4);
			UMaterialExpressionIf* InTileCheck2 =
			    NewMaterialExpressionIfGreater(Function, X, NewMaterialExpressionOneMinus(Function, GroutWidth), GroutReturn, InTileCheck3);
			UMaterialExpressionIf* InTileCheck1 = NewMaterialExpressionIfLess(Function, X, GroutWidth, GroutReturn, InTileCheck2);

			UMaterialExpressionIf* TileHoleAmountCheck = NewMaterialExpressionIfGreater(
			    Function, TileHoleAmount,
			    NewMaterialExpressionIfLess(Function, Noise0, 0.0f, NewMaterialExpressionAdd(Function, Noise0, 1.0f), Noise0), GroutReturn,
			    InTileCheck1);
			UMaterialExpressionIf* TileHoleAmountPositiveCheck =
			    NewMaterialExpressionIfGreater(Function, TileHoleAmount, 0.0f, TileHoleAmountCheck, InTileCheck1);

			UMaterialExpressionComponentMask* IsInTile    = NewMaterialExpressionComponentMask(Function, TileHoleAmountPositiveCheck, 8);
			UMaterialExpressionComponentMask* ColorResult = NewMaterialExpressionComponentMask(Function, TileHoleAmountPositiveCheck, 7);

			NewMaterialExpressionFunctionOutput(Function, TEXT("is_in_tile"), IsInTile);
			NewMaterialExpressionFunctionOutput(Function, TEXT("color_result"), ColorResult);
		}

		void ImporterFlowNoise(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* ImporterGradFlow    = LoadFunction(TEXT("mdlimporter_grad_flow"));
			UMaterialFunction* ImporterPermuteFlow = LoadFunction(TEXT("mdlimporter_permute_flow"));
			UMaterialFunction* MakeFloat2 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat2.MakeFloat2"));
			UMaterialFunction* MakeFloat3 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat3.MakeFloat3"));

			UMaterialExpressionFunctionInput* P = NewMaterialExpressionFunctionInput(Function, TEXT("p"), EFunctionInputType::FunctionInput_Vector2);
			UMaterialExpressionFunctionInput* Rot =
			    NewMaterialExpressionFunctionInput(Function, TEXT("rot"), EFunctionInputType::FunctionInput_Scalar);

			float                     F0 = 0.5f - sqrt(3.0f) / 6.0f;
			float                     F1 = -0.5f - sqrt(3.0f) / 6.0f;
			UMaterialExpressionFloor* Pi = NewMaterialExpressionFloor(
			    Function,
			    NewMaterialExpressionAdd(Function,
			                             P,
			                             NewMaterialExpressionMultiply(Function,
			                                                           NewMaterialExpressionAdd(Function,
			                                                                                    NewMaterialExpressionComponentMask(Function, P, 1),
			                                                                                    NewMaterialExpressionComponentMask(Function, P, 2)),
			                                                           0.5f * sqrt(3.0f) - 0.5f)));
			UMaterialExpressionComponentMask* PiX = NewMaterialExpressionComponentMask(Function, Pi, 1);
			UMaterialExpressionComponentMask* PiY = NewMaterialExpressionComponentMask(Function, Pi, 2);
			UMaterialExpressionAdd*           V0 =
			    NewMaterialExpressionAdd(Function,
			                             NewMaterialExpressionSubtract(Function, P, Pi),
			                             NewMaterialExpressionMultiply(Function, NewMaterialExpressionAdd(Function, PiX, PiY), F0));
			UMaterialExpressionComponentMask* V0X = NewMaterialExpressionComponentMask(Function, V0, 1);
			UMaterialExpressionComponentMask* V0Y = NewMaterialExpressionComponentMask(Function, V0, 2);
			UMaterialExpressionAdd*           V1  = NewMaterialExpressionAdd(
                Function,
                V0,
                NewMaterialExpressionIfLess(Function, V0X, V0Y, NewMaterialExpressionFunctionCall(Function, MakeFloat2, {F0, F1}),
                                            NewMaterialExpressionFunctionCall(Function, MakeFloat2, {F1, F0})));
			UMaterialExpressionSubtract* V2 = NewMaterialExpressionSubtract(Function, V0, 1.0f / sqrt(3.0f));
			UMaterialExpressionMax*      T  = NewMaterialExpressionMax(
                Function,
                NewMaterialExpressionSubtract(Function,
                                              {0.5f, 0.5f, 0.5f},
                                              NewMaterialExpressionFunctionCall(Function, MakeFloat3,
                                                                                {NewMaterialExpressionDotProduct(Function, V0, V0),
                                                                                 NewMaterialExpressionDotProduct(Function, V1, V1),
                                                                                 NewMaterialExpressionDotProduct(Function, V2, V2)})),
                {0.0f, 0.0f, 0.0f});
			UMaterialExpressionAdd* TmpP0 =
			    NewMaterialExpressionAdd(Function, NewMaterialExpressionFunctionCall(Function, ImporterPermuteFlow, {PiX}), PiY);
			UMaterialExpressionAdd* TmpP2 = NewMaterialExpressionAdd(
			    Function, NewMaterialExpressionFunctionCall(Function, ImporterPermuteFlow, {NewMaterialExpressionAdd(Function, PiX, 1.0f)}), PiY);
			UMaterialExpressionAdd* TmpP1 = NewMaterialExpressionAdd(Function, NewMaterialExpressionIfLess(Function, V0X, V0Y, TmpP0, TmpP2),
			                                                         NewMaterialExpressionIfLess(Function, V0X, V0Y, 1.0f, 0.0f));
			UMaterialExpressionMaterialFunctionCall* G0 = NewMaterialExpressionFunctionCall(
			    Function, ImporterGradFlow, {NewMaterialExpressionFunctionCall(Function, ImporterPermuteFlow, {TmpP0}), Rot});
			UMaterialExpressionMaterialFunctionCall* G1 = NewMaterialExpressionFunctionCall(
			    Function, ImporterGradFlow, {NewMaterialExpressionFunctionCall(Function, ImporterPermuteFlow, {TmpP1}), Rot});
			UMaterialExpressionMaterialFunctionCall* G2 = NewMaterialExpressionFunctionCall(
			    Function, ImporterGradFlow, {NewMaterialExpressionFunctionCall(Function, ImporterPermuteFlow, {TmpP2}), Rot});
			UMaterialExpressionMaterialFunctionCall* GV = NewMaterialExpressionFunctionCall(
			    Function, MakeFloat3,
			    {NewMaterialExpressionDotProduct(Function, G0, V0), NewMaterialExpressionDotProduct(Function, G1, V1),
			     NewMaterialExpressionDotProduct(Function, G2, V2)});
			UMaterialExpressionMultiply* T2     = NewMaterialExpressionSquare(Function, T);
			UMaterialExpressionMultiply* T4     = NewMaterialExpressionSquare(Function, T2);
			UMaterialExpressionMultiply* Result = NewMaterialExpressionMultiply(Function, 40.0f, NewMaterialExpressionDotProduct(Function, T4, GV));

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Result);
		}

		void ImporterGradFlow(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* MakeFloat2 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat2.MakeFloat2"));

			UMaterialExpressionFunctionInput* P = NewMaterialExpressionFunctionInput(Function, TEXT("p"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* Rot =
			    NewMaterialExpressionFunctionInput(Function, TEXT("rot"), EFunctionInputType::FunctionInput_Scalar);

			UMaterialExpressionAdd*      Magic = NewMaterialExpressionAdd(Function, NewMaterialExpressionMultiply(Function, P, 1.0f / 41.0f), Rot);
			UMaterialExpressionSubtract* U     = NewMaterialExpressionSubtract(
                Function,
                NewMaterialExpressionMultiply(Function, NewMaterialExpressionSubtract(Function, Magic, NewMaterialExpressionFloor(Function, Magic)),
                                              4.0f),
                2.0f);
			UMaterialExpressionMaterialFunctionCall* Result = NewMaterialExpressionFunctionCall(
			    Function, MakeFloat2,
			    {NewMaterialExpressionSubtract(Function, NewMaterialExpressionAbs(Function, U), 1.0f),
			     NewMaterialExpressionSubtract(
			         Function,
			         NewMaterialExpressionAbs(Function,
			                                  NewMaterialExpressionSubtract(
			                                      Function, NewMaterialExpressionAbs(Function, NewMaterialExpressionAdd(Function, U, 1.0f)), 2.0f)),
			         1.0f)});

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Result);
		}

		void ImporterHSVToRGB(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* MakeFloat3 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat3.MakeFloat3"));

			UMaterialExpressionFunctionInput* HSV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("hsv"), EFunctionInputType::FunctionInput_Vector3);

			UMaterialExpressionComponentMask* HSVX = NewMaterialExpressionComponentMask(Function, HSV, 1);
			UMaterialExpressionComponentMask* HSVY = NewMaterialExpressionComponentMask(Function, HSV, 2);
			UMaterialExpressionComponentMask* HSVZ = NewMaterialExpressionComponentMask(Function, HSV, 4);

			UMaterialExpressionIf* HPrime =
			    NewMaterialExpressionIfEqual(Function, HSVX, 1.0f, 0.0f, NewMaterialExpressionMultiply(Function, HSVX, 6.0f));
			UMaterialExpressionFloor*    HFloor = NewMaterialExpressionFloor(Function, HPrime);
			UMaterialExpressionSubtract* F      = NewMaterialExpressionSubtract(Function, HPrime, HFloor);
			UMaterialExpressionMultiply* ZY     = NewMaterialExpressionMultiply(Function, HSVZ, HSVY);
			UMaterialExpressionSubtract* A      = NewMaterialExpressionSubtract(Function, HSVZ, ZY);
			UMaterialExpressionSubtract* B      = NewMaterialExpressionSubtract(Function, HSVZ, NewMaterialExpressionMultiply(Function, ZY, F));
			UMaterialExpressionAdd*      C      = NewMaterialExpressionAdd(Function, A, NewMaterialExpressionMultiply(Function, ZY, F));

			UMaterialExpressionIf* Check1 =
			    NewMaterialExpressionIf(Function, HFloor, 1.0f, NewMaterialExpressionFunctionCall(Function, MakeFloat3, {HSVZ, C, A}),  // 0
			                            NewMaterialExpressionFunctionCall(Function, MakeFloat3, {B, HSVZ, A}),                          // 1
			                            NewMaterialExpressionFunctionCall(Function, MakeFloat3, {A, HSVZ, C}));                         // 2
			UMaterialExpressionIf* Check4 =
			    NewMaterialExpressionIf(Function, HFloor, 4.0f, NewMaterialExpressionFunctionCall(Function, MakeFloat3, {A, B, HSVZ}),  // 3
			                            NewMaterialExpressionFunctionCall(Function, MakeFloat3, {C, A, HSVZ}),                          // 4
			                            NewMaterialExpressionFunctionCall(Function, MakeFloat3, {HSVZ, A, B}));                         // 5

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), NewMaterialExpressionIf(Function, HFloor, 2.5f, Check1, {}, Check4));
		}

		void ImporterMonoMode(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* MathAverage   = LoadFunction(TEXT("mdl_math_average"));
			UMaterialFunction* MathLuminance = LoadFunction(TEXT("mdl_math_luminance"));
			UMaterialFunction* MathMaxValue  = LoadFunction(TEXT("mdl_math_max_value"));

			UMaterialExpressionFunctionInput* Sample =
			    NewMaterialExpressionFunctionInput(Function, TEXT("sample"), EFunctionInputType::FunctionInput_Vector4);
			UMaterialExpressionFunctionInput* Mode =
			    NewMaterialExpressionFunctionInput(Function, TEXT("mode"), EFunctionInputType::FunctionInput_Scalar, (int)EMonoMode::Average);

			UMaterialExpressionComponentMask* Color = NewMaterialExpressionComponentMask(Function, Sample, 7);
			UMaterialExpressionComponentMask* Alpha = NewMaterialExpressionComponentMask(Function, Sample, 8);

			UMaterialExpressionIf* Result = NewMaterialExpressionSwitch(Function, Mode,
			                                                            {Alpha, NewMaterialExpressionFunctionCall(Function, MathAverage, {Color}),
			                                                             NewMaterialExpressionFunctionCall(Function, MathLuminance, {Color}),
			                                                             NewMaterialExpressionFunctionCall(Function, MathMaxValue, {Color})});

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Result);
		}

		/*static*/ void ImporterMiNoise(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialExpressionFunctionInput* XYZ =
			    NewMaterialExpressionFunctionInput(Function, TEXT("xyz"), EFunctionInputType::FunctionInput_Vector3);

			UMaterialExpressionVectorNoise*   Noise = NewMaterialExpressionVectorNoise(Function, XYZ, VNF_GradientALU, 4);
			UMaterialExpressionComponentMask* Grad  = NewMaterialExpressionComponentMask(Function, Noise, 7);
			UMaterialExpressionComponentMask* Value = NewMaterialExpressionComponentMask(Function, Noise, 8);

			NewMaterialExpressionFunctionOutput(Function, TEXT("grad"), Grad);
			NewMaterialExpressionFunctionOutput(Function, TEXT("value"), Value);
		}

		/*static*/ void ImporterPermuteFlow(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialExpressionFunctionInput* X = NewMaterialExpressionFunctionInput(Function, TEXT("x"), EFunctionInputType::FunctionInput_Scalar);

			UMaterialExpressionFmod* Result = NewMaterialExpressionFmod(
			    Function, NewMaterialExpressionAdd(Function, NewMaterialExpressionMultiply(Function, {X, X, 34.0f}), X), 289.0f);

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Result);
		}

		/*static*/ void ImporterRefract(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);

			// The IncidentVector and the NormalVector are supposed to be normalized !
			UMaterialExpressionFunctionInput* IncidentVector =
			    NewMaterialExpressionFunctionInput(Function, TEXT("incident_vector"), EFunctionInputType::FunctionInput_Vector3);
			UMaterialExpressionFunctionInput* NormalVector =
			    NewMaterialExpressionFunctionInput(Function, TEXT("normal_vector"), EFunctionInputType::FunctionInput_Vector3);
			UMaterialExpressionFunctionInput* Eta =
			    NewMaterialExpressionFunctionInput(Function, TEXT("eta"), EFunctionInputType::FunctionInput_Scalar);

			UMaterialExpressionDotProduct* DotNI = NewMaterialExpressionDotProduct(Function, NormalVector, IncidentVector);
			UMaterialExpressionOneMinus*   K     = NewMaterialExpressionOneMinus(
                Function, NewMaterialExpressionMultiply(Function, {Eta, Eta, NewMaterialExpressionOneMinus(Function, DotNI), DotNI}));
			UMaterialExpressionIf* Result = NewMaterialExpressionIfLess(
			    Function, K, 0.0f, NewMaterialExpressionConstant(Function, 0.0f, 0.0f, 0.0f),
			    NewMaterialExpressionSubtract(
			        Function,
			        NewMaterialExpressionMultiply(Function, Eta, IncidentVector),
			        NewMaterialExpressionMultiply(Function,
			                                      NewMaterialExpressionAdd(Function, NewMaterialExpressionMultiply(Function, Eta, DotNI),
			                                                               NewMaterialExpressionSquareRoot(Function, K)),
			                                      NormalVector)));

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Result);
		}

		/*static*/ void ImporterSelectBSDF(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialExpressionFunctionInput* Value =
			    NewMaterialExpressionFunctionInput(Function, TEXT("Value"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* True =
			    NewMaterialExpressionFunctionInput(Function, TEXT("True"), EFunctionInputType::FunctionInput_MaterialAttributes);
			UMaterialExpressionFunctionInput* False =
			    NewMaterialExpressionFunctionInput(Function, TEXT("False"), EFunctionInputType::FunctionInput_MaterialAttributes);
			UMaterialExpressionFunctionInput* Normal =
			    NewMaterialExpressionFunctionInput(Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3);

			UMaterialExpressionBreakMaterialAttributes* BreakTrue  = NewMaterialExpressionBreakMaterialAttributes(Function, True);
			UMaterialExpressionBreakMaterialAttributes* BreakFalse = NewMaterialExpressionBreakMaterialAttributes(Function, False);

			UMaterialExpressionMakeMaterialAttributes* BSDF =
			    NewMaterialExpressionMakeMaterialAttributes(Function,
			                                                NewMaterialExpressionIfEqual(Function, Value, 0.0f, {BreakFalse, 0}, {BreakTrue, 0}),
			                                                NewMaterialExpressionIfEqual(Function, Value, 0.0f, {BreakFalse, 1}, {BreakTrue, 1}),
			                                                NewMaterialExpressionIfEqual(Function, Value, 0.0f, {BreakFalse, 2}, {BreakTrue, 2}),
			                                                NewMaterialExpressionIfEqual(Function, Value, 0.0f, {BreakFalse, 3}, {BreakTrue, 3}),
			                                                NewMaterialExpressionIfEqual(Function, Value, 0.0f, {BreakFalse, 4}, {BreakTrue, 4}),
			                                                NewMaterialExpressionIfEqual(Function, Value, 0.0f, {BreakFalse, 5}, {BreakTrue, 5}),
			                                                NewMaterialExpressionIfEqual(Function, Value, 0.0f, {BreakFalse, 6}, {BreakTrue, 6}),
			                                                Normal,
			                                                NewMaterialExpressionIfEqual(Function, Value, 0.0f, {BreakFalse, 8}, {BreakTrue, 8}),
			                                                NewMaterialExpressionIfEqual(Function, Value, 0.0f, {BreakFalse, 9}, {BreakTrue, 9}),
			                                                NewMaterialExpressionIfEqual(Function, Value, 0.0f, {BreakFalse, 10}, {BreakTrue, 10}),
			                                                NewMaterialExpressionIfEqual(Function, Value, 0.0f, {BreakFalse, 11}, {BreakTrue, 11}),
			                                                NewMaterialExpressionIfEqual(Function, Value, 0.0f, {BreakFalse, 12}, {BreakTrue, 12}),
			                                                NewMaterialExpressionIfEqual(Function, Value, 0.0f, {BreakFalse, 13}, {BreakTrue, 13}),
			                                                NewMaterialExpressionIfEqual(Function, Value, 0.0f, {BreakFalse, 14}, {BreakTrue, 14}),
			                                                NewMaterialExpressionIfEqual(Function, Value, 0.0f, {BreakFalse, 15}, {BreakTrue, 15}),
			                                                NewMaterialExpressionIfEqual(Function, Value, 0.0f, {BreakFalse, 16}, {BreakTrue, 16}),
			                                                NewMaterialExpressionIfEqual(Function, Value, 0.0f, {BreakFalse, 17}, {BreakTrue, 17}),
			                                                NewMaterialExpressionIfEqual(Function, Value, 0.0f, {BreakFalse, 18}, {BreakTrue, 18}),
			                                                NewMaterialExpressionIfEqual(Function, Value, 0.0f, {BreakFalse, 19}, {BreakTrue, 19}),
			                                                NewMaterialExpressionIfEqual(Function, Value, 0.0f, {BreakFalse, 20}, {BreakTrue, 20}),
			                                                NewMaterialExpressionIfEqual(Function, Value, 0.0f, {BreakFalse, 21}, {BreakTrue, 21}),
			                                                NewMaterialExpressionIfEqual(Function, Value, 0.0f, {BreakFalse, 22}, {BreakTrue, 22}));

			NewMaterialExpressionFunctionOutput(Function, TEXT("bsdf"), BSDF);
		}

		void ImporterSetClipMask(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* ClipMask = LoadFunction(TEXT("/Game/Materials/ClipSphere"), TEXT("ClipMask.ClipMask"));

			UMaterialExpressionFunctionInput* Material =
			    NewMaterialExpressionFunctionInput(Function, TEXT("material"), EFunctionInputType::FunctionInput_MaterialAttributes);

			UMaterialExpressionBreakMaterialAttributes* Break  = NewMaterialExpressionBreakMaterialAttributes(Function, Material);
			UMaterialExpressionMaterialFunctionCall*    Factor = NewMaterialExpressionFunctionCall(Function, ClipMask, {});

			UMaterialExpressionMultiply* Opacity     = NewMaterialExpressionMultiply(Function, {Break, 5}, Factor);
			UMaterialExpressionMultiply* OpacityMask = NewMaterialExpressionMultiply(Function, {Break, 6}, Factor);

			UMaterialExpressionMakeMaterialAttributes* BSDF = NewMaterialExpressionMakeMaterialAttributes(Function,
			                                                                                              {Break, 0},    // BaseColor
			                                                                                              {Break, 1},    // Metallic
			                                                                                              {Break, 2},    // Specular
			                                                                                              {Break, 3},    // Roughness
			                                                                                              {Break, 4},    // EmissiveColor
			                                                                                              Opacity,       // Opacity
			                                                                                              OpacityMask,   // OpacityMask
			                                                                                              {Break, 7},    // Normal
			                                                                                              {Break, 8},    // WorldPositionOffset
			                                                                                              {Break, 9},   // SubsurfaceColor
			                                                                                              {Break, 10},   // ClearCoat
			                                                                                              {Break, 11},   // ClearCoatRoughness
			                                                                                              {Break, 12},   // AmbientOcclusion
			                                                                                              {Break, 13},   // Refraction
			                                                                                              {Break, 14},   // CustomizedUVs0
			                                                                                              {Break, 15},   // CustomizedUVs1
			                                                                                              {Break, 16},   // CustomizedUVs2
			                                                                                              {Break, 17},   // CustomizedUVs3
			                                                                                              {Break, 18},   // CustomizedUVs4
			                                                                                              {Break, 19},   // CustomizedUVs5
			                                                                                              {Break, 20},   // CustomizedUVs6
			                                                                                              {Break, 21},   // CustomizedUVs7
			                                                                                              {Break, 22});  // PixelDepthOffset

			NewMaterialExpressionFunctionOutput(Function, TEXT("bsdf"), BSDF);
		}

		/*static*/ void ImporterSetRefraction(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);

			UMaterialExpressionFunctionInput* Material =
			    NewMaterialExpressionFunctionInput(Function, TEXT("material"), EFunctionInputType::FunctionInput_MaterialAttributes);
			UMaterialExpressionFunctionInput* IOR =
			    NewMaterialExpressionFunctionInput(Function, TEXT("IOR"), EFunctionInputType::FunctionInput_Scalar);

			UMaterialExpressionLinearInterpolate* Refraction = NewMaterialExpressionLinearInterpolate(
			    Function, NewMaterialExpressionConstant(Function, 1.0f), IOR, NewMaterialExpressionFresnel(Function, {}, {}, {}));

			UMaterialExpressionBreakMaterialAttributes* MaterialBreak = NewMaterialExpressionBreakMaterialAttributes(Function, Material);
			UMaterialExpressionMakeMaterialAttributes*  BSDF =
			    NewMaterialExpressionMakeMaterialAttributes(Function,
			                                                {MaterialBreak, 0},    // BaseColor
			                                                {MaterialBreak, 1},    // Metallic
			                                                {MaterialBreak, 2},    // Specular
			                                                {MaterialBreak, 3},    // Roughness
			                                                {MaterialBreak, 4},    // EmissiveColor
			                                                {MaterialBreak, 5},    // Opacity,
			                                                {MaterialBreak, 6},    // OpacityMask
			                                                {MaterialBreak, 7},    // Normal
			                                                {MaterialBreak, 8},    // WorldPositionOffset,
			                                                {MaterialBreak, 9},   // SubsurfaceColor,
			                                                {MaterialBreak, 10},   // ClearCoat
			                                                {MaterialBreak, 11},   // ClearCoatRoughness
			                                                {MaterialBreak, 12},   // AmbientOcclusion,
			                                                Refraction,            // Refraction
			                                                {MaterialBreak, 14},   // CustomizedUVs0
			                                                {MaterialBreak, 15},   // CustomizedUVs1
			                                                {MaterialBreak, 16},   // CustomizedUVs2
			                                                {MaterialBreak, 17},   // CustomizedUVs3
			                                                {MaterialBreak, 18},   // CustomizedUVs4
			                                                {MaterialBreak, 19},   // CustomizedUVs5
			                                                {MaterialBreak, 20},   // CustomizedUVs6
			                                                {MaterialBreak, 21},   // CustomizedUVs7
			                                                {MaterialBreak, 22});  // PixelDepthOffset

			NewMaterialExpressionFunctionOutput(Function, TEXT("bsdf"), BSDF);
		}

		/*static*/ void ImporterSetSubsurfaceColor(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);

			UMaterialExpressionFunctionInput* Material =
			    NewMaterialExpressionFunctionInput(Function, TEXT("material"), EFunctionInputType::FunctionInput_MaterialAttributes);
			UMaterialExpressionFunctionInput* SubsurfaceColor =
			    NewMaterialExpressionFunctionInput(Function, TEXT("subsurface_color"), EFunctionInputType::FunctionInput_Vector3);

			UMaterialExpressionBreakMaterialAttributes* MaterialBreak = NewMaterialExpressionBreakMaterialAttributes(Function, Material);

			UMaterialExpressionMakeMaterialAttributes* BSDF =
			    NewMaterialExpressionMakeMaterialAttributes(Function,
			                                                {MaterialBreak, 0},    // BaseColor
			                                                {MaterialBreak, 1},    // Metallic
			                                                {MaterialBreak, 2},    // Specular
			                                                {MaterialBreak, 3},    // Roughness
			                                                {MaterialBreak, 4},    // EmissiveColor
			                                                {MaterialBreak, 5},    // Opacity,
			                                                {MaterialBreak, 6},    // OpacityMask
			                                                {MaterialBreak, 7},    // Normal
			                                                {MaterialBreak, 8},    // WorldPositionOffset,
			                                                SubsurfaceColor,       // SubsurfaceColor,
			                                                {MaterialBreak, 10},   // ClearCoat
			                                                {MaterialBreak, 11},   // ClearCoatRoughness
			                                                {MaterialBreak, 12},   // AmbientOcclusion,
			                                                {MaterialBreak, 13},   // Refraction
			                                                {MaterialBreak, 14},   // CustomizedUVs0
			                                                {MaterialBreak, 15},   // CustomizedUVs1
			                                                {MaterialBreak, 16},   // CustomizedUVs2
			                                                {MaterialBreak, 17},   // CustomizedUVs3
			                                                {MaterialBreak, 18},   // CustomizedUVs4
			                                                {MaterialBreak, 19},   // CustomizedUVs5
			                                                {MaterialBreak, 20},   // CustomizedUVs6
			                                                {MaterialBreak, 21},   // CustomizedUVs7
			                                                {MaterialBreak, 22});  // PixelDepthOffset

			NewMaterialExpressionFunctionOutput(Function, TEXT("bsdf"), BSDF);
		}

		void ImporterSummedFlowNoise(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* ImporterFlowNoise = LoadFunction(TEXT("mdlimporter_flow_noise"));
			UMaterialFunction* MakeFloat2 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat2.MakeFloat2"));

			UMaterialExpressionFunctionInput* Pos =
			    NewMaterialExpressionFunctionInput(Function, TEXT("pos"), EFunctionInputType::FunctionInput_Vector2);
			UMaterialExpressionFunctionInput* Time =
			    NewMaterialExpressionFunctionInput(Function, TEXT("time"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* Iterations =
			    NewMaterialExpressionFunctionInput(Function, TEXT("iterations"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* AbsNoise =
			    NewMaterialExpressionFunctionInput(Function, TEXT("abs_noise"), EFunctionInputType::FunctionInput_StaticBool);
			UMaterialExpressionFunctionInput* WeightFactor =
			    NewMaterialExpressionFunctionInput(Function, TEXT("weight_factor"), EFunctionInputType::FunctionInput_Scalar, 0.5f);
			UMaterialExpressionFunctionInput* PosFactor =
			    NewMaterialExpressionFunctionInput(Function, TEXT("pos_factor"), EFunctionInputType::FunctionInput_Scalar, 2.0f);
			UMaterialExpressionFunctionInput* UProgressiveScale =
			    NewMaterialExpressionFunctionInput(Function, TEXT("u_progressive_scale"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			UMaterialExpressionFunctionInput* VProgressiveOffset =
			    NewMaterialExpressionFunctionInput(Function, TEXT("v_progressive_offset"), EFunctionInputType::FunctionInput_Scalar, 0.0f);

			// i == 0
			UMaterialExpressionAppendVector*         P0       = NewMaterialExpressionAppendVector(Function, Pos, Time);
			UMaterialExpressionIf*                   LerpPos0 = NewMaterialExpressionIfGreater(Function, Iterations, 1.0f, 0.0f, 1.0f);
			UMaterialExpressionMaterialFunctionCall* Noise0   = NewMaterialExpressionFunctionCall(
                Function, ImporterFlowNoise,
                {NewMaterialExpressionFunctionCall(
                     Function, MakeFloat2,
                     {NewMaterialExpressionMultiply(Function,
                                                    NewMaterialExpressionComponentMask(Function, P0, 1),
                                                    NewMaterialExpressionAdd(Function,
                                                                             NewMaterialExpressionOneMinus(Function, LerpPos0),
                                                                             NewMaterialExpressionMultiply(Function, LerpPos0, UProgressiveScale))),
                      NewMaterialExpressionAdd(Function,
                                               NewMaterialExpressionComponentMask(Function, P0, 2),
                                               NewMaterialExpressionMultiply(Function, LerpPos0, VProgressiveOffset))}),
                 NewMaterialExpressionComponentMask(Function, P0, 4)});
			UMaterialExpressionStaticSwitch* Sum1 =
			    NewMaterialExpressionStaticSwitch(Function, AbsNoise, NewMaterialExpressionAbs(Function, Noise0), Noise0);

			// i == 1
			UMaterialExpressionDivide* InvIterations =
			    NewMaterialExpressionDivide(Function, 1.0f, NewMaterialExpressionSubtract(Function, Iterations, 1.0f));
			UMaterialExpressionMultiply* P1 = NewMaterialExpressionMultiply(Function, P0, PosFactor);
			UMaterialExpressionIf*       LerpPos1 =
			    NewMaterialExpressionIfGreater(Function, Iterations, 1.0f, NewMaterialExpressionMultiply(Function, 1.0f, InvIterations), 1.0f);
			UMaterialExpressionMaterialFunctionCall* Noise1 = NewMaterialExpressionFunctionCall(
			    Function, ImporterFlowNoise,
			    {NewMaterialExpressionFunctionCall(
			         Function, MakeFloat2,
			         {NewMaterialExpressionMultiply(Function,
			                                        NewMaterialExpressionComponentMask(Function, P1, 1),
			                                        NewMaterialExpressionAdd(Function,
			                                                                 NewMaterialExpressionOneMinus(Function, LerpPos1),
			                                                                 NewMaterialExpressionMultiply(Function, LerpPos1, UProgressiveScale))),
			          NewMaterialExpressionAdd(Function,
			                                   {NewMaterialExpressionComponentMask(Function, P1, 2),
			                                    NewMaterialExpressionMultiply(Function, LerpPos1, VProgressiveOffset), 7.0f})}),
			     NewMaterialExpressionComponentMask(Function, P1, 4)});
			UMaterialExpressionAdd* Sum2 = NewMaterialExpressionAdd(
			    Function,
			    Sum1,
			    NewMaterialExpressionMultiply(
			        Function, WeightFactor,
			        NewMaterialExpressionStaticSwitch(Function, AbsNoise, NewMaterialExpressionAbs(Function, Noise1), Noise1)));

			// i == 2
			UMaterialExpressionMultiply* Weight2 = NewMaterialExpressionMultiply(Function, WeightFactor, WeightFactor);
			UMaterialExpressionMultiply* P2      = NewMaterialExpressionMultiply(Function, P1, PosFactor);
			UMaterialExpressionIf*       LerpPos2 =
			    NewMaterialExpressionIfGreater(Function, Iterations, 1.0f, NewMaterialExpressionMultiply(Function, 2.0f, InvIterations), 1.0f);
			UMaterialExpressionMaterialFunctionCall* Noise2 = NewMaterialExpressionFunctionCall(
			    Function, ImporterFlowNoise,
			    {NewMaterialExpressionFunctionCall(
			         Function, MakeFloat2,
			         {NewMaterialExpressionMultiply(Function,
			                                        NewMaterialExpressionComponentMask(Function, P2, 1),
			                                        NewMaterialExpressionAdd(Function,
			                                                                 NewMaterialExpressionOneMinus(Function, LerpPos2),
			                                                                 NewMaterialExpressionMultiply(Function, LerpPos2, UProgressiveScale))),
			          NewMaterialExpressionAdd(Function,
			                                   {NewMaterialExpressionComponentMask(Function, P2, 2),
			                                    NewMaterialExpressionMultiply(Function, LerpPos2, VProgressiveOffset), 14.0f})}),
			     NewMaterialExpressionComponentMask(Function, P2, 4)});
			UMaterialExpressionAdd* Sum3 = NewMaterialExpressionAdd(
			    Function,
			    Sum2,
			    NewMaterialExpressionMultiply(
			        Function, Weight2, NewMaterialExpressionStaticSwitch(Function, AbsNoise, NewMaterialExpressionAbs(Function, Noise2), Noise2)));

			// i == 3
			UMaterialExpressionMultiply* Weight3 = NewMaterialExpressionMultiply(Function, Weight2, WeightFactor);
			UMaterialExpressionMultiply* P3      = NewMaterialExpressionMultiply(Function, P2, PosFactor);
			UMaterialExpressionIf*       LerpPos3 =
			    NewMaterialExpressionIfGreater(Function, Iterations, 1.0f, NewMaterialExpressionMultiply(Function, 3.0f, InvIterations), 1.0f);
			UMaterialExpressionMaterialFunctionCall* Noise3 = NewMaterialExpressionFunctionCall(
			    Function, ImporterFlowNoise,
			    {NewMaterialExpressionFunctionCall(
			         Function, MakeFloat2,
			         {NewMaterialExpressionMultiply(Function,
			                                        NewMaterialExpressionComponentMask(Function, P3, 1),
			                                        NewMaterialExpressionAdd(Function,
			                                                                 NewMaterialExpressionOneMinus(Function, LerpPos3),
			                                                                 NewMaterialExpressionMultiply(Function, LerpPos3, UProgressiveScale))),
			          NewMaterialExpressionAdd(Function,
			                                   {NewMaterialExpressionComponentMask(Function, P3, 2),
			                                    NewMaterialExpressionMultiply(Function, LerpPos3, VProgressiveOffset), 21.0f})}),
			     NewMaterialExpressionComponentMask(Function, P3, 4)});
			UMaterialExpressionAdd* Sum4 = NewMaterialExpressionAdd(
			    Function,
			    Sum3,
			    NewMaterialExpressionMultiply(
			        Function, Weight3, NewMaterialExpressionStaticSwitch(Function, AbsNoise, NewMaterialExpressionAbs(Function, Noise3), Noise3)));

			// i == 4
			UMaterialExpressionMultiply* Weight4 = NewMaterialExpressionMultiply(Function, Weight3, WeightFactor);
			UMaterialExpressionMultiply* P4      = NewMaterialExpressionMultiply(Function, P3, PosFactor);
			UMaterialExpressionIf*       LerpPos4 =
			    NewMaterialExpressionIfGreater(Function, Iterations, 1.0f, NewMaterialExpressionMultiply(Function, 4.0f, InvIterations), 1.0f);
			UMaterialExpressionMaterialFunctionCall* Noise4 = NewMaterialExpressionFunctionCall(
			    Function, ImporterFlowNoise,
			    {NewMaterialExpressionFunctionCall(
			         Function, MakeFloat2,
			         {NewMaterialExpressionMultiply(Function,
			                                        NewMaterialExpressionComponentMask(Function, P4, 1),
			                                        NewMaterialExpressionAdd(Function,
			                                                                 NewMaterialExpressionOneMinus(Function, LerpPos4),
			                                                                 NewMaterialExpressionMultiply(Function, LerpPos4, UProgressiveScale))),
			          NewMaterialExpressionAdd(Function,
			                                   {NewMaterialExpressionComponentMask(Function, P4, 2),
			                                    NewMaterialExpressionMultiply(Function, LerpPos4, VProgressiveOffset), 28.0f})}),
			     NewMaterialExpressionComponentMask(Function, P4, 4)});
			UMaterialExpressionAdd* Sum5 = NewMaterialExpressionAdd(
			    Function,
			    Sum4,
			    NewMaterialExpressionMultiply(
			        Function, Weight4, NewMaterialExpressionStaticSwitch(Function, AbsNoise, NewMaterialExpressionAbs(Function, Noise4), Noise4)));

			UMaterialExpressionIf* SumGreaterTwo = NewMaterialExpressionIf(Function, Iterations, 4.0f, Sum3, Sum4, Sum5);
			UMaterialExpressionIf* Sum           = NewMaterialExpressionIf(Function, Iterations, 2.0f, Sum1, Sum2, SumGreaterTwo);

			NewMaterialExpressionFunctionOutput(Function, TEXT("sum"), Sum);
		}

		/*static*/ void ImporterTexremapu1(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialExpressionFunctionInput* TexRes =
			    NewMaterialExpressionFunctionInput(Function, TEXT("texres"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* CropOffset =
			    NewMaterialExpressionFunctionInput(Function, TEXT("crop_ofs"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* TexiIn =
			    NewMaterialExpressionFunctionInput(Function, TEXT("texi_in"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* Wrap =
			    NewMaterialExpressionFunctionInput(Function, TEXT("wrap"), EFunctionInputType::FunctionInput_Scalar);

			UMaterialExpressionIf*    S         = NewMaterialExpressionIfLess(Function, TexiIn, 0.0f, 1.0f, 0.0f);
			UMaterialExpressionFloor* D         = NewMaterialExpressionFloor(Function, NewMaterialExpressionDivide(Function, TexiIn, TexRes));
			UMaterialExpressionFmod*  Texi0     = NewMaterialExpressionFmod(Function, TexiIn, TexRes);
			UMaterialExpressionIf*    Alternate = NewMaterialExpressionIfEqual(
                Function, Wrap, wrap_mirrored_repeat,
                NewMaterialExpressionIfEqual(Function, NewMaterialExpressionFmod(Function, D, 2.0f), S, 0.0f, 1.0f), 0.0f);
			UMaterialExpressionIf* Texi1 =
			    NewMaterialExpressionIfEqual(Function, Alternate, 1.0f, NewMaterialExpressionNegate(Function, Texi0), Texi0);
			UMaterialExpressionIf* Texi2 =
			    NewMaterialExpressionIfEqual(Function, S, Alternate, Texi1, NewMaterialExpressionAdd(Function, {Texi1, TexRes, -1.0f}));
			UMaterialExpressionIf* WrapCheck = NewMaterialExpressionIfEqual(
			    Function, Wrap, wrap_clamp, NewMaterialExpressionClamp(Function, TexiIn, 0.0f, NewMaterialExpressionSubtract(Function, TexRes, 1.0f)),
			    Texi2);
			UMaterialExpressionAdd* Texi = NewMaterialExpressionAdd(Function, WrapCheck, CropOffset);

			NewMaterialExpressionFunctionOutput(Function, TEXT("texi"), Texi);
		}

		void ImporterTexremapu2(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* ImporterTexremapu1 = LoadFunction(TEXT("mdlimporter_texremapu1"));
			UMaterialFunction* MakeFloat2 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat2.MakeFloat2"));

			UMaterialExpressionFunctionInput* TexRes =
			    NewMaterialExpressionFunctionInput(Function, TEXT("texres"), EFunctionInputType::FunctionInput_Vector2);
			UMaterialExpressionFunctionInput* CropOffset =
			    NewMaterialExpressionFunctionInput(Function, TEXT("crop_ofs"), EFunctionInputType::FunctionInput_Vector2);
			UMaterialExpressionFunctionInput* Tex =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tex"), EFunctionInputType::FunctionInput_Vector2);
			UMaterialExpressionFunctionInput* WrapU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("wrap_u"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* WrapV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("wrap_v"), EFunctionInputType::FunctionInput_Scalar);

			UMaterialExpressionMaterialFunctionCall* Result =
			    NewMaterialExpressionFunctionCall(Function, MakeFloat2,
			                                      {NewMaterialExpressionFunctionCall(Function, ImporterTexremapu1,
			                                                                         {NewMaterialExpressionComponentMask(Function, TexRes, 1),
			                                                                          NewMaterialExpressionComponentMask(Function, CropOffset, 1),
			                                                                          NewMaterialExpressionComponentMask(Function, Tex, 1), WrapU}),
			                                       NewMaterialExpressionFunctionCall(Function, ImporterTexremapu1,
			                                                                         {NewMaterialExpressionComponentMask(Function, TexRes, 2),
			                                                                          NewMaterialExpressionComponentMask(Function, CropOffset, 2),
			                                                                          NewMaterialExpressionComponentMask(Function, Tex, 2), WrapV})});

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Result);
		}

		void ImporterTextureSample(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* BreakFloat2 =
			    LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("BreakFloat2Components.BreakFloat2Components"));
			UMaterialFunction* MakeFloat2 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat2.MakeFloat2"));

			UMaterialExpressionFunctionInput* Tex = NewMaterialExpressionFunctionInput(
			    Function, TEXT("tex"), EFunctionInputType::FunctionInput_Texture2D, NewMaterialExpressionTextureObject(Function, nullptr));
			UMaterialExpressionFunctionInput* Coord = NewMaterialExpressionFunctionInput(
			    Function, TEXT("coord"), EFunctionInputType::FunctionInput_Vector2, NewMaterialExpressionTextureCoordinate(Function, 0));
			UMaterialExpressionFunctionInput* WrapU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("wrap_u"), EFunctionInputType::FunctionInput_Scalar, wrap_repeat);
			UMaterialExpressionFunctionInput* WrapV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("wrap_v"), EFunctionInputType::FunctionInput_Scalar, wrap_repeat);
			UMaterialExpressionFunctionInput* CropU =
			    NewMaterialExpressionFunctionInput(Function, TEXT("crop_u"), EFunctionInputType::FunctionInput_Vector2, {0.0f, 1.0f});
			UMaterialExpressionFunctionInput* CropV =
			    NewMaterialExpressionFunctionInput(Function, TEXT("crop_v"), EFunctionInputType::FunctionInput_Vector2, {0.0f, 1.0f});

			UMaterialExpressionMaterialFunctionCall* UpperLeftCorner = NewMaterialExpressionFunctionCall(
			    Function, MakeFloat2,
			    {NewMaterialExpressionComponentMask(Function, CropU, 1), NewMaterialExpressionComponentMask(Function, CropV, 1)});
			UMaterialExpressionMaterialFunctionCall* LowerRightCorner = NewMaterialExpressionFunctionCall(
			    Function, MakeFloat2,
			    {NewMaterialExpressionComponentMask(Function, CropU, 2), NewMaterialExpressionComponentMask(Function, CropV, 2)});

			// we need to OneMinus the y-component of the texture coordinate, to get the correct texture access
			UMaterialExpressionMaterialFunctionCall* CoordBreak = NewMaterialExpressionFunctionCall(Function, BreakFloat2, {Coord});
			UMaterialExpressionAppendVector*         AdjustedCoord =
			    NewMaterialExpressionAppendVector(Function, {CoordBreak, 0}, NewMaterialExpressionOneMinus(Function, {CoordBreak, 1}));
			UMaterialExpressionDivide* CroppedUVs =
			    NewMaterialExpressionDivide(Function,
			                                NewMaterialExpressionSubtract(Function, AdjustedCoord, UpperLeftCorner),
			                                NewMaterialExpressionSubtract(Function, LowerRightCorner, UpperLeftCorner));

			UMaterialExpressionTextureSample* Sample = NewMaterialExpressionTextureSample(Function, Tex, CroppedUVs);

			NewMaterialExpressionFunctionOutput(Function, TEXT("rgb"), {Sample, 0});
			NewMaterialExpressionFunctionOutput(Function, TEXT("r"), {Sample, 1});
			NewMaterialExpressionFunctionOutput(Function, TEXT("g"), {Sample, 2});
			NewMaterialExpressionFunctionOutput(Function, TEXT("b"), {Sample, 3});
			NewMaterialExpressionFunctionOutput(Function, TEXT("a"), {Sample, 4});
		}

		void ImporterWorldAlignedTextureFloat3(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* CheapContrast =
			    LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions01/ImageAdjustment"), TEXT("CheapContrast.CheapContrast"));

			UMaterialExpressionFunctionInput* Tex =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tex"), EFunctionInputType::FunctionInput_Texture2D);
			UMaterialExpressionFunctionInput* Position =
			    NewMaterialExpressionFunctionInput(Function, TEXT("position"), EFunctionInputType::FunctionInput_Vector3);

			UMaterialExpressionTextureSample* RGSample =
			    NewMaterialExpressionTextureSample(Function, Tex, NewMaterialExpressionComponentMask(Function, Position, 3));
			UMaterialExpressionTextureSample* RBSample =
			    NewMaterialExpressionTextureSample(Function, Tex, NewMaterialExpressionComponentMask(Function, Position, 5));
			UMaterialExpressionTextureSample* GBSample =
			    NewMaterialExpressionTextureSample(Function, Tex, NewMaterialExpressionComponentMask(Function, Position, 6));

			UMaterialExpressionVertexNormalWS*       Normal    = NewMaterialExpression<UMaterialExpressionVertexNormalWS>(Function);
			UMaterialExpressionMaterialFunctionCall* ContrastR = NewMaterialExpressionFunctionCall(
			    Function, CheapContrast, {NewMaterialExpressionAbs(Function, NewMaterialExpressionComponentMask(Function, Normal, 1)), 1.0f});
			UMaterialExpressionMaterialFunctionCall* ContrastB = NewMaterialExpressionFunctionCall(
			    Function, CheapContrast, {NewMaterialExpressionAbs(Function, NewMaterialExpressionComponentMask(Function, Normal, 4)), 1.0f});

			UMaterialExpressionLinearInterpolate* Lerp1 = NewMaterialExpressionLinearInterpolate(Function, RBSample, GBSample, ContrastR);
			UMaterialExpressionLinearInterpolate* Lerp2 = NewMaterialExpressionLinearInterpolate(Function, Lerp1, RGSample, ContrastB);

			NewMaterialExpressionFunctionOutput(Function, TEXT("xyz texture"), Lerp2);
		}

		void ImporterWorldAlignedTextureFloat4(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* CheapContrast =
			    LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions01/ImageAdjustment"), TEXT("CheapContrast.CheapContrast"));

			UMaterialExpressionFunctionInput* Tex =
			    NewMaterialExpressionFunctionInput(Function, TEXT("tex"), EFunctionInputType::FunctionInput_Texture2D);
			UMaterialExpressionFunctionInput* Position =
			    NewMaterialExpressionFunctionInput(Function, TEXT("position"), EFunctionInputType::FunctionInput_Vector3);

			UMaterialExpressionTextureSample* RGSample =
			    NewMaterialExpressionTextureSample(Function, Tex, NewMaterialExpressionComponentMask(Function, Position, 3));
			UMaterialExpressionTextureSample* RBSample =
			    NewMaterialExpressionTextureSample(Function, Tex, NewMaterialExpressionComponentMask(Function, Position, 5));
			UMaterialExpressionTextureSample* GBSample =
			    NewMaterialExpressionTextureSample(Function, Tex, NewMaterialExpressionComponentMask(Function, Position, 6));

			UMaterialExpressionVertexNormalWS*       Normal    = NewMaterialExpression<UMaterialExpressionVertexNormalWS>(Function);
			UMaterialExpressionMaterialFunctionCall* ContrastR = NewMaterialExpressionFunctionCall(
			    Function, CheapContrast, {NewMaterialExpressionAbs(Function, NewMaterialExpressionComponentMask(Function, Normal, 1)), 1.0f});
			UMaterialExpressionMaterialFunctionCall* ContrastB = NewMaterialExpressionFunctionCall(
			    Function, CheapContrast, {NewMaterialExpressionAbs(Function, NewMaterialExpressionComponentMask(Function, Normal, 4)), 1.0f});

			UMaterialExpressionLinearInterpolate* Lerp1 =
			    NewMaterialExpressionLinearInterpolate(Function,
			                                           NewMaterialExpressionAppendVector(Function, {RBSample, 0}, {RBSample, 4}),
			                                           NewMaterialExpressionAppendVector(Function, {GBSample, 0}, {GBSample, 4}),
			                                           ContrastR);
			UMaterialExpressionLinearInterpolate* Lerp2 = NewMaterialExpressionLinearInterpolate(
			    Function, Lerp1, NewMaterialExpressionAppendVector(Function, {RGSample, 0}, {RGSample, 4}), ContrastB);

			NewMaterialExpressionFunctionOutput(Function, TEXT("xyz texture"), Lerp2);
		}

		void ImporterWorleyNoise(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* MakeFloat2 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat2.MakeFloat2"));

			UMaterialExpressionFunctionInput* Pos =
			    NewMaterialExpressionFunctionInput(Function, TEXT("pos"), EFunctionInputType::FunctionInput_Vector3);
			UMaterialExpressionFunctionInput* Jitter =
			    NewMaterialExpressionFunctionInput(Function, TEXT("jitter"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* Metric =
			    NewMaterialExpressionFunctionInput(Function, TEXT("metric"), EFunctionInputType::FunctionInput_Scalar);

			UMaterialExpressionVectorNoise*          Noise       = NewMaterialExpressionVectorNoise(Function, Pos, VNF_VoronoiALU, 4);
			UMaterialExpressionComponentMask*        NearestPos0 = NewMaterialExpressionComponentMask(Function, Noise, 7);
			UMaterialExpressionComponentMask*        Value       = NewMaterialExpressionComponentMask(Function, Noise, 8);
			UMaterialExpressionMaterialFunctionCall* Val =
			    NewMaterialExpressionFunctionCall(Function, MakeFloat2, {Value, Value});  // Val.y should correspond to NearestPos1 !!

			NewMaterialExpressionFunctionOutput(Function, TEXT("nearest_pos_0"), NearestPos0);
			NewMaterialExpressionFunctionOutput(Function, TEXT("nearest_pos_1"), NearestPos0);  // should be the second nearest position !!
			NewMaterialExpressionFunctionOutput(Function, TEXT("val"), Val);
		}

		void ImporterWorleyNoiseExt(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* ImporterWorleyNoise = LoadFunction(TEXT("mdlimporter_worley_noise"));
			UMaterialFunction* MathMaxValue        = LoadFunction(TEXT("mdl_math_max_value"));
			UMaterialFunction* MathSum             = LoadFunction(TEXT("mdl_math_sum"));

			UMaterialExpressionFunctionInput* Pos =
			    NewMaterialExpressionFunctionInput(Function, TEXT("pos"), EFunctionInputType::FunctionInput_Vector3);
			UMaterialExpressionFunctionInput* TurbulenceWeight =
			    NewMaterialExpressionFunctionInput(Function, TEXT("turbulence_weight"), EFunctionInputType::FunctionInput_Vector3);
			UMaterialExpressionFunctionInput* StepThreshold =
			    NewMaterialExpressionFunctionInput(Function, TEXT("step_threshold"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* Mode =
			    NewMaterialExpressionFunctionInput(Function, TEXT("mode"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* Metric =
			    NewMaterialExpressionFunctionInput(Function, TEXT("metric"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* Jitter =
			    NewMaterialExpressionFunctionInput(Function, TEXT("jitter"), EFunctionInputType::FunctionInput_Scalar, 1.0f);

			UMaterialExpressionMaterialFunctionCall* WorleyNoise =
			    NewMaterialExpressionFunctionCall(Function, ImporterWorleyNoise, {Pos, Jitter, Metric});
			UMaterialExpressionSubtract* PosDiff = NewMaterialExpressionSubtract(Function, {WorleyNoise, 0}, {WorleyNoise, 1});

			UMaterialExpressionComponentMask*        F1F2X     = NewMaterialExpressionComponentMask(Function, {WorleyNoise, 2}, 1);
			UMaterialExpressionComponentMask*        F1F2Y     = NewMaterialExpressionComponentMask(Function, {WorleyNoise, 2}, 2);
			UMaterialExpressionSubtract*             Diff      = NewMaterialExpressionSubtract(Function, F1F2Y, F1F2X);
			UMaterialExpressionAdd*                  Sum       = NewMaterialExpressionAdd(Function, F1F2Y, F1F2X);
			UMaterialExpression*                     Simple0   = F1F2X;
			UMaterialExpressionMultiply*             Simple1   = NewMaterialExpressionSquare(Function, F1F2X);
			UMaterialExpression*                     Cell      = F1F2X;  // would require some random/xor -> just do something here, for now!
			UMaterialExpressionIf*                   Step0     = NewMaterialExpressionIfLess(Function, Diff, StepThreshold, 0.0f, 1.0f);
			UMaterialExpression*                     Step1     = Diff;
			UMaterialExpressionDivide*               Step2     = NewMaterialExpressionDivide(Function, Diff, Sum);
			UMaterialExpressionMultiply*             Mul       = NewMaterialExpressionMultiply(Function, F1F2X, F1F2Y);
			UMaterialExpression*                     Add       = Sum;
			UMaterialExpression*                     Simple2   = F1F2Y;
			UMaterialExpressionMultiply*             Simple3   = NewMaterialExpressionSquare(Function, F1F2Y);
			UMaterialExpressionMaterialFunctionCall* Manhattan = NewMaterialExpressionFunctionCall(Function, MathSum, {PosDiff});
			UMaterialExpressionMaterialFunctionCall* Chebyshev =
			    NewMaterialExpressionFunctionCall(Function, MathMaxValue, {NewMaterialExpressionAbs(Function, PosDiff)});

			UMaterialExpressionIf* ModeResult = NewMaterialExpressionSwitch(
			    Function, Mode, {Simple0, Simple1, Cell, Step0, Step1, Step2, Mul, Add, Simple2, Simple3, Manhattan, Chebyshev});

			UMaterialExpressionIf* Result = NewMaterialExpressionIfEqual(
			    Function, TurbulenceWeight, {0.0f, 0.0f, 0.0f}, ModeResult,
			    NewMaterialExpressionSine(
			        Function, NewMaterialExpressionAdd(Function, NewMaterialExpressionDotProduct(Function, Pos, TurbulenceWeight), ModeResult)));

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Result);
		}

		// distillation support functions

		void DistillingSupportAddDetailNormal(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* StateNormal = LoadFunction(TEXT("mdl_state_normal"));

			Function->Description = TEXT("Returns a normal by adding a detail normal to a global normal.");

			UMaterialExpressionFunctionInput* ND = NewMaterialExpressionFunctionInput(Function, TEXT("nd"), EFunctionInputType::FunctionInput_Vector3,
			                                                                          NewMaterialExpressionFunctionCall(Function, StateNormal, {}));
			UMaterialExpressionFunctionInput* N  = NewMaterialExpressionFunctionInput(Function, TEXT("n"), EFunctionInputType::FunctionInput_Vector3,
                                                                                     NewMaterialExpressionFunctionCall(Function, StateNormal, {}));

			UMaterialExpressionAdd*      NT = NewMaterialExpressionAdd(Function, N, NewMaterialExpressionConstant(Function, 0.0f, 0.0f, 1.0f));
			UMaterialExpressionMultiply* NDT =
			    NewMaterialExpressionMultiply(Function, ND, NewMaterialExpressionConstant(Function, -1.0f, -1.0f, 1.0f));
			UMaterialExpressionNormalize* Result = NewMaterialExpressionNormalize(
			    Function,
			    NewMaterialExpressionSubtract(
			        Function,
			        NewMaterialExpressionMultiply(Function,
			                                      NT,
			                                      NewMaterialExpressionDivide(Function, NewMaterialExpressionDotProduct(Function, NT, NDT),
			                                                                  NewMaterialExpressionComponentMask(Function, NT, 4))),
			        NDT));

			NewMaterialExpressionFunctionOutput(Function, TEXT("n"), Result);
		}

		/*static*/ void DistillingSupportAverageFloatFloatFloatFloat(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("Returns the weighted and re-normalized average of r1 and r2. Returns 0 if w1 + w2 is too small.");

			UMaterialExpressionFunctionInput* InputW1 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("w1"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* R1 = NewMaterialExpressionFunctionInput(Function, TEXT("r1"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* InputW2 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("w2"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* R2 = NewMaterialExpressionFunctionInput(Function, TEXT("r2"), EFunctionInputType::FunctionInput_Scalar);

			UMaterialExpressionSaturate* W1 = NewMaterialExpressionSaturate(Function, InputW1);
			UMaterialExpressionSaturate* W2 = NewMaterialExpressionSaturate(Function, InputW2);

			UMaterialExpressionAdd* Sum    = NewMaterialExpressionAdd(Function, W1, W2);
			UMaterialExpressionIf*  Result = NewMaterialExpressionIfGreater(
                Function,
                NewMaterialExpressionAbs(Function, Sum),
                NewMaterialExpressionConstant(Function, 0.00001f),
                NewMaterialExpressionDivide(Function,
                                            NewMaterialExpressionAdd(Function, NewMaterialExpressionMultiply(Function, W1, R1),
                                                                     NewMaterialExpressionMultiply(Function, W2, R2)),
                                            Sum),
                NewMaterialExpressionConstant(Function, 0.0f));

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Result);
		}

		/*static*/ void DistillingSupportAverageFloatFloatFloatFloatFloatFloat(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("Returns the weighted and re-normalized average of r1, r2 and r3. Returns 0 if w1 + w2 + w3 is too small.");

			UMaterialExpressionFunctionInput* W1 = NewMaterialExpressionFunctionInput(Function, TEXT("w1"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* R1 = NewMaterialExpressionFunctionInput(Function, TEXT("r1"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* W2 = NewMaterialExpressionFunctionInput(Function, TEXT("w2"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* R2 = NewMaterialExpressionFunctionInput(Function, TEXT("r2"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* W3 = NewMaterialExpressionFunctionInput(Function, TEXT("w3"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* R3 = NewMaterialExpressionFunctionInput(Function, TEXT("r3"), EFunctionInputType::FunctionInput_Scalar);

			UMaterialExpressionAdd* Sum    = NewMaterialExpressionAdd(Function, {W1, W2, W3});
			UMaterialExpressionIf*  Result = NewMaterialExpressionIfGreater(
                Function,
                NewMaterialExpressionAbs(Function, Sum),
                NewMaterialExpressionConstant(Function, 0.00001f),
                NewMaterialExpressionDivide(Function,
                                            NewMaterialExpressionAdd(Function, {NewMaterialExpressionMultiply(Function, W1, R1),
                                                                                NewMaterialExpressionMultiply(Function, W2, R2),
                                                                                NewMaterialExpressionMultiply(Function, W3, R3)}),
                                            Sum),
                NewMaterialExpressionConstant(Function, 0.0f));

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Result);
		}

		void DistillingSupportAverageFloatColorFloatColor(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("Returns the weighted and re-normalized average of r1 and r2. Returns (0,0,0) if w1 + w2 is too small.");

			UMaterialFunction* MakeFloat3 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat3.MakeFloat3"));

			UMaterialExpressionFunctionInput* InputW1 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("w1"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* R1 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("r1"), EFunctionInputType::FunctionInput_Vector3);
			UMaterialExpressionFunctionInput* InputW2 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("w2"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* R2 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("r2"), EFunctionInputType::FunctionInput_Vector3);

			UMaterialExpressionSaturate* W1 = NewMaterialExpressionSaturate(Function, InputW1);
			UMaterialExpressionSaturate* W2 = NewMaterialExpressionSaturate(Function, InputW2);

			UMaterialExpressionAdd*    Sum = NewMaterialExpressionAdd(Function, W1, W2);
			UMaterialExpressionDivide* Yes = NewMaterialExpressionDivide(
			    Function,
			    NewMaterialExpressionAdd(Function, NewMaterialExpressionMultiply(Function, W1, R1), NewMaterialExpressionMultiply(Function, W2, R2)),
			    Sum);
			UMaterialExpressionIf* Result =
			    NewMaterialExpressionIfGreater(Function, NewMaterialExpressionAbs(Function, Sum), NewMaterialExpressionConstant(Function, 0.00001f),
			                                   Yes, NewMaterialExpressionFunctionCall(Function, MakeFloat3, {0.0f, 0.0f, 0.0f}));

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Result);
		}

		void DistillingSupportAverageFloatColorFloatColorFloatColor(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description =
			    TEXT("Returns the weighted and re-normalized average of r1, r2 and r3. Returns (0,0,0) if w1 + w2 + w3 is too small.");

			UMaterialFunction* MakeFloat3 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat3.MakeFloat3"));

			UMaterialExpressionFunctionInput* W1 = NewMaterialExpressionFunctionInput(Function, TEXT("w1"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* R1 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("r1"), EFunctionInputType::FunctionInput_Vector3);
			UMaterialExpressionFunctionInput* W2 = NewMaterialExpressionFunctionInput(Function, TEXT("w2"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* R2 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("r2"), EFunctionInputType::FunctionInput_Vector3);
			UMaterialExpressionFunctionInput* W3 = NewMaterialExpressionFunctionInput(Function, TEXT("w3"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* R3 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("r3"), EFunctionInputType::FunctionInput_Vector3);

			UMaterialExpressionAdd*    Sum = NewMaterialExpressionAdd(Function, {W1, W2, W3});
			UMaterialExpressionDivide* Yes = NewMaterialExpressionDivide(
			    Function,
			    NewMaterialExpressionAdd(Function, {NewMaterialExpressionMultiply(Function, W1, R1), NewMaterialExpressionMultiply(Function, W2, R2),
			                                        NewMaterialExpressionMultiply(Function, W3, R3)}),
			    Sum);
			UMaterialExpressionIf* Result =
			    NewMaterialExpressionIfGreater(Function, NewMaterialExpressionAbs(Function, Sum), NewMaterialExpressionConstant(Function, 0.00001f),
			                                   Yes, NewMaterialExpressionFunctionCall(Function, MakeFloat3, {0.0f, 0.0f, 0.0f}));

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Result);
		}

		/*static*/ void DistillingSupportCombineAnisotropicRoughness(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT(
			    "Combines the two anisotropic roughness values into one. The current heuristic chooses the lower roughness. Just averaging leads to too dull looking materials.");

			UMaterialExpressionFunctionInput* R1 = NewMaterialExpressionFunctionInput(Function, TEXT("r1"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* R2 = NewMaterialExpressionFunctionInput(Function, TEXT("r2"), EFunctionInputType::FunctionInput_Scalar);

			UMaterialExpressionMin* Result = NewMaterialExpressionMin(Function, R1, R2);

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Result);
		}

		void DistillingSupportAffineNormalSumFloatFloat3(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("Returns a normal for a weighted combination of n normals.");

			UMaterialFunction*                       StateNormal = LoadFunction(TEXT("mdl_state_normal"));
			UMaterialExpressionMaterialFunctionCall* N           = NewMaterialExpressionFunctionCall(Function, StateNormal, {});

			UMaterialFunction* MakeFloat3 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat3.MakeFloat3"));

			UMaterialExpressionFunctionInput* W1 = NewMaterialExpressionFunctionInput(Function, TEXT("w1"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* N1 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("n1"), EFunctionInputType::FunctionInput_Vector3, N);
			UMaterialExpressionMultiply*  A      = NewMaterialExpressionMultiply(Function, NewMaterialExpressionSubtract(Function, N1, N),
                                                                           NewMaterialExpressionFunctionCall(Function, MakeFloat3, {W1, W1, W1}));
			UMaterialExpressionNormalize* Result = NewMaterialExpressionNormalize(Function, NewMaterialExpressionAdd(Function, A, N));

			NewMaterialExpressionFunctionOutput(Function, TEXT("n"), Result);
		}

		void DistillingSupportAffineNormalSumFloatFloat3FloatFloat3(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("Returns a normal for a weighted combination of n normals.");

			UMaterialFunction*                       StateNormal = LoadFunction(TEXT("mdl_state_normal"));
			UMaterialExpressionMaterialFunctionCall* N           = NewMaterialExpressionFunctionCall(Function, StateNormal, {});

			UMaterialFunction* MakeFloat3 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat3.MakeFloat3"));

			UMaterialExpressionFunctionInput* W1 = NewMaterialExpressionFunctionInput(Function, TEXT("w1"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* N1 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("n1"), EFunctionInputType::FunctionInput_Vector3, N);
			UMaterialExpressionFunctionInput* W2 = NewMaterialExpressionFunctionInput(Function, TEXT("w2"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* N2 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("n2"), EFunctionInputType::FunctionInput_Vector3, N);

			UMaterialExpressionNormalize* Result = NewMaterialExpressionNormalize(
			    Function, NewMaterialExpressionAdd(
			                  Function, {NewMaterialExpressionMultiply(Function, NewMaterialExpressionSubtract(Function, N1, N),
			                                                           NewMaterialExpressionFunctionCall(Function, MakeFloat3, {W1, W1, W1})),
			                             NewMaterialExpressionMultiply(Function, NewMaterialExpressionSubtract(Function, N2, N),
			                                                           NewMaterialExpressionFunctionCall(Function, MakeFloat3, {W2, W2, W2})),
			                             N}));

			NewMaterialExpressionFunctionOutput(Function, TEXT("n"), Result);
		}

		void DistillingSupportAffineNormalSumFloatFloat3FloatFloat3FloatFloat3(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("Returns a normal for a weighted combination of n normals.");

			UMaterialFunction*                       StateNormal = LoadFunction(TEXT("mdl_state_normal"));
			UMaterialExpressionMaterialFunctionCall* N           = NewMaterialExpressionFunctionCall(Function, StateNormal, {});

			UMaterialFunction* MakeFloat3 = LoadFunction(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility"), TEXT("MakeFloat3.MakeFloat3"));

			UMaterialExpressionFunctionInput* W1 = NewMaterialExpressionFunctionInput(Function, TEXT("w1"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* N1 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("n1"), EFunctionInputType::FunctionInput_Vector3, N);
			UMaterialExpressionFunctionInput* W2 = NewMaterialExpressionFunctionInput(Function, TEXT("w2"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* N2 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("n2"), EFunctionInputType::FunctionInput_Vector3, N);
			UMaterialExpressionFunctionInput* W3 = NewMaterialExpressionFunctionInput(Function, TEXT("w3"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* N3 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("n3"), EFunctionInputType::FunctionInput_Vector3, N);

			UMaterialExpressionNormalize* Result = NewMaterialExpressionNormalize(
			    Function, NewMaterialExpressionAdd(
			                  Function, {NewMaterialExpressionMultiply(Function, NewMaterialExpressionSubtract(Function, N1, N),
			                                                           NewMaterialExpressionFunctionCall(Function, MakeFloat3, {W1, W1, W1})),
			                             NewMaterialExpressionMultiply(Function, NewMaterialExpressionSubtract(Function, N2, N),
			                                                           NewMaterialExpressionFunctionCall(Function, MakeFloat3, {W2, W2, W2})),
			                             NewMaterialExpressionMultiply(Function, NewMaterialExpressionSubtract(Function, N3, N),
			                                                           NewMaterialExpressionFunctionCall(Function, MakeFloat3, {W3, W3, W3})),
			                             N}));

			NewMaterialExpressionFunctionOutput(Function, TEXT("n"), Result);
		}

		void DistillingSupportCombineNormals(UMaterialFunction* Function, int32 ArraySize)
		{
			// alternative way of combining the 2 normals
			// float3 n = float3(n1_t.x + n2_t.x,n1_t.y + n2_t.y, n1_t.z*n2_t.z);

			check(0 == ArraySize);
			UMaterialFunction* StateNormal = LoadFunction(TEXT("mdl_state_normal"));

			Function->Description = TEXT("Returns a normal as a weighted combination of two normals.");

			UMaterialExpressionFunctionInput* W1 = NewMaterialExpressionFunctionInput(Function, TEXT("w1"), EFunctionInputType::FunctionInput_Scalar,
			                                                                          NewMaterialExpressionConstant(Function, 1.0f));
			UMaterialExpressionFunctionInput* N1 = NewMaterialExpressionFunctionInput(Function, TEXT("n1"), EFunctionInputType::FunctionInput_Vector3,
			                                                                          NewMaterialExpressionFunctionCall(Function, StateNormal, {}));
			UMaterialExpressionFunctionInput* W2 = NewMaterialExpressionFunctionInput(Function, TEXT("w2"), EFunctionInputType::FunctionInput_Scalar,
			                                                                          NewMaterialExpressionConstant(Function, 1.0f));
			UMaterialExpressionFunctionInput* N2 = NewMaterialExpressionFunctionInput(Function, TEXT("n2"), EFunctionInputType::FunctionInput_Vector3,
			                                                                          NewMaterialExpressionFunctionCall(Function, StateNormal, {}));

			UMaterialExpressionConstant3Vector* Z = NewMaterialExpressionConstant(Function, 0.0f, 0.0f, 1.0f);
			UMaterialExpressionAdd*       N1T = NewMaterialExpressionAdd(Function, NewMaterialExpressionLinearInterpolate(Function, Z, N1, W1), Z);
			UMaterialExpressionMultiply*  N2T = NewMaterialExpressionMultiply(Function, NewMaterialExpressionLinearInterpolate(Function, Z, N2, W2),
                                                                             NewMaterialExpressionConstant(Function, -1.0f, -1.0f, 1.0f));
			UMaterialExpressionNormalize* N   = NewMaterialExpressionNormalize(
                Function,
                NewMaterialExpressionSubtract(
                    Function,
                    NewMaterialExpressionMultiply(Function,
                                                  N1T,
                                                  NewMaterialExpressionDivide(Function, NewMaterialExpressionDotProduct(Function, N1T, N2T),
                                                                              NewMaterialExpressionComponentMask(Function, N1T, 4))),
                    N2T));

			NewMaterialExpressionFunctionOutput(Function, TEXT("n"), N);
		}

		/*static*/ void DistillingSupportDirectionalColoring(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT(
			    "A helper function, marked as \"noinline\", to communicate directional dependent coloring information to be picked up by later passes and integration code. Note: this cannot be expressed fully functional in MDL itself, it targets a different model.");

			UMaterialExpressionFunctionInput* C0 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("c_0"), EFunctionInputType::FunctionInput_Vector3);
			UMaterialExpressionFunctionInput* C90 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("c_90"), EFunctionInputType::FunctionInput_Vector3);
			UMaterialExpressionFunctionInput* E = NewMaterialExpressionFunctionInput(Function, TEXT("e"), EFunctionInputType::FunctionInput_Scalar);
			C0->PreviewValue.X                  = 1.f;
			C90->PreviewValue.X                 = 0.f;
			E->PreviewValue.X                   = 5.f;

			UMaterialExpression* Fresnel = NewMaterialExpressionFresnel(Function, E, {}, {});
			UMaterialExpression* Result  = NewMaterialExpressionLinearInterpolate(Function, {C0, C90}, Fresnel);

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Result);
		}

		/*static*/ void DistillingSupportDirectionalWeighting(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT(
			    "A helper function, marked as \"noinline\", to communicate directional dependent coloring information to be picked up by later passes and integration code. Note: this cannot be expressed fully functional in MDL itself, it targets a different model.");

			UMaterialExpressionFunctionInput* W0 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("w_0"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* W90 =
			    NewMaterialExpressionFunctionInput(Function, TEXT("w_90"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* E = NewMaterialExpressionFunctionInput(Function, TEXT("e"), EFunctionInputType::FunctionInput_Scalar);
			W0->PreviewValue.X                  = 1.f;
			W90->PreviewValue.X                 = 0.f;
			E->PreviewValue.X                   = 5.f;

			UMaterialExpression* Fresnel = NewMaterialExpressionFresnel(Function, E, {}, {});
			UMaterialExpression* Result  = NewMaterialExpressionLinearInterpolate(Function, {W0, W90}, Fresnel);

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Result);
		}

		/*static*/ void DistillingSupportPartNormalized(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);

			UMaterialExpressionFunctionInput* A  = NewMaterialExpressionFunctionInput(Function, TEXT("a"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* W1 = NewMaterialExpressionFunctionInput(Function, TEXT("w1"), EFunctionInputType::FunctionInput_Scalar);
			UMaterialExpressionFunctionInput* W2 = NewMaterialExpressionFunctionInput(Function, TEXT("w2"), EFunctionInputType::FunctionInput_Scalar);

			UMaterialExpressionAdd* Sum    = NewMaterialExpressionAdd(Function, W1, W2);
			UMaterialExpressionIf*  Result = NewMaterialExpressionIfGreater(Function,
                                                                           NewMaterialExpressionAbs(Function, Sum),
                                                                           NewMaterialExpressionConstant(Function, 0.00001f),
                                                                           NewMaterialExpressionDivide(Function, A, Sum),
                                                                           NewMaterialExpressionConstant(Function, 0.0f));

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Result);
		}

		/*static*/ void DistillingSupportReflFromIORFloat3(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("Returns a normal incidence direction reflectivity value for a given IOR.");

			UMaterialExpressionFunctionInput* IOR =
			    NewMaterialExpressionFunctionInput(Function, TEXT("ior"), EFunctionInputType::FunctionInput_Vector3);

			UMaterialExpressionDivide* RootR =
			    NewMaterialExpressionDivide(Function, NewMaterialExpressionAdd(Function, IOR, -1), NewMaterialExpressionAdd(Function, IOR, 1));
			UMaterialExpressionMultiply* Refl = NewMaterialExpressionSquare(Function, RootR);

			NewMaterialExpressionFunctionOutput(Function, TEXT("refl"), Refl);
		}

		/*static*/ void DistillingSupportReflFromIORFloat(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			Function->Description = TEXT("Returns a normal incidence direction reflectivity value for a given IOR.");

			UMaterialExpressionFunctionInput* IOR =
				NewMaterialExpressionFunctionInput(Function, TEXT("ior"), EFunctionInputType::FunctionInput_Scalar);

			UMaterialExpressionDivide* RootR =
				NewMaterialExpressionDivide(Function, NewMaterialExpressionAdd(Function, IOR, -1), NewMaterialExpressionAdd(Function, IOR, 1));
			UMaterialExpressionMultiply* Refl = NewMaterialExpressionSquare(Function, RootR);

			NewMaterialExpressionFunctionOutput(Function, TEXT("refl"), Refl);
		}

		void UnrealPixelNormalWS(UMaterialFunction* Function, int32 ArraySize)
		{
			UMaterialExpressionPixelNormalWS* Normal = NewMaterialExpression<UMaterialExpressionPixelNormalWS>(Function);
			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Normal);
		}

		void UnrealVertexNormalWS(UMaterialFunction* Function, int32 ArraySize)
		{
			UMaterialExpressionVertexNormalWS* Normal = NewMaterialExpression<UMaterialExpressionVertexNormalWS>(Function);
			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Normal);
		}

		void UnrealFresnel(UMaterialFunction* Function, int32 ArraySize)
		{
			UMaterialExpressionFunctionInput* Exponent = NewMaterialExpressionFunctionInput(Function, TEXT("exponent"), EFunctionInputType::FunctionInput_Scalar, 5.0f);
			UMaterialExpressionFunctionInput* BaseReflectRraction = NewMaterialExpressionFunctionInput(Function, TEXT("base_reflect_fraction"), EFunctionInputType::FunctionInput_Scalar, 0.04f);
			UMaterialExpressionFunctionInput* Normal = NewMaterialExpressionFunctionInput(Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3, NewMaterialExpression<UMaterialExpressionPixelNormalWS>(Function));

			UMaterialExpressionFresnel* Fresnel = NewMaterialExpressionFresnel(Function, Exponent, BaseReflectRraction, Normal);

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), Fresnel);
		}

		void UnrealCameraVector(UMaterialFunction* Function, int32 ArraySize)
		{
			UMaterialExpressionCameraVectorWS* CameraVector = NewMaterialExpression<UMaterialExpressionCameraVectorWS>(Function);
			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), CameraVector);
		}

		void UnrealTransformTangentToWorld(UMaterialFunction* Function, int32 ArraySize)
		{
			UMaterialExpressionFunctionInput* Vector = NewMaterialExpressionFunctionInput(Function, TEXT("vector"), EFunctionInputType::FunctionInput_Vector3, {});

			UMaterialExpressionTransform* TransformedVector = NewMaterialExpressionTransform(Function, Vector, TRANSFORMSOURCE_Tangent, TRANSFORM_World);

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), TransformedVector);
		}

		void UnrealTransformWorldToTangent(UMaterialFunction* Function, int32 ArraySize)
		{
			UMaterialExpressionFunctionInput* Vector = NewMaterialExpressionFunctionInput(Function, TEXT("vector"), EFunctionInputType::FunctionInput_Vector3, {});

			UMaterialExpressionTransform* TransformedVector = NewMaterialExpressionTransform(Function, Vector, TRANSFORMSOURCE_World, TRANSFORM_Tangent);

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), TransformedVector);
		}

		void UnrealTangentSpaceNormal(UMaterialFunction* Function, int32 ArraySize)
		{
			check(0 == ArraySize);
			UMaterialFunction* StateNormal = LoadFunction(TEXT("mdl_state_normal"));
			UMaterialFunction* StateTextureTangentU = LoadFunction(TEXT("mdl_state_texture_tangent_u"));
			UMaterialFunction* StateTextureTangentV = LoadFunction(TEXT("mdl_state_texture_tangent_v"));

			Function->Description = TEXT("Interprets the color values as a vector in tangent space.");

			UMaterialExpressionFunctionInput* Normal = NewMaterialExpressionFunctionInput(Function, TEXT("normal"), EFunctionInputType::FunctionInput_Vector3, NewMaterialExpressionFunctionCall(Function, StateNormal, {}));
			UMaterialExpressionFunctionInput* TangentU = NewMaterialExpressionFunctionInput(Function, TEXT("tangent_u"), EFunctionInputType::FunctionInput_Vector3, NewMaterialExpressionFunctionCall(Function, StateTextureTangentU, { 0 }));
			UMaterialExpressionFunctionInput* TangentV = NewMaterialExpressionFunctionInput(Function, TEXT("tangent_v"), EFunctionInputType::FunctionInput_Vector3, NewMaterialExpressionFunctionCall(Function, StateTextureTangentV, { 0 }));

			UMaterialExpressionMaterialFunctionCall* DefaultNormal = NewMaterialExpressionFunctionCall(Function, StateNormal, {});

			UMaterialExpressionNormalize* UnclippedNormal =
				NewMaterialExpressionNormalize(Function,
					NewMaterialExpressionAdd(Function,
						{
							NewMaterialExpressionMultiply(Function, TangentU, NewMaterialExpressionComponentMask(Function, Normal, 1)),
							NewMaterialExpressionMultiply(Function, TangentV, NewMaterialExpressionComponentMask(Function, Normal, 2)),
							NewMaterialExpressionMultiply(Function,
							DefaultNormal,
							NewMaterialExpressionComponentMask(Function, Normal, 4)
							)
						}));

			NewMaterialExpressionFunctionOutput(Function, TEXT("normal"), UnclippedNormal);
		}

		void UnrealEmissiveMultiplier(UMaterialFunction* Function, int32 ArraySize)
		{
			// Emissive multiplier is always 1.0 in UE
			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), NewMaterialExpressionConstant(Function, 1.0f));
		}

		void UnrealSubsurfaceWeight(UMaterialFunction* Function, int32 ArraySize)
		{
			NewMaterialExpressionFunctionOutput(Function, TEXT("weight"), 0.0f);
		}

		void UnrealSubsurfaceColor(UMaterialFunction* Function, int32 ArraySize)
		{
			UMaterialExpressionFunctionInput* Color = NewMaterialExpressionFunctionInput(Function, TEXT("color"), EFunctionInputType::FunctionInput_Vector3, { 1.0f, 1.0f, 1.0f });
			NewMaterialExpressionFunctionOutput(Function, TEXT("color"), Color);
		}

		void UnrealSubsurfaceOpacity(UMaterialFunction* Function, int32 ArraySize)
		{
			UMaterialExpressionFunctionInput* Opacity = NewMaterialExpressionFunctionInput(Function, TEXT("opacity"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			NewMaterialExpressionFunctionOutput(Function, TEXT("opacity"), Opacity);
		}

		void UnrealOpacityWeight(UMaterialFunction* Function, int32 ArraySize)
		{
			// always return 1
			UMaterialExpressionFunctionInput* Opacity = NewMaterialExpressionFunctionInput(Function, TEXT("opacity"), EFunctionInputType::FunctionInput_Scalar, 1.0f);
			NewMaterialExpressionFunctionOutput(Function, TEXT("weight"), 1.0f);
		}

		void UnrealTextureLookup(UMaterialFunction* Function, int32 ArraySize)
		{
			// Do nothing for this function, because UE did in Common.ush
			UMaterialExpressionFunctionInput* TextureSample = NewMaterialExpressionFunctionInput(Function, TEXT("texture_sample"), EFunctionInputType::FunctionInput_Vector4, { 0.0f, 0.0f, 0.0f, 1.0f });

			NewMaterialExpressionFunctionOutput(Function, TEXT("result"), TextureSample);
		}

		void UnrealTranslucentGetTint(UMaterialFunction* Function, int32 ArraySize)
		{
			UMaterialExpressionFunctionInput* BaseColor = NewMaterialExpressionFunctionInput(Function, TEXT("base_color"), EFunctionInputType::FunctionInput_Vector3, { 0.0f, 0.0f, 0.0f });
			UMaterialExpressionFunctionInput* Opacity = NewMaterialExpressionFunctionInput(Function, TEXT("opacity"), EFunctionInputType::FunctionInput_Scalar, 1.0f);

			NewMaterialExpressionFunctionOutput(Function, TEXT("tint"), BaseColor);
		}

	private:
		static TPair<UMaterialExpression*, UMaterialExpression*> MixAttributesRecursive(UMaterialFunction* Function,
		                                                                                UMaterialFunction* MatLayerBlend_Standard,
		                                                                                const TArray<UMaterialExpressionClamp*>&         Weights,
		                                                                                const TArray<UMaterialExpressionFunctionInput*>& Components,
		                                                                                int32 Begin, int32 End)
		{
			check(Begin < End);

			int32 Count = End - Begin;
			if (Count == 1)
			{
				return TPair<UMaterialExpression*, UMaterialExpression*>(Weights[Begin], Components[Begin]);
			}
			else
			{
				int32                                             Center = (Begin + End) / 2;
				TPair<UMaterialExpression*, UMaterialExpression*> LeftMix =
				    MixAttributesRecursive(Function, MatLayerBlend_Standard, Weights, Components, Begin, Center);
				TPair<UMaterialExpression*, UMaterialExpression*> RightMix =
				    MixAttributesRecursive(Function, MatLayerBlend_Standard, Weights, Components, Center, End);
				UMaterialExpressionAdd*    WeightSum = NewMaterialExpressionAdd(Function, LeftMix.Key, RightMix.Key);
				UMaterialExpressionDivide* NormalizedWeight =
				    NewMaterialExpressionDivide(Function, LeftMix.Key, NewMaterialExpressionClamp(Function, WeightSum, SMALL_NUMBER, Count));
				UMaterialExpressionMaterialFunctionCall* ComponentMix =
				    NewMaterialExpressionFunctionCall(Function, MatLayerBlend_Standard, {RightMix.Value, LeftMix.Value, NormalizedWeight});
				return TPair<UMaterialExpression*, UMaterialExpression*>(WeightSum, ComponentMix);
			}
		}

		static TPair<UMaterialExpressionClamp*, UMaterialExpression*> MixAttributes(UMaterialFunction*                       Function,
		                                                                            UMaterialFunction*                       MatLayerBlend_Standard,
		                                                                            const TArray<UMaterialExpressionClamp*>& Weights,
		                                                                            const TArray<UMaterialExpressionFunctionInput*>& Components)
		{
			TPair<UMaterialExpression*, UMaterialExpression*> Mix =
			    MixAttributesRecursive(Function, MatLayerBlend_Standard, Weights, Components, 0, Weights.Num());
			return TPair<UMaterialExpressionClamp*, UMaterialExpression*>(NewMaterialExpressionClamp(Function, Mix.Key, 0.0f, 1.0f), Mix.Value);
		}
	};
}  // namespace Generator
