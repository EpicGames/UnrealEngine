// Copyright Epic Games, Inc. All Rights Reserved.

#include "FunctionLoader.h"

#include "FunctionGenerator.h"
#include "common/Logging.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Factories/MaterialFunctionFactoryNew.h"
#include "MaterialEditingLibrary.h"
#include "PackageTools.h"
#include "UObject/Package.h"

namespace Generator
{
	class FFunctionGenerator : public FBaseFunctionGenerator
	{
	public:
		FFunctionGenerator(FFunctionLoader* Loader)
		    : Loader(Loader)
		{
		}

		virtual UMaterialFunction* LoadFunction(const FString& AssetName) override
		{
			return Loader->Load(AssetName, 0);
		}

		virtual UMaterialFunction* LoadFunction(const FString& AssetPath, const FString& AssetName) override
		{
			return Loader->Load(AssetPath, AssetName, 0);
		}

	private:
		FFunctionLoader* Loader;
	};

	namespace
	{
		int32 GetVersion(UMaterialFunction* Function)
		{
			int32 Version         = -1;
			int32 VersionPosition = Function->Description.Find(TEXT("\nVersion "), ESearchCase::CaseSensitive, ESearchDir::FromEnd);
			if (0 <= VersionPosition)
			{
				check(VersionPosition + 9 < Function->Description.Len());
				FString VersionString = Function->Description.RightChop(VersionPosition + 9);
				Version               = FCString::Atoi(*VersionString);
			}
			return Version;
		}

		UMaterialFunction* GetFunction(const FString& AssetPath, FString FunctionName)
		{
			FunctionName = FunctionName + TEXT(".") + FunctionName;
			auto* Fct    = LoadObject<UMaterialFunction>(nullptr, *(AssetPath / FunctionName), nullptr, LOAD_EditorOnly | LOAD_NoWarn, nullptr);
			check(Fct != nullptr);
			return Fct;
		}
	}

	FFunctionLoader::FFunctionLoader()
	    : FunctionFactory(NewObject<UMaterialFunctionFactoryNew>())
	    , FunctionGenerator(new FFunctionGenerator(this))
		, ObjectFlags(RF_Public | RF_Standalone)
	{
		FunctionFactory->AddToRoot();  // prevent garbage collection of this object

		const int32 BaseVersion = 1;  // Increase this version when need to regenerate every function(e.g. when some base functionality changes which affects most/a lot of functions)

		FunctionGenerateMap.Add(TEXT("mdl_base_abbe_number_ior"), {&FFunctionGenerator::BaseAbbeNumberIOR, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_base_anisotropy_conversion"), {&FFunctionGenerator::BaseAnisotropyConversion, 2 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_base_architectural_gloss_to_rough"), {&FFunctionGenerator::BaseArchitecturalGlossToRough, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_base_blend_color_layers"), {&FFunctionGenerator::BaseBlendColorLayers, 2 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_base_checker_bump_texture"), {&FFunctionGenerator::BaseCheckerBumpTexture, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_base_checker_texture"), {&FFunctionGenerator::BaseCheckerTexture, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_base_coordinate_projection"), {&FFunctionGenerator::BaseCoordinateProjection, 2 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_base_coordinate_source"), {&FFunctionGenerator::BaseCoordinateSource, 3 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_base_file_bump_texture"), {&FFunctionGenerator::BaseFileBumpTexture, 5 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_base_file_texture"), {&FFunctionGenerator::BaseFileTexture, 4 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_base_flake_noise_bump_texture"), {&FFunctionGenerator::BaseFlakeNoiseBumpTexture, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_base_flake_noise_texture"), {&FFunctionGenerator::BaseFlakeNoiseTexture, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_base_flow_noise_bump_texture"), {&FFunctionGenerator::BaseFlowNoiseBumpTexture, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_base_flow_noise_texture"), {&FFunctionGenerator::BaseFlowNoiseTexture, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_base_gloss_to_rough"), {&FFunctionGenerator::BaseGlossToRough, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_base_perlin_noise_bump_texture"), {&FFunctionGenerator::BasePerlinNoiseBumpTexture, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_base_perlin_noise_texture"), {&FFunctionGenerator::BasePerlinNoiseTexture, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_base_rotation_translation_scale"), {&FFunctionGenerator::BaseRotationTranslationScale, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_base_sellmeier_coefficients_ior"), {&FFunctionGenerator::BaseSellmeierCoefficientsIOR, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_base_tangent_space_normal_texture"), {&FFunctionGenerator::BaseTangentSpaceNormalTexture, 4 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_base_texture_coordinate_info"), {&FFunctionGenerator::BaseTextureCoordinateInfo, 4 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_base_tile_bump_texture"), {&FFunctionGenerator::BaseTileBumpTexture, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_base_transform_coordinate"), {&FFunctionGenerator::BaseTransformCoordinate, 2 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_base_volume_coefficient"), {&FFunctionGenerator::BaseVolumeCoefficient, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_base_worley_noise_bump_texture"), {&FFunctionGenerator::BaseWorleyNoiseBumpTexture, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_base_worley_noise_texture"), {&FFunctionGenerator::BaseWorleyNoiseTexture, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_df_anisotropic_vdf"), {&FFunctionGenerator::DFAnisotropicVDF, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_df_backscattering_glossy_reflection_bsdf"), {&FFunctionGenerator::DFBackscatteringGlossyReflectionBSDF, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_df_custom_curve_layer"), {&FFunctionGenerator::DFCustomCurveLayer, 4 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_df_diffuse_edf"), {&FFunctionGenerator::DFDiffuseEDF, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_df_diffuse_reflection_bsdf"), {&FFunctionGenerator::DFDiffuseReflectionBSDF, 3 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_df_diffuse_transmission_bsdf"), {&FFunctionGenerator::DFDiffuseTransmissionBSDF, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_df_directional_factor"), {&FFunctionGenerator::DFDirectionalFactor, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_df_fresnel_layer"), {&FFunctionGenerator::DFFresnelLayer, 7 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_df_light_profile_maximum"), {&FFunctionGenerator::DFLightProfileMaximum, 2 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_df_light_profile_power"), {&FFunctionGenerator::DFLightProfilePower, 2 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_df_measured_bsdf"), {&FFunctionGenerator::DFMeasuredBSDF, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_df_measured_edf"), {&FFunctionGenerator::DFMeasuredEDF, 2 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_df_measured_curve_factor"), {&FFunctionGenerator::DFMeasuredCurveFactor, 2 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_df_microfacet_beckmann_smith_bsdf"), {&FFunctionGenerator::DFMicrofacetBeckmannSmithBSDF, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_df_microfacet_beckmann_vcavities_bsdf"), {&FFunctionGenerator::DFMicrofacetBeckmannVCavitiesBSDF, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_df_microfacet_ggx_smith_bsdf"), {&FFunctionGenerator::DFMicrofacetGGXSmithBSDF, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_df_microfacet_ggx_vcavities_bsdf"), {&FFunctionGenerator::DFMicrofacetGGXVCavitiesBSDF, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_df_normalized_mix"), {&FFunctionGenerator::DFNormalizedMix, 2 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_df_simple_glossy_bsdf"), {&FFunctionGenerator::DFSimpleGlossyBSDF, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_df_specular_bsdf"), {&FFunctionGenerator::DFSpecularBSDF, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_df_spot_edf"), {&FFunctionGenerator::DFSpotEDF, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_df_thin_film"), {&FFunctionGenerator::DFThinFilm, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_df_tint"), {&FFunctionGenerator::DFTint, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_df_ward_geisler_moroder_bsdf"), {&FFunctionGenerator::DFWardGeislerMoroderBSDF, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_df_weighted_layer"), {&FFunctionGenerator::DFWeightedLayer, 4 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_math_average"), {&FFunctionGenerator::MathAverage, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_math_cos_float"), {&FFunctionGenerator::MathCosFloat, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_math_cos_float3"), {&FFunctionGenerator::MathCosFloat3, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_math_log_float"), {&FFunctionGenerator::MathLogFloat, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_math_log_float3"), {&FFunctionGenerator::MathLogFloat3, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_math_log10_float"), {&FFunctionGenerator::MathLog10Float, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_math_log10_float3"), {&FFunctionGenerator::MathLog10Float3, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_math_log2_float"), {&FFunctionGenerator::MathLog2Float, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_math_log2_float3"), {&FFunctionGenerator::MathLog2Float3, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_math_luminance"), {&FFunctionGenerator::MathLuminance, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_math_max_value"), {&FFunctionGenerator::MathMaxValue, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_math_min_value"), {&FFunctionGenerator::MathMinValue, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_math_multiply_float4x4_float4"), {&FFunctionGenerator::MathMultiplyFloat4x4Float4, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_math_multiply_float4x4_float4x4"), {&FFunctionGenerator::MathMultiplyFloat4x4Float4x4, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_math_sin_float"), {&FFunctionGenerator::MathSinFloat, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_math_sin_float3"), {&FFunctionGenerator::MathSinFloat3, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_math_sum"), {&FFunctionGenerator::MathSum, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_state_animation_time"), {&FFunctionGenerator::StateAnimationTime, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_state_direction"), {&FFunctionGenerator::StateDirection, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_state_geometry_normal"), {&FFunctionGenerator::StateGeometryNormal, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_state_geometry_tangent_u"), {&FFunctionGenerator::StateGeometryTangentU, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_state_geometry_tangent_v"), {&FFunctionGenerator::StateGeometryTangentV, 2 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_state_meters_per_scene_unit"), {&FFunctionGenerator::StateMetersPerSceneUnit, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_state_normal"), {&FFunctionGenerator::StateNormal, 2 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_state_object_id"), {&FFunctionGenerator::StateObjectId, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_state_position"), {&FFunctionGenerator::StatePosition, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_state_scene_units_per_meter"), {&FFunctionGenerator::StateSceneUnitsPerMeter, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_state_tangent_space"), {&FFunctionGenerator::StateTangentSpace, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_state_texture_coordinate"), {&FFunctionGenerator::StateTextureCoordinate, 2 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_state_texture_space_max"), {&FFunctionGenerator::StateTextureSpaceMax, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_state_texture_tangent_u"), {&FFunctionGenerator::StateTextureTangentU, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_state_texture_tangent_v"), {&FFunctionGenerator::StateTextureTangentV, 2 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_state_transform_point"), {&FFunctionGenerator::StateTransformPoint, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_state_transform_vector"), {&FFunctionGenerator::StateTransformVector, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_tex_lookup_color"), {&FFunctionGenerator::TexLookupFloat3, 7 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_tex_lookup_float"), {&FFunctionGenerator::TexLookupFloat, 2 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_tex_lookup_float2"), {&FFunctionGenerator::TexLookupFloat2, 2 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_tex_lookup_float3"), {&FFunctionGenerator::TexLookupFloat3, 7 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_tex_lookup_float4"), {&FFunctionGenerator::TexLookupFloat4, 7 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_add_detail_normal"), {&FFunctionGenerator::ImporterAddDetailNormal, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_apply_noise_modifications"), {&FFunctionGenerator::ImporterApplyNoiseModifications, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_blend_clear_coat"), {&FFunctionGenerator::ImporterBlendClearCoat, 4 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_blend_colors"), {&FFunctionGenerator::ImporterBlendColors, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_calculate_hue"), {&FFunctionGenerator::ImporterCalculateHue, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_calculate_saturation"), {&FFunctionGenerator::ImporterCalculateSaturation, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_compute_cubic_transform"), {&FFunctionGenerator::ImporterComputeCubicTransform, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_compute_cylindric_transform"), {&FFunctionGenerator::ImporterComputeCylindricTransform, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_compute_spheric_projection"), {&FFunctionGenerator::ImporterComputeSphericProjection, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_compute_spheric_transform"), {&FFunctionGenerator::ImporterComputeSphericTransform, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_compute_tangents"), {&FFunctionGenerator::ImporterComputeTangents, 2 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_compute_tangents_transformed"), {&FFunctionGenerator::ImporterComputeTangentsTransformed, 2 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_eval_checker"), {&FFunctionGenerator::ImporterEvalChecker, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_eval_tile_function"), {&FFunctionGenerator::ImporterEvalTileFunction, 2 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_flow_noise"), {&FFunctionGenerator::ImporterFlowNoise, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_grad_flow"), {&FFunctionGenerator::ImporterGradFlow, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_hsv_to_rgb"), {&FFunctionGenerator::ImporterHSVToRGB, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_mono_mode"), {&FFunctionGenerator::ImporterMonoMode, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_mi_noise"), {&FFunctionGenerator::ImporterMiNoise, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_perlin_noise"), {&FFunctionGenerator::ImporterPerlinNoise, 2 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_permute_flow"), {&FFunctionGenerator::ImporterPermuteFlow, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_refract"), {&FFunctionGenerator::ImporterRefract, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_select_bsdf"), {&FFunctionGenerator::ImporterSelectBSDF, 2 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_set_clip_mask"), {&FFunctionGenerator::ImporterSetClipMask, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_set_refraction"), {&FFunctionGenerator::ImporterSetRefraction, 2 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_set_subsurface_color"), {&FFunctionGenerator::ImporterSetSubsurfaceColor, 2 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_summed_flow_noise"), {&FFunctionGenerator::ImporterSummedFlowNoise, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_summed_perlin_noise"), {&FFunctionGenerator::ImporterSummedPerlinNoise, 2 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_texremapu1"), {&FFunctionGenerator::ImporterTexremapu1, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_texremapu2"), {&FFunctionGenerator::ImporterTexremapu2, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_texture_sample"), {&FFunctionGenerator::ImporterTextureSample, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_world_aligned_texture_float3"), {&FFunctionGenerator::ImporterWorldAlignedTextureFloat3, 2 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_world_aligned_texture_float4"), {&FFunctionGenerator::ImporterWorldAlignedTextureFloat4, 2 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_worley_noise"), {&FFunctionGenerator::ImporterWorleyNoise, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdlimporter_worley_noise_ext"), {&FFunctionGenerator::ImporterWorleyNoiseExt, 1 + BaseVersion});
		// distillation support functions
		FunctionGenerateMap.Add(TEXT("mdl_nvidia_distilling_support_add_detail_normal_float3_float3"),
		                        {&FFunctionGenerator::DistillingSupportAddDetailNormal, 3 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_nvidia_distilling_support_average_float_float_float_float"),
		                        {&FFunctionGenerator::DistillingSupportAverageFloatFloatFloatFloat, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_nvidia_distilling_support_average_float_color_float_color"),
		                        {&FFunctionGenerator::DistillingSupportAverageFloatColorFloatColor, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_nvidia_distilling_support_average_float_float_float_float_float_float"),
		                        {&FFunctionGenerator::DistillingSupportAverageFloatFloatFloatFloatFloatFloat, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_nvidia_distilling_support_average_float_color_float_color_float_color"),
		                        {&FFunctionGenerator::DistillingSupportAverageFloatColorFloatColorFloatColor, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_nvidia_distilling_support_combine_anisotropic_roughness_float_float"),
		                        {&FFunctionGenerator::DistillingSupportCombineAnisotropicRoughness, 2 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_nvidia_distilling_support_combine_normals_float_float3_float_float3"),
		                        {&FFunctionGenerator::DistillingSupportCombineNormals, 3 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_nvidia_distilling_support_affine_normal_sum_float_float3"),
		                        {&FFunctionGenerator::DistillingSupportAffineNormalSumFloatFloat3, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_nvidia_distilling_support_affine_normal_sum_float_float3_float_float3"),
		                        {&FFunctionGenerator::DistillingSupportAffineNormalSumFloatFloat3FloatFloat3, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_nvidia_distilling_support_affine_normal_sum_float_float3_float_float3_float_float3"),
		                        {&FFunctionGenerator::DistillingSupportAffineNormalSumFloatFloat3FloatFloat3FloatFloat3, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_nvidia_distilling_support_directional_coloring_color_color_float"),
		                        {&FFunctionGenerator::DistillingSupportDirectionalColoring, 3 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_nvidia_distilling_support_directional_weighting_float_float_float"),
		                        {&FFunctionGenerator::DistillingSupportDirectionalWeighting, 3 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_nvidia_distilling_support_part_normalized_float_float_float"),
		                        {&FFunctionGenerator::DistillingSupportPartNormalized, 1 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_nvidia_distilling_support_refl_from_ior_color"), {&FFunctionGenerator::DistillingSupportReflFromIORFloat3, 3 + BaseVersion});
		FunctionGenerateMap.Add(TEXT("mdl_nvidia_distilling_support_refl_from_ior_float"), {&FFunctionGenerator::DistillingSupportReflFromIORFloat, 1 + BaseVersion});

		// common functions generation
		const FString EngineUtilityPath = TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility");
		CommonFunctions.SetNum((int)ECommonFunction::Count);
		// engine
		CommonFunctions[(int)ECommonFunction::MakeFloat2]       = GetFunction(EngineUtilityPath, TEXT("MakeFloat2"));
		CommonFunctions[(int)ECommonFunction::MakeFloat3]       = GetFunction(EngineUtilityPath, TEXT("MakeFloat3"));
		CommonFunctions[(int)ECommonFunction::MakeFloat4]       = GetFunction(EngineUtilityPath, TEXT("MakeFloat4"));
		CommonFunctions[(int)ECommonFunction::DitherTemporalAA] = GetFunction(EngineUtilityPath, TEXT("DitherTemporalAA"));

		// plugin
		const FString PluginMaterialsPath                              = TEXT("/MDLImporter/Materials/MDL/");
		CommonFunctions[(int)ECommonFunction::AdjustNormal]            = GetFunction(PluginMaterialsPath, TEXT("AdjustNormal"));
		CommonFunctions[(int)ECommonFunction::AngularDirection]        = GetFunction(PluginMaterialsPath, TEXT("AngularDirection"));
		CommonFunctions[(int)ECommonFunction::ColorMap]                = GetFunction(PluginMaterialsPath, TEXT("ColorMap"));
		CommonFunctions[(int)ECommonFunction::NormalMap]               = GetFunction(PluginMaterialsPath, TEXT("NormalMap"));
		CommonFunctions[(int)ECommonFunction::GrayscaleMap]            = GetFunction(PluginMaterialsPath, TEXT("GrayscaleMap"));
		CommonFunctions[(int)ECommonFunction::CarColorTable]           = GetFunction(PluginMaterialsPath, TEXT("CarColorTable"));
		CommonFunctions[(int)ECommonFunction::CarFlakes]               = GetFunction(PluginMaterialsPath, TEXT("CarFlakes"));
		CommonFunctions[(int)ECommonFunction::EstimateObjectThickness] = GetFunction(PluginMaterialsPath, TEXT("EstimateObjectThickness"));
		CommonFunctions[(int)ECommonFunction::VolumeAbsorptionColor]   = GetFunction(PluginMaterialsPath, TEXT("VolumeAbsorptionColor"));
		CommonFunctions[(int)ECommonFunction::TranslucentOpacity]      = GetFunction(PluginMaterialsPath, TEXT("TranslucentOpacity"));
		check(CommonFunctions.FindByPredicate([](const auto Fct) { return Fct == nullptr; }) == nullptr);

		// OmniPBR noinline functions. These are annotated as noinline and require UE specific implementations.
		NoInlineFunctionGenerateMap.Add(TEXT("OmniUe4Function_unpack_normal_map"), { &FFunctionGenerator::UnrealTextureLookup, 1  + BaseVersion});
		NoInlineFunctionGenerateMap.Add(TEXT("OmniUe4Function_greyscale_texture_lookup"), { &FFunctionGenerator::UnrealTextureLookup, 1  + BaseVersion});
		NoInlineFunctionGenerateMap.Add(TEXT("OmniUe4Function_pixel_normal_world_space"), { &FFunctionGenerator::UnrealPixelNormalWS, 1  + BaseVersion});
		NoInlineFunctionGenerateMap.Add(TEXT("OmniUe4Function_vertex_normal_world_space"), { &FFunctionGenerator::UnrealVertexNormalWS, 1  + BaseVersion});
		NoInlineFunctionGenerateMap.Add(TEXT("OmniUe4Function_fresnel"), { &FFunctionGenerator::UnrealFresnel, 1  + BaseVersion});
		NoInlineFunctionGenerateMap.Add(TEXT("OmniUe4Function_camera_vector"), { &FFunctionGenerator::UnrealCameraVector, 1  + BaseVersion});
		NoInlineFunctionGenerateMap.Add(TEXT("OmniUe4Function_transform_vector_from_tangent_to_world"), { &FFunctionGenerator::UnrealTransformTangentToWorld, 1  + BaseVersion});
		NoInlineFunctionGenerateMap.Add(TEXT("OmniUe4Function_transform_vector_from_world_to_tangent"), { &FFunctionGenerator::UnrealTransformWorldToTangent, 1  + BaseVersion});

		NoInlineFunctionGenerateMap.Add(TEXT("OmniUe4Base_tangent_space_normal"), { &FFunctionGenerator::UnrealTangentSpaceNormal, 1  + BaseVersion});
		NoInlineFunctionGenerateMap.Add(TEXT("OmniUe4Base_emissive_multiplier"), { &FFunctionGenerator::UnrealEmissiveMultiplier, 1  + BaseVersion});

		NoInlineFunctionGenerateMap.Add(TEXT("OmniUe4Translucent_tangent_space_normal"), { &FFunctionGenerator::UnrealTangentSpaceNormal, 1  + BaseVersion});
		NoInlineFunctionGenerateMap.Add(TEXT("OmniUe4Translucent_emissive_multiplier"), { &FFunctionGenerator::UnrealEmissiveMultiplier, 1  + BaseVersion});
		NoInlineFunctionGenerateMap.Add(TEXT("OmniUe4Translucent_get_translucent_tint"), { &FFunctionGenerator::UnrealTranslucentGetTint, 1  + BaseVersion});
		NoInlineFunctionGenerateMap.Add(TEXT("OmniUe4Translucent_get_emissive_intensity"), { &FFunctionGenerator::UnrealTranslucentGetTint, 1  + BaseVersion});
		NoInlineFunctionGenerateMap.Add(TEXT("OmniUe4Translucent_get_translucent_opacity"), { &FFunctionGenerator::UnrealOpacityWeight, 1  + BaseVersion});

		NoInlineFunctionGenerateMap.Add(TEXT("OmniUe4Subsurface_tangent_space_normal"), { &FFunctionGenerator::UnrealTangentSpaceNormal, 1  + BaseVersion});
		NoInlineFunctionGenerateMap.Add(TEXT("OmniUe4Subsurface_emissive_multiplier"), { &FFunctionGenerator::UnrealEmissiveMultiplier, 1  + BaseVersion});
		NoInlineFunctionGenerateMap.Add(TEXT("OmniUe4Subsurface_get_subsurface_weight"), { &FFunctionGenerator::UnrealSubsurfaceWeight, 1  + BaseVersion});
		NoInlineFunctionGenerateMap.Add(TEXT("OmniUe4Subsurface_get_subsurface_color"), { &FFunctionGenerator::UnrealSubsurfaceColor, 1  + BaseVersion});
		NoInlineFunctionGenerateMap.Add(TEXT("OmniUe4Subsurface_get_subsurface_opacity"), { &FFunctionGenerator::UnrealSubsurfaceOpacity, 1  + BaseVersion});

		NoInlineFunctionGenerateMap.Add(TEXT("OmniUe4Unlit_emissive_multiplier"), { &FFunctionGenerator::UnrealEmissiveMultiplier, 1  + BaseVersion});
	}

	FFunctionLoader::~FFunctionLoader() {}

	UMaterialFunction* FFunctionLoader::Generate(const FString& AssetPath, const FString& AssetName, int32 ArraySize)
	{
		FGenerationData* GenerationData = FunctionGenerateMap.Find(AssetName);
		if (!GenerationData)
		{
			for ( TMap< FString, FGenerationData >::TIterator It = NoInlineFunctionGenerateMap.CreateIterator(); It; ++It )
			{
				if ( AssetName.EndsWith( It.Key() ) )
				{
					GenerationData = &It.Value();
					break;
				}
			}

			if (!GenerationData)
			{
				UE_LOGF(LogMDLImporter, Warning, "Unknown function: %ls", *AssetName);
				return nullptr;
			}
		}

		FString FunctionName = AssetName;
		if (0 < ArraySize)
		{
			FunctionName += TEXT("_") + FString::FromInt(ArraySize);
		}

		FString   FunctionPackageName = UPackageTools::SanitizePackageName(*(AssetPath / FunctionName));
		UPackage* Package             = CreatePackage(*FunctionPackageName);

		UMaterialFunction* Function = Cast<UMaterialFunction>(
		    FunctionFactory->FactoryCreateNew(UMaterialFunction::StaticClass(), Package, *FunctionName, ObjectFlags, nullptr, GWarn));

		check(Function);
		Function->StateId = FGuid::NewGuid();

		((*FunctionGenerator).*(GenerationData->Generator))(Function, ArraySize);
		Function->Description += TEXT("\nVersion ") + FString::FromInt(GenerationData->Version);

		// Arrange editor nodes
		UMaterialEditingLibrary::LayoutMaterialFunctionExpressions(Function);

		Function->PostLoad();

		FAssetRegistryModule::AssetCreated(Function); 
		Function->MarkPackageDirty();

		return Function;
	}

	int32 FFunctionLoader::GetVersion(const FString& AssetName) const
	{
		const FGenerationData* GenerationData = FunctionGenerateMap.Find(AssetName);
		return GenerationData ? GenerationData->Version : -1;
	}

	UMaterialFunction* FFunctionLoader::Load(const FString& AssetPath, const FString& AssetName, int32 ArraySize)
	{
		check(!FunctionsAssetPath.IsEmpty());

		FString FunctionName = AssetName;
		if (0 < ArraySize)
		{
			FunctionName += TEXT("_") + FString::FromInt(ArraySize);
		}

		UMaterialFunction*  Function    = nullptr;
		auto* FunctionPtr = LoadedFunctions.Find(FunctionName);
		if (FunctionPtr && (*FunctionPtr)->IsValidLowLevel())
		{
			Function = *FunctionPtr;
		}
		else
		{
			Function =
			    LoadObject<UMaterialFunction>(nullptr, *(AssetPath / FunctionName), nullptr, LOAD_EditorOnly | LOAD_NoWarn | LOAD_Quiet, nullptr);

			if (Function && (Generator::GetVersion(Function) < GetVersion(AssetName)))
			{
				Function = nullptr;
			}
			if (!Function)
			{
				Function = Generate(AssetPath, AssetName, ArraySize);
			}

			LoadedFunctions.Add(AssetName, Function);
		}

		return Function;
	}

	void FFunctionLoader::AddReferencedObjects(FReferenceCollector& Collector)
	{
		Collector.AddReferencedObjects(LoadedFunctions);
		Collector.AddReferencedObjects(CommonFunctions);
	}

}  // namespace Generator
