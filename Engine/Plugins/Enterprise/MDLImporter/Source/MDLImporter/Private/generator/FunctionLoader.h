// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/MaterialFunction.h"
#include "UObject/GCObject.h"

class UMaterialFunctionFactoryNew;

namespace Generator
{
	class FFunctionGenerator;

	enum class ECommonFunction
	{
		MakeFloat2 = 0,
		MakeFloat3,
		MakeFloat4,
		DitherTemporalAA,
		AdjustNormal,
		AngularDirection,
		ColorMap,
		GrayscaleMap,
		NormalMap,
		CarColorTable,
		CarFlakes,
		EstimateObjectThickness,
		VolumeAbsorptionColor,
		TranslucentOpacity,
		Count
	};

	class FFunctionLoader : FNoncopyable, FGCObject
	{
	public:
		FFunctionLoader();
		~FFunctionLoader();

		UMaterialFunction* Load(const FString& AssetName, int32 ArraySize = 0);
		UMaterialFunction* Load(const FString& AssetPath, const FString& AssetName, int32 ArraySize = 0);
		UMaterialFunction& Get(ECommonFunction Function);

		void SetAssetPath(const FString& FunctionsAssetPath);
		void SetObjectFlags(EObjectFlags InObjectFlags) { ObjectFlags = InObjectFlags; }

		// FGCObject interface
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		virtual FString GetReferencerName() const override
		{
			return TEXT("Generator::FFunctionLoader");
		}
	private:
		int32              GetVersion(const FString& AssetName) const;
		UMaterialFunction* Generate(const FString& AssetPath, const FString& AssetName, int32 ArraySize);

	private:
		struct FGenerationData
		{
			void (FFunctionGenerator::*Generator)(UMaterialFunction*, int32 ArraySize);
			int32 Version;
		};

		UMaterialFunctionFactoryNew*      FunctionFactory;
		TUniquePtr<FFunctionGenerator>    FunctionGenerator;
		TMap<FString, FGenerationData>    FunctionGenerateMap;
		TMap<FString, FGenerationData>    NoInlineFunctionGenerateMap;
		TMap<FString, TObjectPtr<UMaterialFunction>> LoadedFunctions;
		TArray<TObjectPtr<UMaterialFunction>>        CommonFunctions;
		FString                           FunctionsAssetPath;
		EObjectFlags                      ObjectFlags;
	};

	inline UMaterialFunction& FFunctionLoader::Get(ECommonFunction Function)
	{
		check(CommonFunctions.Num() > (int)Function);
		check(CommonFunctions[(int)Function] != nullptr);
		return *CommonFunctions[(int)Function];
	}

	inline UMaterialFunction* FFunctionLoader::Load(const FString& AssetName, int32 ArraySize)
	{
		return Load(FunctionsAssetPath, AssetName, ArraySize);
	}

	inline void FFunctionLoader::SetAssetPath(const FString& InFunctionsAssetPath)
	{
		FunctionsAssetPath = InFunctionsAssetPath;
	}
}  // namespace Generator
