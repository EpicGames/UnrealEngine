// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialExpressions.h"

#include "Materials/MaterialExpressionAbs.h"
#include "common/Logging.h"
#include "Materials/MaterialExpressionAdd.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialExpressionAppendVector.h"
#include "Materials/MaterialExpressionArccosine.h"
#include "Materials/MaterialExpressionArcsine.h"
#include "Materials/MaterialExpressionArctangent.h"
#include "Materials/MaterialExpressionArctangent2.h"
#include "Materials/MaterialExpressionBlackBody.h"
#include "Materials/MaterialExpressionBreakMaterialAttributes.h"
#include "Materials/MaterialExpressionCameraPositionWS.h"
#include "Materials/MaterialExpressionCameraVectorWS.h"
#include "Materials/MaterialExpressionCeil.h"
#include "Materials/MaterialExpressionClamp.h"
#include "Materials/MaterialExpressionClearCoatNormalCustomOutput.h"
#include "Materials/MaterialExpressionComponentMask.h"
#include "Materials/MaterialExpressionConstant.h"
#include "Materials/MaterialExpressionConstant2Vector.h"
#include "Materials/MaterialExpressionConstant3Vector.h"
#include "Materials/MaterialExpressionConstant4Vector.h"
#include "Materials/MaterialExpressionCosine.h"
#include "Materials/MaterialExpressionCrossProduct.h"
#include "Materials/MaterialExpressionCustom.h"
#include "Materials/MaterialExpressionDistance.h"
#include "Materials/MaterialExpressionDivide.h"
#include "Materials/MaterialExpressionDotProduct.h"
#include "Materials/MaterialExpressionFloor.h"
#include "Materials/MaterialExpressionFmod.h"
#include "Materials/MaterialExpressionFrac.h"
#include "Materials/MaterialExpressionFresnel.h"
#include "Materials/MaterialExpressionFunctionInput.h"
#include "Materials/MaterialExpressionFunctionOutput.h"
#include "Materials/MaterialExpressionIf.h"
#include "Materials/MaterialExpressionLinearInterpolate.h"
#include "Materials/MaterialExpressionLogarithm2.h"
#include "Materials/MaterialExpressionMakeMaterialAttributes.h"
#include "Materials/MaterialExpressionMax.h"
#include "Materials/MaterialExpressionMin.h"
#include "Materials/MaterialExpressionMultiply.h"
#include "Materials/MaterialExpressionNoise.h"
#include "Materials/MaterialExpressionNormalize.h"
#include "Materials/MaterialExpressionOneMinus.h"
#include "Materials/MaterialExpressionPixelDepth.h"
#include "Materials/MaterialExpressionPixelNormalWS.h"
#include "Materials/MaterialExpressionPower.h"
#include "Materials/MaterialExpressionReflectionVectorWS.h"
#include "Materials/MaterialExpressionSaturate.h"
#include "Materials/MaterialExpressionScalarParameter.h"
#include "Materials/MaterialExpressionSine.h"
#include "Materials/MaterialExpressionSphereMask.h"
#include "Materials/MaterialExpressionSquareRoot.h"
#include "Materials/MaterialExpressionStaticBool.h"
#include "Materials/MaterialExpressionStaticBoolParameter.h"
#include "Materials/MaterialExpressionStaticSwitch.h"
#include "Materials/MaterialExpressionSubtract.h"
#include "Materials/MaterialExpressionTangent.h"
#include "Materials/MaterialExpressionTextureCoordinate.h"
#include "Materials/MaterialExpressionTextureObject.h"
#include "Materials/MaterialExpressionTextureObjectParameter.h"
#include "Materials/MaterialExpressionTextureProperty.h"
#include "Materials/MaterialExpressionTime.h"
#include "Materials/MaterialExpressionTransform.h"
#include "Materials/MaterialExpressionTransformPosition.h"
#include "Materials/MaterialExpressionTwoSidedSign.h"
#include "Materials/MaterialExpressionVectorNoise.h"
#include "Materials/MaterialExpressionVectorParameter.h"
#include "Materials/MaterialExpressionVertexNormalWS.h"
#include "Materials/MaterialExpressionWorldPosition.h"
#include "Materials/MaterialExpressionUtils.h"
#include "generator/MaterialExpressionConnection.h"

namespace Generator
{
	float  EvaluateFloat(const FMaterialExpressionConnection& Connection);
	float  EvaluateFloat(UMaterialExpression* Expression, int32 OutputIndex);
	uint32 ComponentCount(const FMaterialExpressionConnection& Input);

	void SetMaterialExpressionGroup(const FString& GroupName, UMaterialExpression* ParameterExpression)
	{
		if (ParameterExpression->IsA<UMaterialExpressionParameter>())
		{
			Cast<UMaterialExpressionParameter>(ParameterExpression)->Group = *GroupName;
		}
		else if (ParameterExpression->IsA<UMaterialExpressionTextureObjectParameter>())
		{
			Cast<UMaterialExpressionTextureObjectParameter>(ParameterExpression)->Group = *GroupName;
		}
	}

	float EvaluateFloat(const FMaterialExpressionConnection& Connection)
	{
		check(Connection.HasExpression());
		check(IsScalar(Connection));
		return EvaluateFloat(Connection.GetExpressionUnused(), Connection.GetExpressionOutputIndex());
	}

	float  EvaluateFloat(UMaterialExpression* Expression, int32 OutputIndex)
	{
		if (Expression->IsA<UMaterialExpressionConstant>())
		{
			return Cast<UMaterialExpressionConstant>(Expression)->R;
		}
		else if (Expression->IsA<UMaterialExpressionScalarParameter>())
		{
			return Cast<UMaterialExpressionScalarParameter>(Expression)->DefaultValue;
		}
		else
		{
			check(false);
			return 0.0f;
		}
	}

	bool IsBool(UMaterialExpression* Expression, int32 OutputIndex)
	{
		if (Expression->IsA<UMaterialExpressionIf>())
		{
			UMaterialExpressionIf* If = Cast<UMaterialExpressionIf>(Expression);
			bool                   bIsALessThanB = IsBool(If->ALessThanB.Expression, If->ALessThanB.OutputIndex);
			check((bIsALessThanB == IsBool(If->AGreaterThanB.Expression, If->AGreaterThanB.OutputIndex)) &&
				(!If->AEqualsB.Expression || (bIsALessThanB == IsBool(If->AEqualsB.Expression, If->AEqualsB.OutputIndex))));
			return bIsALessThanB;
		}
		else if (Expression->IsA<UMaterialExpressionFunctionInput>())
		{
			return (Cast<UMaterialExpressionFunctionInput>(Expression)->InputType == FunctionInput_StaticBool) || (Cast<UMaterialExpressionFunctionInput>(Expression)->InputType == FunctionInput_Bool);
		}
		else if (Expression->IsA<UMaterialExpressionMaterialFunctionCall>())
		{
			UMaterialExpressionMaterialFunctionCall* MaterialFunctionCall =
				Cast<UMaterialExpressionMaterialFunctionCall>(Expression);
			check(MaterialFunctionCall->FunctionOutputs[OutputIndex].ExpressionOutput->A.Expression);
			return IsBool(MaterialFunctionCall->FunctionOutputs[OutputIndex].ExpressionOutput->A.Expression,
						   MaterialFunctionCall->FunctionOutputs[OutputIndex].ExpressionOutput->A.OutputIndex);
		}
		else if (Expression->IsA<UMaterialExpressionStaticBool>() ||
			Expression->IsA<UMaterialExpressionStaticBoolParameter>())
		{
			return true;
		}
		else if (Expression->IsA<UMaterialExpressionStaticSwitch>())
		{
			UMaterialExpressionStaticSwitch* StaticSwitch = Cast<UMaterialExpressionStaticSwitch>(Expression);
			bool                             bValue = IsBool(StaticSwitch->A.Expression, StaticSwitch->A.OutputIndex);
			check(bValue == IsBool(StaticSwitch->B.Expression, StaticSwitch->B.OutputIndex));
			return bValue;
		}
		else
		{
			check(Expression->IsA<UMaterialExpressionAbs>() ||
				Expression->IsA<UMaterialExpressionAdd>() ||
				Expression->IsA<UMaterialExpressionAppendVector>() ||
				Expression->IsA<UMaterialExpressionComponentMask>() ||
				Expression->IsA<UMaterialExpressionConstant>() ||
				Expression->IsA<UMaterialExpressionConstant3Vector>() ||
				Expression->IsA<UMaterialExpressionCosine>() ||
				Expression->IsA<UMaterialExpressionDivide>() ||
				Expression->IsA<UMaterialExpressionDotProduct>() ||
				Expression->IsA<UMaterialExpressionLinearInterpolate>() ||
				Expression->IsA<UMaterialExpressionMakeMaterialAttributes>() ||
				Expression->IsA<UMaterialExpressionMax>() ||
				Expression->IsA<UMaterialExpressionMultiply>() ||
				Expression->IsA<UMaterialExpressionNormalize>() ||
				Expression->IsA<UMaterialExpressionOneMinus>() ||
				Expression->IsA<UMaterialExpressionScalarParameter>() ||
				Expression->IsA<UMaterialExpressionSine>() ||
				Expression->IsA<UMaterialExpressionSubtract>() ||
				Expression->IsA<UMaterialExpressionVectorParameter>() ||
				Expression->IsA<UMaterialExpressionTransform>() ||
				Expression->IsA<UMaterialExpressionTransformPosition>());
			return false;
		}
	}

	bool IsBool(const FMaterialExpressionConnection& Input)
	{
		switch (Input.GetConnectionType())
		{
			case Expression:
				return IsBool(Input.GetExpressionUnused(), Input.GetExpressionOutputIndex());
			case Boolean:
				return true;
			case Float:
			case Float2:
			case Float3:
			case Float4:
			case Texture:
			case TextureSelection:
				return false;
			default:
				check(false);
				return false;
		}
	}

	bool IsMaterialAttribute(UMaterialExpression* Expression, int32 OutputIndex)
	{
		if (Expression->IsA<UMaterialExpressionFunctionInput>())
		{
			return (Cast<UMaterialExpressionFunctionInput>(Expression)->InputType == FunctionInput_MaterialAttributes);
		}
		else if (Expression->IsA<UMaterialExpressionIf>())
		{
			UMaterialExpressionIf* If = Cast<UMaterialExpressionIf>(Expression);
			bool                   bIsAGreaterThanB = IsMaterialAttribute(If->AGreaterThanB.Expression, If->AGreaterThanB.OutputIndex);
			check((!If->AEqualsB.Expression || (bIsAGreaterThanB == IsMaterialAttribute(If->AEqualsB.Expression, If->AEqualsB.OutputIndex))) &&
				(bIsAGreaterThanB == IsMaterialAttribute(If->ALessThanB.Expression, If->ALessThanB.OutputIndex)));
			return bIsAGreaterThanB;
		}
		else if (Expression->IsA<UMaterialExpressionMakeMaterialAttributes>())
		{
			return true;
		}
		else if (Expression->IsA<UMaterialExpressionMaterialFunctionCall>())
		{
			UMaterialExpressionMaterialFunctionCall* MaterialFunctionCall =
				Cast<UMaterialExpressionMaterialFunctionCall>(Expression);
			check(MaterialFunctionCall->FunctionOutputs[OutputIndex].ExpressionOutput->A.Expression);
			return IsMaterialAttribute(MaterialFunctionCall->FunctionOutputs[OutputIndex].ExpressionOutput->A.Expression,
										MaterialFunctionCall->FunctionOutputs[OutputIndex].ExpressionOutput->A.OutputIndex);
		}
		else if (Expression->IsA<UMaterialExpressionStaticSwitch>())
		{
			UMaterialExpressionStaticSwitch* StaticSwitch = Cast<UMaterialExpressionStaticSwitch>(Expression);
			bool                             bValue = IsMaterialAttribute(StaticSwitch->A.Expression, StaticSwitch->A.OutputIndex);
			check(bValue == IsMaterialAttribute(StaticSwitch->B.Expression, StaticSwitch->B.OutputIndex));
			return bValue;
		}
		else
		{
			check(Expression->IsA<UMaterialExpressionAbs>() ||
				Expression->IsA<UMaterialExpressionAdd>() ||
				Expression->IsA<UMaterialExpressionAppendVector>() ||
				Expression->IsA<UMaterialExpressionClamp>() ||
				Expression->IsA<UMaterialExpressionComponentMask>() ||
				Expression->IsA<UMaterialExpressionConstant>() ||
				Expression->IsA<UMaterialExpressionConstant3Vector>() ||
				Expression->IsA<UMaterialExpressionCosine>() ||
				Expression->IsA<UMaterialExpressionCustom>() ||
				Expression->IsA<UMaterialExpressionDivide>() ||
				Expression->IsA<UMaterialExpressionDotProduct>() ||
				Expression->IsA<UMaterialExpressionLinearInterpolate>() ||
				Expression->IsA<UMaterialExpressionMax>() ||
				Expression->IsA<UMaterialExpressionMultiply>() ||
				Expression->IsA<UMaterialExpressionNormalize>() ||
				Expression->IsA<UMaterialExpressionOneMinus>() ||
				Expression->IsA<UMaterialExpressionPower>() ||
				Expression->IsA<UMaterialExpressionScalarParameter>() ||
				Expression->IsA<UMaterialExpressionSine>() ||
				Expression->IsA<UMaterialExpressionSubtract>() ||
				Expression->IsA<UMaterialExpressionTextureObject>() ||
				Expression->IsA<UMaterialExpressionTextureSample>() ||
				Expression->IsA<UMaterialExpressionTransform>() ||
				Expression->IsA<UMaterialExpressionTransformPosition>() ||
				Expression->IsA<UMaterialExpressionVectorParameter>());
			return false;
		}
	}

	bool IsMaterialAttribute(const FMaterialExpressionConnection& Input)
	{
		if (Input.GetConnectionType() == EConnectionType::Expression)
		{
			check(Input.HasExpression());
			return IsMaterialAttribute(Input.GetExpressionUnused(), Input.GetExpressionOutputIndex());
		}
		return false;
	}

	bool IsScalar(const FMaterialExpressionConnection& Input)
	{
		// no expression always means, it's a fit
		return ((Input.GetConnectionType() == EConnectionType::Expression) && !Input.GetExpressionUnused()) || (ComponentCount(Input) == 1);
	}

	bool IsStatic(const FMaterialExpressionConnection& Input)
	{
		// check that, if Input is a StaticSwitch and Input->A is Static, then Input->B has to be Static as well
		check(!(Input.IsExpressionA<UMaterialExpressionStaticSwitch>() &&
		        IsStatic(Cast<UMaterialExpressionStaticSwitch>(Input.GetExpressionUnused())->A.Expression)) ||
		      IsStatic(Cast<UMaterialExpressionStaticSwitch>(Input.GetExpressionUnused())->B.Expression));

		return Input.IsExpressionA<UMaterialExpressionStaticBool>() ||
		       Input.IsExpressionA<UMaterialExpressionStaticBoolParameter>() ||
		       (Input.IsExpressionA<UMaterialExpressionStaticSwitch>() &&
		        IsStatic(Cast<UMaterialExpressionStaticSwitch>(Input.GetExpressionUnused())->A.Expression));
	}

	bool IsTexture(UMaterialExpression* Expression, int32 OutputIndex)
	{
		if (Expression->IsA<UMaterialExpressionFunctionInput>())
		{
			return (Cast<UMaterialExpressionFunctionInput>(Expression)->InputType == FunctionInput_Texture2D);
		}
		else if (Expression->IsA<UMaterialExpressionIf>())
		{
			UMaterialExpressionIf* If = Cast<UMaterialExpressionIf>(Expression);
			bool                   bIsTex = IsTexture(If->ALessThanB.Expression, If->ALessThanB.OutputIndex);
			check((bIsTex == IsTexture(If->AEqualsB.Expression, If->AEqualsB.OutputIndex)) &&
				(bIsTex == IsTexture(If->AGreaterThanB.Expression, If->AGreaterThanB.OutputIndex)));
			return bIsTex;
		}
		else if (Expression->IsA<UMaterialExpressionStaticSwitch>())
		{
			UMaterialExpressionStaticSwitch* StaticSwitch = Cast<UMaterialExpressionStaticSwitch>(Expression);
			bool                             bValue = IsTexture(StaticSwitch->A.Expression, StaticSwitch->A.OutputIndex);
			check(bValue == IsTexture(StaticSwitch->B.Expression, StaticSwitch->B.OutputIndex));
			return bValue;
		}
		else if (Expression->IsA<UMaterialExpressionTextureObject>() ||
			Expression->IsA<UMaterialExpressionTextureObjectParameter>())
		{
			return true;
		}
		else
		{
			check(Expression->IsA<UMaterialExpressionAbs>() ||
				Expression->IsA<UMaterialExpressionAdd>() ||
				Expression->IsA<UMaterialExpressionBreakMaterialAttributes>() ||
				Expression->IsA<UMaterialExpressionComponentMask>() ||
				Expression->IsA<UMaterialExpressionConstant>() ||
				Expression->IsA<UMaterialExpressionConstant3Vector>() ||
				Expression->IsA<UMaterialExpressionCosine>() ||
				Expression->IsA<UMaterialExpressionDivide>() ||
				Expression->IsA<UMaterialExpressionLinearInterpolate>() ||
				Expression->IsA<UMaterialExpressionMakeMaterialAttributes>() ||
				Expression->IsA<UMaterialExpressionMaterialFunctionCall>() ||
				Expression->IsA<UMaterialExpressionMultiply>() ||
				Expression->IsA<UMaterialExpressionNormalize>() ||
				Expression->IsA<UMaterialExpressionOneMinus>() ||
				Expression->IsA<UMaterialExpressionPower>() ||
				Expression->IsA<UMaterialExpressionScalarParameter>() ||
				Expression->IsA<UMaterialExpressionSine>() ||
				Expression->IsA<UMaterialExpressionSubtract>() ||
				Expression->IsA<UMaterialExpressionTextureSample>() ||
				Expression->IsA<UMaterialExpressionTransform>() ||
				Expression->IsA<UMaterialExpressionTransformPosition>() ||
				Expression->IsA<UMaterialExpressionVectorParameter>());
			return false;
		}
	}

	bool IsTexture(const FMaterialExpressionConnection& Input)
	{
		switch (Input.GetConnectionType())
		{
			case EConnectionType::Expression:
				check(Input.HasExpression());
				return IsTexture(Input.GetExpressionUnused(), Input.GetExpressionOutputIndex());
			case EConnectionType::Texture:
				return !!Input.GetTextureUnused();
			case EConnectionType::Boolean:
			case EConnectionType::Float:
			case EConnectionType::Float2:
			case EConnectionType::Float3:
			case EConnectionType::Float4:
				return false;
			default:
				check(false);
				return false;
		}
	}

	bool IsVector3(const FMaterialExpressionConnection& Input)
	{
		// no expression always means, it's a fit
		return (Input.IsExpressionWithoutExpression()) || (ComponentCount(Input) == 3);
	}

	void CheckedConnect(UObject* Parent, const FMaterialExpressionConnection& Connection, FExpressionInput& Input)
	{
		switch (Connection.GetConnectionType())
		{
			case EConnectionType::Expression:
				if (Connection.HasExpression())
				{
					Input.Connect(Connection.GetExpressionOutputIndex(), Connection.GetExpressionAndUse());
				}
				break;
			case EConnectionType::Boolean:
				Input.Connect(0, NewMaterialExpressionStaticBool(Parent, Connection.GetBoolValue()));
				break;
			case EConnectionType::Float:
				Input.Connect(0, NewMaterialExpressionConstant(Parent, Connection.GetVectorValue()[0]));
				break;
			case EConnectionType::Float2:
				Input.Connect(0, NewMaterialExpressionConstant(Parent, Connection.GetVectorValue()[0], Connection.GetVectorValue()[1]));
				break;
			case EConnectionType::Float3:
				Input.Connect(0, NewMaterialExpressionConstant(Parent, Connection.GetVectorValue()[0], Connection.GetVectorValue()[1], Connection.GetVectorValue()[2]));
				break;
			case EConnectionType::Float4:
				Input.Connect(
				    0, NewMaterialExpressionConstant(Parent, 
						Connection.GetVectorValue()[0], Connection.GetVectorValue()[1], Connection.GetVectorValue()[2], Connection.GetVectorValue()[3]));
				break;
			case EConnectionType::Texture:
			{
				UTexture* Texture = Connection.GetTextureAndUse();
				if (ensure(Texture))
				{
					Input.Connect(0, NewMaterialExpressionTextureObject(Parent, Texture));
				}
				break;
			}
			default:
				check(false);
		}
	}

	void CheckedConnect(UObject* Parent, const FMaterialExpressionConnection& Connection, FExpressionInput& Input, float& Value)
	{
		if (Connection.GetConnectionType() == EConnectionType::Float)
		{
			Value = Connection.GetVectorValue()[0];
		}
		else
		{
			CheckedConnect(Parent, Connection, Input);
		}
	}

	void CheckedConnect(UObject* Parent, const FMaterialExpressionConnection& Connection, FExpressionInput& Input, UTexture** TexturePtr)
	{
		if (Connection.GetConnectionType() == EConnectionType::Texture)
		{
			UTexture* Texture = Connection.GetTextureAndUse();
			if (ensure(TexturePtr && Texture))
			{
				*TexturePtr = Texture;
			}
		}
		else
		{
			CheckedConnect(Parent, Connection, Input);
		}
	}

	void CheckedConnect(UObject* Parent, const FMaterialExpressionConnection& Connection, FExpressionInput& Input, TObjectPtr<UTexture>* OutTexturePtr)
	{
		if (Connection.GetConnectionType() == EConnectionType::Texture)
		{
			UTexture* Texture = Connection.GetTextureAndUse();
			if (ensure(OutTexturePtr && Texture))
			{
				*OutTexturePtr = Texture;
			}
		}
		else
		{
			CheckedConnect(Parent, Connection, Input);
		}
	}

	uint32 ComponentCount(UMaterialExpression* Expression, int32 OutputIndex = 0)
	{
		if (Expression->IsA<UMaterialExpressionAbs>())
		{
			UMaterialExpressionAbs* Abs = Cast<UMaterialExpressionAbs>(Expression);
			return ComponentCount(Abs->Input.Expression, Abs->Input.OutputIndex);
		}
		else if (Expression->IsA<UMaterialExpressionAdd>())
		{
			UMaterialExpressionAdd* Add = Cast<UMaterialExpressionAdd>(Expression);
			const uint32 ACount = Add->A.Expression ? ComponentCount(Add->A.Expression, Add->A.OutputIndex) : 1;
			const uint32 BCount = Add->B.Expression ? ComponentCount(Add->B.Expression, Add->B.OutputIndex) : 1;
			ensure((ACount == 1) || (BCount == 1) || (ACount == BCount));
			return FMath::Max(ACount, BCount);
		}
		else if (Expression->IsA<UMaterialExpressionAppendVector>())
		{
			UMaterialExpressionAppendVector* AppendVector = Cast<UMaterialExpressionAppendVector>(Expression);
			return ComponentCount(AppendVector->A.Expression, AppendVector->A.OutputIndex) +
				ComponentCount(AppendVector->B.Expression, AppendVector->B.OutputIndex);
		}
		else if (Expression->IsA<UMaterialExpressionArccosine>())
		{
			// arccosine always returns a float... check that it also just gets one!
			ensure(ComponentCount(Cast<UMaterialExpressionArccosine>(Expression)->Input.Expression,
								  OutputIndex) == 1);
			return 1;
		}
		else if (Expression->IsA<UMaterialExpressionArctangent2>())
		{
			// arctangent2 always returns a float... check that it also just gets one!
			UMaterialExpressionArctangent2* ArcTangent2 = Cast<UMaterialExpressionArctangent2>(Expression);
			ensure(ComponentCount(ArcTangent2->Y.Expression, ArcTangent2->Y.OutputIndex) == 1);
			ensure(ComponentCount(ArcTangent2->X.Expression, ArcTangent2->X.OutputIndex) == 1);
			return 1;
		}
		else if (Expression->IsA<UMaterialExpressionBlackBody>())
		{
			// BlackBody always returns a color, and digests a float
			UMaterialExpressionBlackBody* BlackBody = Cast<UMaterialExpressionBlackBody>(Expression);
			ensure(ComponentCount(BlackBody->Temp.Expression, BlackBody->Temp.OutputIndex) == 1);
			return 3;
		}
		else if (Expression->IsA<UMaterialExpressionBreakMaterialAttributes>())
		{
			ensure(OutputIndex < Expression->Outputs.Num());
			FExpressionOutput const& Output = Expression->Outputs[OutputIndex];
			return Output.MaskR + Output.MaskG + Output.MaskB + Output.MaskA;
		}
		else if (Expression->IsA<UMaterialExpressionCameraVectorWS>()
			|| Expression->IsA<UMaterialExpressionReflectionVectorWS>())
		{
			return 3;
		}
		else if (Expression->IsA<UMaterialExpressionClamp>())
		{
			UMaterialExpressionClamp* Clamp = Cast<UMaterialExpressionClamp>(Expression);
			return ComponentCount({ Clamp->Input.Expression, Clamp->Input.OutputIndex });
		}
		else if (Expression->IsA<UMaterialExpressionComponentMask>())
		{
			UMaterialExpressionComponentMask* ComponentMask = Cast<UMaterialExpressionComponentMask>(Expression);
			return ComponentMask->R + ComponentMask->G + ComponentMask->B + ComponentMask->A;
		}
		else if (Expression->IsA<UMaterialExpressionConstant>())
		{
			return 1;
		}
		else if (Expression->IsA<UMaterialExpressionConstant2Vector>())
		{
			return 2;
		}
		else if (Expression->IsA<UMaterialExpressionConstant3Vector>())
		{
			return 3;
		}
		else if (Expression->IsA<UMaterialExpressionConstant4Vector>())
		{
			return 4;
		}
		else if (Expression->IsA<UMaterialExpressionCosine>())
		{
			// cosine always returns a float... ensure that it also just gets one!
			ensure(ComponentCount(
				Cast<UMaterialExpressionCosine>(Expression)->Input.Expression, OutputIndex) == 1);
			return 1;
		}
		else if (Expression->IsA<UMaterialExpressionCrossProduct>())
		{
			// cross product always returns a float3... check that it also just gets such!
			UMaterialExpressionCrossProduct* CrossProduct = Cast<UMaterialExpressionCrossProduct>(Expression);
			ensure(ComponentCount(CrossProduct->A.Expression, CrossProduct->A.OutputIndex) == 3);
			ensure(ComponentCount(CrossProduct->B.Expression, CrossProduct->B.OutputIndex) == 3);
			return 3;
		}
		else if (Expression->IsA<UMaterialExpressionCustom>())
		{
			UMaterialExpressionCustom* Custom = Cast<UMaterialExpressionCustom>(Expression);
			switch (Custom->OutputType)
			{
			case CMOT_Float1:
				return 1;
			case CMOT_Float2:
				return 2;
			case CMOT_Float3:
				return 3;
			case CMOT_Float4:
				return 4;
			default:
				ensure(false);
				return 0;
			}
		}
		else if (Expression->IsA<UMaterialExpressionDistance>())
		{
			UMaterialExpressionDistance* Distance = Cast<UMaterialExpressionDistance>(Expression);
			const uint32 ACount = ComponentCount(Distance->A.Expression, Distance->A.OutputIndex);
			const uint32 BCount = ComponentCount(Distance->B.Expression, Distance->B.OutputIndex);
			ensure((1 == ACount) || (1 == BCount) || (ACount == BCount));
			return 1;
		}
		else if (Expression->IsA<UMaterialExpressionDivide>())
		{
			UMaterialExpressionDivide* Divide = Cast<UMaterialExpressionDivide>(Expression);
			const uint32 ACount = Divide->A.Expression ? ComponentCount(Divide->A.Expression, Divide->A.OutputIndex) : 1;
			const uint32 BCount = Divide->B.Expression ? ComponentCount(Divide->B.Expression, Divide->B.OutputIndex) : 1;
			ensure((1 == ACount) || (1 == BCount) || (ACount == BCount));
			return FMath::Max(ACount, BCount);
		}
		else if (Expression->IsA<UMaterialExpressionDotProduct>())
		{
			return 1;
		}
		else if (Expression->IsA<UMaterialExpressionFloor>())
		{
			UMaterialExpressionFloor* Floor = Cast<UMaterialExpressionFloor>(Expression);
			return ComponentCount(Floor->Input.Expression, Floor->Input.OutputIndex);
		}
		else if (Expression->IsA<UMaterialExpressionFmod>())
		{
			UMaterialExpressionFmod* Fmod = Cast<UMaterialExpressionFmod>(Expression);
			const uint32 ACount = ComponentCount(Fmod->A.Expression, Fmod->A.OutputIndex);
			ensure(ACount == ComponentCount(Fmod->B.Expression, Fmod->B.OutputIndex));
			return ACount;
		}
		else if (Expression->IsA<UMaterialExpressionFrac>())
		{
			UMaterialExpressionFrac* Frac = Cast<UMaterialExpressionFrac>(Expression);
			return ComponentCount(Frac->Input.Expression, Frac->Input.OutputIndex);
		}
		else if (Expression->IsA<UMaterialExpressionFresnel>())
		{
			return 1;
		}
		else if (Expression->IsA<UMaterialExpressionFunctionInput>())
		{
			switch (Cast<UMaterialExpressionFunctionInput>(Expression)->InputType)
			{
			case FunctionInput_Scalar:
				return 1;
			case FunctionInput_Vector2:
				return 2;
			case FunctionInput_Vector3:
				return 3;
			case FunctionInput_Vector4:
				return 4;
			case FunctionInput_Texture2D:
			case FunctionInput_TextureCube:
			case FunctionInput_VolumeTexture:
			case FunctionInput_StaticBool:
			case FunctionInput_Bool:
			case FunctionInput_MaterialAttributes:
			default:
				ensure(false);
				return 0;
			}
		}
		else if (Expression->IsA<UMaterialExpressionIf>())
		{
			UMaterialExpressionIf* If = Cast<UMaterialExpressionIf>(Expression);
			const uint32 ALessThanBCount = ComponentCount(If->ALessThanB.Expression, If->ALessThanB.OutputIndex);
			ensure((!If->AEqualsB.Expression || (ALessThanBCount == ComponentCount(If->AEqualsB.Expression, If->AEqualsB.OutputIndex))) &&
				(ALessThanBCount == ComponentCount(If->AGreaterThanB.Expression, If->AGreaterThanB.OutputIndex)));
			return ALessThanBCount;
		}
		else if (Expression->IsA<UMaterialExpressionLinearInterpolate>())
		{
			UMaterialExpressionLinearInterpolate* LinearInterpolate =
				Cast<UMaterialExpressionLinearInterpolate>(Expression);
			const uint32 ACount =
				LinearInterpolate->A.Expression ? ComponentCount(LinearInterpolate->A.Expression, LinearInterpolate->A.OutputIndex) : 1;
			const uint32 BCount =
				LinearInterpolate->B.Expression ? ComponentCount(LinearInterpolate->B.Expression, LinearInterpolate->B.OutputIndex) : 1;
			ensure(ACount == BCount);
			return ACount;
		}
		else if (Expression->IsA<UMaterialExpressionLogarithm2>())
		{
			UMaterialExpressionLogarithm2* Logarithm2 = Cast<UMaterialExpressionLogarithm2>(Expression);
			return ComponentCount(Logarithm2->X.Expression, Logarithm2->X.OutputIndex);
		}
		else if (Expression->IsA<UMaterialExpressionMaterialFunctionCall>())
		{
			UMaterialExpressionMaterialFunctionCall* MaterialFunctionCall =
				Cast<UMaterialExpressionMaterialFunctionCall>(Expression);
			ensure(MaterialFunctionCall->FunctionOutputs[OutputIndex].ExpressionOutput->A.Expression);
			return ComponentCount(MaterialFunctionCall->FunctionOutputs[OutputIndex].ExpressionOutput->A.Expression,
								   MaterialFunctionCall->FunctionOutputs[OutputIndex].ExpressionOutput->A.OutputIndex);
		}
		else if (Expression->IsA<UMaterialExpressionMax>())
		{
			UMaterialExpressionMax* Max = Cast<UMaterialExpressionMax>(Expression);
			const uint32 ACount = Max->A.Expression ? ComponentCount(Max->A.Expression, Max->A.OutputIndex) : 1;
			const uint32 BCount = Max->B.Expression ? ComponentCount(Max->B.Expression, Max->B.OutputIndex) : 1;
			ensure((ACount == 1) || (BCount == 1) || (ACount == BCount));
			return FMath::Max(ACount, BCount);
		}
		else if (Expression->IsA<UMaterialExpressionMin>())
		{
			UMaterialExpressionMin* Min = Cast<UMaterialExpressionMin>(Expression);
			const uint32 ACount = Min->A.Expression ? ComponentCount(Min->A.Expression, Min->A.OutputIndex) : 1;
			const uint32 BCount = Min->B.Expression ? ComponentCount(Min->B.Expression, Min->B.OutputIndex) : 1;
			ensure((ACount == 1) || (BCount == 1) || (ACount == BCount));

			return FMath::Max(ACount, BCount);
		}
		else if (Expression->IsA<UMaterialExpressionMultiply>())
		{
			UMaterialExpressionMultiply* Multiply = Cast<UMaterialExpressionMultiply>(Expression);
			const uint32 ACount = Multiply->A.Expression ? ComponentCount(Multiply->A.Expression, Multiply->A.OutputIndex) : 1;
			const uint32 BCount = Multiply->B.Expression ? ComponentCount(Multiply->B.Expression, Multiply->B.OutputIndex) : 1;
			ensure((ACount == 1) || (BCount == 1) || (ACount == BCount));
			return FMath::Max(ACount, BCount);
		}
		else if (Expression->IsA<UMaterialExpressionNormalize>())
		{
			UMaterialExpressionNormalize* Normalize = Cast<UMaterialExpressionNormalize>(Expression);
			return ComponentCount(Normalize->VectorInput.Expression, Normalize->VectorInput.OutputIndex);
		}
		else if (Expression->IsA<UMaterialExpressionOneMinus>())
		{
			UMaterialExpressionOneMinus* OneMinus = Cast<UMaterialExpressionOneMinus>(Expression);
			return ComponentCount(OneMinus->Input.Expression, OneMinus->Input.OutputIndex);
		}
		else if (Expression->IsA<UMaterialExpressionPower>())
		{
			UMaterialExpressionPower* Power = Cast<UMaterialExpressionPower>(Expression);
			return ComponentCount(Power->Base.Expression, Power->Base.OutputIndex);
		}
		else if (Expression->IsA<UMaterialExpressionScalarParameter>())
		{
			return 1;
		}
		else if (Expression->IsA<UMaterialExpressionSine>())
		{
			// sine always returns a float... ensure that it also just gets one!
			ensure(ComponentCount(
				Cast<UMaterialExpressionSine>(Expression)->Input.Expression, OutputIndex) == 1);
			return 1;
		}
		else if (Expression->IsA<UMaterialExpressionSphereMask>())
		{
			UMaterialExpressionSphereMask* SphereMask = Cast<UMaterialExpressionSphereMask>(Expression);
			ensure(ComponentCount(SphereMask->A.Expression, SphereMask->A.OutputIndex) ==
				ComponentCount(SphereMask->B.Expression, SphereMask->B.OutputIndex));
			return 1;
		}
		else if (Expression->IsA<UMaterialExpressionSquareRoot>())
		{
			UMaterialExpressionSquareRoot* SquareRoot = Cast<UMaterialExpressionSquareRoot>(Expression);
			return ComponentCount(SquareRoot->Input.Expression, SquareRoot->Input.OutputIndex);
		}
		else if (Expression->IsA<UMaterialExpressionStaticBool>())
		{
			return 1;
		}
		else if (Expression->IsA<UMaterialExpressionStaticBoolParameter>())
		{
			return 1;
		}
		else if (Expression->IsA<UMaterialExpressionStaticSwitch>())
		{
			UMaterialExpressionStaticSwitch* StaticSwitch = Cast<UMaterialExpressionStaticSwitch>(Expression);
			const uint32 ACount = ComponentCount(StaticSwitch->A.Expression, StaticSwitch->A.OutputIndex);
			const uint32 BCount = ComponentCount(StaticSwitch->B.Expression, StaticSwitch->B.OutputIndex);
			ensure(ACount == BCount);
			return ACount;
		}
		else if (Expression->IsA<UMaterialExpressionSubtract>())
		{
			UMaterialExpressionSubtract* Subtract = Cast<UMaterialExpressionSubtract>(Expression);
			const uint32 ACount = Subtract->A.Expression ? ComponentCount(Subtract->A.Expression, Subtract->A.OutputIndex) : 1;
			const uint32 BCount = Subtract->B.Expression ? ComponentCount(Subtract->B.Expression, Subtract->B.OutputIndex) : 1;
			ensure((ACount == 1) || (BCount == 1) || (ACount == BCount));
			return FMath::Max(ACount, BCount);
		}
		else if (Expression->IsA<UMaterialExpressionTextureCoordinate>())
		{
			return 2;
		}
		else if (Expression->IsA<UMaterialExpressionTextureProperty>())
		{
			return 2;
		}
		else if (Expression->IsA<UMaterialExpressionTextureSample>())
		{
			return (OutputIndex == 0) ? 3 : 1;  // output 0 is color, the others are floats
		}
		else if (Expression->IsA<UMaterialExpressionTransform>())
		{
			UMaterialExpressionTransform* Transform = Cast<UMaterialExpressionTransform>(Expression);
			return ComponentCount(Transform->Input.Expression, Transform->Input.OutputIndex);
		}
		else if (Expression->IsA<UMaterialExpressionTransformPosition>())
		{
			UMaterialExpressionTransformPosition* TransformPosition =
				Cast<UMaterialExpressionTransformPosition>(Expression);
			return ComponentCount(TransformPosition->Input.Expression, TransformPosition->Input.OutputIndex);
		}
		else if (Expression->IsA<UMaterialExpressionTwoSidedSign>())
		{
			return 1;
		}
		else if (Expression->IsA<UMaterialExpressionVectorParameter>())
		{
			return (OutputIndex == 0) ? 3 : 1;  // output 0 is color, the others are floats
		}
		else if (Expression->IsA<UMaterialExpressionArcsine>())
		{
			// arcsine always returns a float... ensure that it also just gets one!
			ensure(ComponentCount(Cast<UMaterialExpressionArcsine>(Expression)->Input.Expression, OutputIndex) == 1);
			return 1;
		}
		else if (Expression->IsA<UMaterialExpressionVertexNormalWS>()
			|| Expression->IsA<UMaterialExpressionPixelNormalWS>()
			|| Expression->IsA<UMaterialExpressionCameraPositionWS>()
			|| Expression->IsA<UMaterialExpressionWorldPosition>())
		{
			return 3;
		}
		else if (Expression->IsA<UMaterialExpressionTime>()
			|| Expression->IsA<UMaterialExpressionPixelDepth>())
		{
			return 1;
		}
		else if (Expression->IsA<UMaterialExpressionSaturate>())
		{
			UMaterialExpressionSaturate* Saturate = Cast<UMaterialExpressionSaturate>(Expression);
			return ComponentCount(Saturate->Input.Expression);
		}
		else if (Expression->IsA<UMaterialExpressionCeil>())
		{
			UMaterialExpressionCeil* Ceil = Cast<UMaterialExpressionCeil>(Expression);
			return ComponentCount({ Ceil->Input.Expression, Ceil->Input.OutputIndex });
		}
		ensure(false);
		return 0;
	}

	uint32 ComponentCount(const FMaterialExpressionConnection& Input)
	{
		switch (Input.GetConnectionType())
		{
			case Expression:
				if (!ensure(Input.HasExpression()))
				{
					return 0;
				}
				return ComponentCount(Input.GetExpressionUnused(), Input.GetExpressionOutputIndex());

			case Boolean:
			case Float:
				return 1;
			case Float2:
				return 2;
			case Float3:
				return 3;
			case Float4:
				return 4;
			case Texture:
			default:
				ensure(false);
				return 0;
		}
	}

	template <typename ExpressionType, typename ExpressionArrayType>
	int32 CountExpressions(const ExpressionArrayType& Expressions)
	{
		int32 Count = 0;
		for (UMaterialExpression* Expression : Expressions)
		{
			if (Expression->IsA<ExpressionType>())
			{
				++Count;
			}
		}
		return Count;
	}

	UMaterialExpressionAbs* NewMaterialExpressionAbs(UObject* Parent, const FMaterialExpressionConnection& Input)
	{
		// can digest any number of components
		UMaterialExpressionAbs* Expression = NewMaterialExpression<UMaterialExpressionAbs>(Parent);
		CheckedConnect(Parent, Input, Expression->Input);
		return Expression;
	}

	UMaterialExpressionAdd* NewMaterialExpressionAdd(UObject* Parent, const FMaterialExpressionConnection& A, const FMaterialExpressionConnection& B)
	{
		uint32 ACount = ComponentCount(A);
		uint32 BCount = ComponentCount(B);
		check((ACount == 1) || (BCount == 1) || (ACount == BCount));

		UMaterialExpressionAdd* Expression = NewMaterialExpression<UMaterialExpressionAdd>(Parent);
		CheckedConnect(Parent, A, Expression->A, Expression->ConstA);
		CheckedConnect(Parent, B, Expression->B, Expression->ConstB);
		return Expression;
	}

	UMaterialExpressionAdd* NewMaterialExpressionAddRecursive(UObject* Parent, uint32 Begin, uint32 End,
	                                                          const TArray<FMaterialExpressionConnection>& Arguments)
	{
		uint32 Count = End - Begin;
		check(1 < Count);
		switch (Count)
		{
			case 2:
				return NewMaterialExpressionAdd(Parent, Arguments[Begin], Arguments[Begin + 1]);
			case 3:
				return NewMaterialExpressionAdd(Parent, NewMaterialExpressionAdd(Parent, Arguments[Begin], Arguments[Begin + 1]),
				                                Arguments[Begin + 2]);
			default:
			{
				uint32 Center = Begin + Count / 2;
				return NewMaterialExpressionAdd(Parent, NewMaterialExpressionAddRecursive(Parent, Begin, Center, Arguments),
				                                NewMaterialExpressionAddRecursive(Parent, Center, End, Arguments));
			}
		}
	}

	UMaterialExpressionAdd* NewMaterialExpressionAdd(UObject* Parent, const TArray<FMaterialExpressionConnection>& Arguments)
	{
		return NewMaterialExpressionAddRecursive(Parent, 0, Arguments.Num(), Arguments);
	}

	UMaterialExpressionAppendVector* NewMaterialExpressionAppendVector(UObject* Parent, const FMaterialExpressionConnection& A,
	                                                                   const FMaterialExpressionConnection& B)
	{
		// can digest any number of components
		UMaterialExpressionAppendVector* Expression = NewMaterialExpression<UMaterialExpressionAppendVector>(Parent);
		CheckedConnect(Parent, A, Expression->A);
		CheckedConnect(Parent, B, Expression->B);
		return Expression;
	}

	UMaterialExpressionAppendVector* NewMaterialExpressionAppendVector(UObject* Parent, const FMaterialExpressionConnection& A,
	                                                                   const FMaterialExpressionConnection& B, const FMaterialExpressionConnection& C)
	{
		return NewMaterialExpressionAppendVector(Parent, NewMaterialExpressionAppendVector(Parent, A, B), C);
	}

	UMaterialExpressionArccosine* NewMaterialExpressionArccosine(UObject* Parent, const FMaterialExpressionConnection& Input)
	{
		check(IsScalar(Input));
		UMaterialExpressionArccosine* Expression = NewMaterialExpression<UMaterialExpressionArccosine>(Parent);
		CheckedConnect(Parent, Input, Expression->Input);
		return Expression;
	}

	UMaterialExpressionArcsine* NewMaterialExpressionArcsine(UObject* Parent, const FMaterialExpressionConnection& Input)
	{
		check(IsScalar(Input));
		UMaterialExpressionArcsine* Expression = NewMaterialExpression<UMaterialExpressionArcsine>(Parent);
		CheckedConnect(Parent, Input, Expression->Input);
		return Expression;
	}

	UMaterialExpressionArctangent* NewMaterialExpressionArctangent(UObject* Parent, const FMaterialExpressionConnection& Input)
	{
		check(IsScalar(Input));
		UMaterialExpressionArctangent* Expression = NewMaterialExpression<UMaterialExpressionArctangent>(Parent);
		CheckedConnect(Parent, Input, Expression->Input);
		return Expression;
	}

	UMaterialExpressionArctangent2* NewMaterialExpressionArctangent2(UObject* Parent, const FMaterialExpressionConnection& Y,
	                                                                 const FMaterialExpressionConnection& X)
	{
		check(IsScalar(Y) && IsScalar(X));
		UMaterialExpressionArctangent2* Expression = NewMaterialExpression<UMaterialExpressionArctangent2>(Parent);
		CheckedConnect(Parent, Y, Expression->Y);
		CheckedConnect(Parent, X, Expression->X);
		return Expression;
	}

	UMaterialExpressionBlackBody* NewMaterialExpressionBlackBody(UObject* Parent, const FMaterialExpressionConnection& Temp)
	{
		check(IsScalar(Temp));  // can digest any number of components, but uses just the first
		UMaterialExpressionBlackBody* Expression = NewMaterialExpression<UMaterialExpressionBlackBody>(Parent);
		CheckedConnect(Parent, Temp, Expression->Temp);
		return Expression;
	}

	UMaterialExpressionBreakMaterialAttributes* NewMaterialExpressionBreakMaterialAttributes(UObject*                             Parent,
	                                                                                         const FMaterialExpressionConnection& MaterialAttributes)
	{
		check(IsMaterialAttribute(MaterialAttributes));
		UMaterialExpressionBreakMaterialAttributes* Expression = NewMaterialExpression<UMaterialExpressionBreakMaterialAttributes>(Parent);
		CheckedConnect(Parent, MaterialAttributes, Expression->MaterialAttributes);
		return Expression;
	}

	UMaterialExpressionCeil* NewMaterialExpressionCeil(UObject* Parent, const FMaterialExpressionConnection& Input)
	{
		// can digest any number of components
		UMaterialExpressionCeil* Expression = NewMaterialExpression<UMaterialExpressionCeil>(Parent);
		CheckedConnect(Parent, Input, Expression->Input);
		return Expression;
	}

	UMaterialExpressionClamp* NewMaterialExpressionClamp(UObject* Parent, const FMaterialExpressionConnection& Input,
	                                                     const FMaterialExpressionConnection& Min, const FMaterialExpressionConnection& Max)
	{
		uint32 InputCount = ComponentCount(Input);
		uint32 MinCount   = ComponentCount(Min);
		uint32 MaxCount   = ComponentCount(Max);
		check(((InputCount == 2) && (MinCount == 2) && (MaxCount == 2)) || ((InputCount != 2) && (MinCount != 2) && (MaxCount != 2)));

		UMaterialExpressionClamp* Expression = NewMaterialExpression<UMaterialExpressionClamp>(Parent);
		CheckedConnect(Parent, Input, Expression->Input);
		CheckedConnect(Parent, Min, Expression->Min, Expression->MinDefault);
		CheckedConnect(Parent, Max, Expression->Max, Expression->MaxDefault);
		return Expression;
	}

	UMaterialExpressionComponentMask* NewMaterialExpressionComponentMask(UObject* Parent, const FMaterialExpressionConnection& Input, uint32 Mask)
	{
		// can digest any number of components
		UMaterialExpressionComponentMask* Expression = NewMaterialExpression<UMaterialExpressionComponentMask>(Parent);
		CheckedConnect(Parent, Input, Expression->Input);
		Expression->R = !!(Mask & 1);
		Expression->G = !!(Mask & 2);
		Expression->B = !!(Mask & 4);
		Expression->A = !!(Mask & 8);
		return Expression;
	}

	UMaterialExpressionClearCoatNormalCustomOutput* NewMaterialExpressionClearCoatNormalCustomOutput(UObject*                             Parent,
	                                                                                                 const FMaterialExpressionConnection& Input)
	{
		// can digest any number of components
		UMaterialExpressionClearCoatNormalCustomOutput* Expression = NewMaterialExpression<UMaterialExpressionClearCoatNormalCustomOutput>(Parent);
		CheckedConnect(Parent, Input, Expression->Input);
		return Expression;
	}

	UMaterialExpressionConstant* NewMaterialExpressionConstant(UObject* Parent, float X)
	{
		UMaterialExpressionConstant* Expression = NewMaterialExpression<UMaterialExpressionConstant>(Parent);
		Expression->R                           = X;
		return Expression;
	}

	UMaterialExpressionConstant2Vector* NewMaterialExpressionConstant(UObject* Parent, float X, float Y)
	{
		UMaterialExpressionConstant2Vector* Expression = NewMaterialExpression<UMaterialExpressionConstant2Vector>(Parent);
		Expression->R                                  = X;
		Expression->G                                  = Y;
		return Expression;
	}

	UMaterialExpressionConstant3Vector* NewMaterialExpressionConstant(UObject* Parent, float X, float Y, float Z)
	{
		UMaterialExpressionConstant3Vector* Expression = NewMaterialExpression<UMaterialExpressionConstant3Vector>(Parent);
		Expression->Constant                           = FLinearColor(X, Y, Z);
		return Expression;
	}

	UMaterialExpressionConstant4Vector* NewMaterialExpressionConstant(UObject* Parent, float X, float Y, float Z, float W)
	{
		UMaterialExpressionConstant4Vector* Expression = NewMaterialExpression<UMaterialExpressionConstant4Vector>(Parent);
		Expression->Constant                           = FLinearColor(X, Y, Z, W);
		return Expression;
	}

	UMaterialExpressionCosine* NewMaterialExpressionCosine(UObject* Parent, const FMaterialExpressionConnection& Input)
	{
		check(IsScalar(Input));
		UMaterialExpressionCosine* Expression = NewMaterialExpression<UMaterialExpressionCosine>(Parent);
		Expression->Period                    = 2 * PI;
		CheckedConnect(Parent, Input, Expression->Input);
		return Expression;
	}

	UMaterialExpressionCrossProduct* NewMaterialExpressionCrossProduct(UObject* Parent, const FMaterialExpressionConnection& A,
	                                                                   const FMaterialExpressionConnection& B)
	{
		check(IsVector3(A) && IsVector3(B));  // can consume scalar as well, but only vector3 are meaningfull !
		UMaterialExpressionCrossProduct* Expression = NewMaterialExpression<UMaterialExpressionCrossProduct>(Parent);
		CheckedConnect(Parent, A, Expression->A);
		CheckedConnect(Parent, B, Expression->B);
		return Expression;
	}

	UMaterialExpressionDistance* NewMaterialExpressionDistance(UObject* Parent, const FMaterialExpressionConnection& A,
	                                                           const FMaterialExpressionConnection& B)
	{
		check(ComponentCount(A) == ComponentCount(B));
		UMaterialExpressionDistance* Expression = NewMaterialExpression<UMaterialExpressionDistance>(Parent);
		CheckedConnect(Parent, A, Expression->A);
		CheckedConnect(Parent, B, Expression->B);
		return Expression;
	}

	UMaterialExpressionDivide* NewMaterialExpressionDivide(UObject* Parent, const FMaterialExpressionConnection& A,
	                                                       const FMaterialExpressionConnection& B)
	{
		uint32 ACount = ComponentCount(A);
		uint32 BCount = ComponentCount(B);
		check((ACount == 1) || (BCount == 1) || (ACount == BCount));

		UMaterialExpressionDivide* Expression = NewMaterialExpression<UMaterialExpressionDivide>(Parent);
		CheckedConnect(Parent, A, Expression->A, Expression->ConstA);
		CheckedConnect(Parent, B, Expression->B, Expression->ConstB);
		return Expression;
	}

	UMaterialExpressionDotProduct* NewMaterialExpressionDotProduct(UObject* Parent, const FMaterialExpressionConnection& A,
	                                                               const FMaterialExpressionConnection& B)
	{
		check(ComponentCount(A) == ComponentCount(B));
		UMaterialExpressionDotProduct* Expression = NewMaterialExpression<UMaterialExpressionDotProduct>(Parent);
		CheckedConnect(Parent, A, Expression->A);
		CheckedConnect(Parent, B, Expression->B);
		return Expression;
	}

	UMaterialExpressionFloor* NewMaterialExpressionFloor(UObject* Parent, const FMaterialExpressionConnection& Input)
	{
		// can digest any number of components
		UMaterialExpressionFloor* Expression = NewMaterialExpression<UMaterialExpressionFloor>(Parent);
		CheckedConnect(Parent, Input, Expression->Input);
		return Expression;
	}

	UMaterialExpressionFmod* NewMaterialExpressionFmod(UObject* Parent, const FMaterialExpressionConnection& A,
	                                                   const FMaterialExpressionConnection& B)
	{
		check(ComponentCount(A) == ComponentCount(B));
		UMaterialExpressionFmod* Expression = NewMaterialExpression<UMaterialExpressionFmod>(Parent);
		CheckedConnect(Parent, A, Expression->A);
		CheckedConnect(Parent, B, Expression->B);
		return Expression;
	}

	UMaterialExpressionFrac* NewMaterialExpressionFrac(UObject* Parent, const FMaterialExpressionConnection& Input)
	{
		// can digest any number of components
		UMaterialExpressionFrac* Expression = NewMaterialExpression<UMaterialExpressionFrac>(Parent);
		CheckedConnect(Parent, Input, Expression->Input);
		return Expression;
	}

	UMaterialExpressionFresnel* NewMaterialExpressionFresnel(UObject* Parent, const FMaterialExpressionConnection& Exponent,
	                                                         const FMaterialExpressionConnection& BaseReflectFraction,
	                                                         const FMaterialExpressionConnection& Normal)
	{
		check(IsScalar(Exponent) && IsScalar(BaseReflectFraction) && IsVector3(Normal));
		UMaterialExpressionFresnel* Expression = NewMaterialExpression<UMaterialExpressionFresnel>(Parent);
		CheckedConnect(Parent, Exponent, Expression->ExponentIn, Expression->Exponent);
		CheckedConnect(Parent, BaseReflectFraction, Expression->BaseReflectFractionIn, Expression->BaseReflectFraction);
		CheckedConnect(Parent, Normal, Expression->Normal);
		return Expression;
	}

	UMaterialExpressionFunctionInput* NewMaterialExpressionFunctionInput(UObject* Parent, const FString& Name, EFunctionInputType Type)
	{
		check(Parent->IsA<UMaterialFunction>());

		UMaterialExpressionFunctionInput* Expression = NewMaterialExpression<UMaterialExpressionFunctionInput>(Parent);
		Expression->Id                               = FGuid::NewGuid();
		Expression->InputName                        = *Name;
		Expression->InputType                        = Type;
		Expression->SortPriority = CountExpressions<UMaterialExpressionFunctionInput>(Cast<UMaterialFunction>(Parent)->GetExpressions());
		Expression->bCollapsed   = true;
		return Expression;
	}

	UMaterialExpressionFunctionInput* NewMaterialExpressionFunctionInput(UObject* Parent, const FString& Name, EFunctionInputType Type,
	                                                                     const FMaterialExpressionConnection& DefaultExpression)
	{
		UMaterialExpressionFunctionInput* Expression = NewMaterialExpressionFunctionInput(Parent, Name, Type);
		Expression->bUsePreviewValueAsDefault        = true;
		CheckedConnect(Parent, DefaultExpression, Expression->Preview);
		return Expression;
	}

	UMaterialExpressionFunctionOutput* NewMaterialExpressionFunctionOutput(UObject* Parent, const FString& Name,
	                                                                       const FMaterialExpressionConnection& Output)
	{
		check(Parent->IsA<UMaterialFunction>());

		UMaterialExpressionFunctionOutput* Expression = NewMaterialExpression<UMaterialExpressionFunctionOutput>(Parent);
		Expression->Id                                = FGuid::NewGuid();
		Expression->OutputName                        = *Name;
		CheckedConnect(Parent, Output, Expression->A);
		Expression->SortPriority = CountExpressions<UMaterialExpressionFunctionOutput>(Cast<UMaterialFunction>(Parent)->GetExpressions());
		Expression->bCollapsed   = true;
		return Expression;
	}

	UMaterialExpressionIf* NewMaterialExpressionIf(UObject* Parent, const FMaterialExpressionConnection& A, const FMaterialExpressionConnection& B,
	                                               const FMaterialExpressionConnection& Less, const FMaterialExpressionConnection& Equal,
	                                               const FMaterialExpressionConnection& Greater)
	{
		ensure(IsScalar(A) && IsScalar(B) &&
		      (((ComponentCount(Less) == ComponentCount(Greater)) &&
		        (!Equal.GetExpressionUnused() || (ComponentCount(Less) == ComponentCount(Equal))))));
		UMaterialExpressionIf* Expression = NewMaterialExpression<UMaterialExpressionIf>(Parent);
		CheckedConnect(Parent, A, Expression->A);
		CheckedConnect(Parent, B, Expression->B, Expression->ConstB);
		CheckedConnect(Parent, Less, Expression->ALessThanB);
		CheckedConnect(Parent, Equal, Expression->AEqualsB);
		CheckedConnect(Parent, Greater, Expression->AGreaterThanB);
		return Expression;
	}

	UMaterialExpressionIf* NewMaterialExpressionIfEqual(UObject* Parent, const FMaterialExpressionConnection& A,
	                                                    const FMaterialExpressionConnection& B, const FMaterialExpressionConnection& Yes,
	                                                    const FMaterialExpressionConnection& No)
	{
		if (B.GetConnectionType() == EConnectionType::Float3)
		{
			return NewMaterialExpressionIfEqual(
			    Parent, NewMaterialExpressionComponentMask(Parent, A, 1), B.GetVectorValue()[0],
			    NewMaterialExpressionIfEqual(
			        Parent, NewMaterialExpressionComponentMask(Parent, A, 2), B.GetVectorValue()[1],
			        NewMaterialExpressionIfEqual(Parent, NewMaterialExpressionComponentMask(Parent, A, 4), B.GetVectorValue()[2], Yes, No), No),
			    No);
		}
		else
		{
			return NewMaterialExpressionIf(Parent, A, B, No, Yes, No);
		}
	}

	UMaterialExpressionIf* NewMaterialExpressionIfGreater(UObject* Parent, const FMaterialExpressionConnection& A,
	                                                      const FMaterialExpressionConnection& B, const FMaterialExpressionConnection& Yes,
	                                                      const FMaterialExpressionConnection& No)
	{
		return NewMaterialExpressionIf(Parent, A, B, No, No, Yes);
	}

	UMaterialExpressionIf* NewMaterialExpressionIfLess(UObject* Parent, const FMaterialExpressionConnection& A,
	                                                   const FMaterialExpressionConnection& B, const FMaterialExpressionConnection& Yes,
	                                                   const FMaterialExpressionConnection& No)
	{
		return NewMaterialExpressionIf(Parent, A, B, Yes, No, No);
	}

	UMaterialExpressionIf* NewMaterialExpressionSwitchRecursive(UObject* Parent, const FMaterialExpressionConnection& Switch, uint32 Begin,
	                                                            uint32 End, const TArray<FMaterialExpressionConnection>& Cases)
	{
		uint32 Count = End - Begin;
		check(1 < Count);
		switch (Count)
		{
			case 2:
				return NewMaterialExpressionIfEqual(Parent, Switch, static_cast<float>(Begin), Cases[Begin], Cases[Begin + 1]);
			case 3:
				return NewMaterialExpressionIf(Parent, Switch, static_cast<float>(Begin + 1), Cases[Begin], Cases[Begin + 1], Cases[Begin + 2]);
			default:
			{
				uint32 Center = Begin + Count / 2;
				return NewMaterialExpressionIf(
				    Parent, Switch, static_cast<float>(Center), NewMaterialExpressionSwitchRecursive(Parent, Switch, Begin, Center, Cases),
				    Cases[Center],
				    (End - Center == 2) ? Cases[Center + 1] : NewMaterialExpressionSwitchRecursive(Parent, Switch, Center + 1, End, Cases));
			}
		}
	}

	UMaterialExpressionIf* NewMaterialExpressionSwitch(UObject* Parent, const FMaterialExpressionConnection& Switch,
	                                                   const TArray<FMaterialExpressionConnection>& Cases)
	{
		return NewMaterialExpressionSwitchRecursive(Parent, Switch, 0, Cases.Num(), Cases);
	}

	UMaterialExpressionLinearInterpolate* NewMaterialExpressionLinearInterpolate(UObject* Parent, const FMaterialExpressionConnection& A,
	                                                                             const FMaterialExpressionConnection& B,
	                                                                             const FMaterialExpressionConnection& Alpha)
	{
		uint32 ACount     = ComponentCount(A);
		uint32 BCount     = ComponentCount(B);
		uint32 AlphaCount = ComponentCount(Alpha);
		ensure(((ACount == 1) || (BCount == 1) || (ACount == BCount)) && ((AlphaCount == 1) || ((AlphaCount == ACount) && (AlphaCount == BCount))));

		UMaterialExpressionLinearInterpolate* Expression = NewMaterialExpression<UMaterialExpressionLinearInterpolate>(Parent);
		CheckedConnect(Parent, A, Expression->A, Expression->ConstA);
		CheckedConnect(Parent, B, Expression->B, Expression->ConstB);
		CheckedConnect(Parent, Alpha, Expression->Alpha, Expression->ConstAlpha);
		return Expression;
	}

	UMaterialExpression* NewMaterialExpressionLinearInterpolateRecursive(UObject* Function, const FMaterialExpressionConnection& Alpha, int32 Begin,
	                                                                     int32 End, const TArray<FMaterialExpressionConnection>& Values,
	                                                                     const FMaterialExpressionConnection& Weight)
	{
		check(Begin < End);

		int32 Center = (Begin + End) / 2;
		int32 Count  = End - Begin;
		check(1 < Count);
		switch (Count)
		{
			case 2:
				return NewMaterialExpressionLinearInterpolate(Function, Values[Begin], Values[Begin + 1], Weight);
			case 3:
				return NewMaterialExpressionIf(Function, Alpha, static_cast<float>(Center),
				                               NewMaterialExpressionLinearInterpolate(Function, Values[Begin], Values[Begin + 1], Weight),
				                               Values[Center],
				                               NewMaterialExpressionLinearInterpolate(Function, Values[Center], Values[Center + 1], Weight));
			default:
				return NewMaterialExpressionIf(Function, Alpha, static_cast<float>(Center),
				                               NewMaterialExpressionLinearInterpolateRecursive(Function, Alpha, Begin, Center + 1, Values, Weight),
				                               Values[Center],
				                               NewMaterialExpressionLinearInterpolateRecursive(Function, Alpha, Center, End, Values, Weight));
		}
	}

	UMaterialExpression* NewMaterialExpressionLinearInterpolate(UObject* Function, const TArray<FMaterialExpressionConnection>& Values,
	                                                            const FMaterialExpressionConnection& Alpha)
	{
		check(1 < Values.Num());
		return NewMaterialExpressionLinearInterpolateRecursive(Function, Alpha, 0, Values.Num(), Values, NewMaterialExpressionFrac(Function, Alpha));
	}

	UMaterialExpressionLogarithm2* NewMaterialExpressionLogarithm2(UObject* Parent, const FMaterialExpressionConnection& X)
	{
		ensure(IsScalar(X));
		UMaterialExpressionLogarithm2* Expression = NewMaterialExpression<UMaterialExpressionLogarithm2>(Parent);
		CheckedConnect(Parent, X, Expression->X);
		return Expression;
	}

	UMaterialExpressionMaterialFunctionCall* NewMaterialExpressionFunctionCall(UObject* Parent, UMaterialFunction* Function,
	                                                                           const TArray<FMaterialExpressionConnection>& Inputs)
	{
		check(Parent && Function);

		UMaterialExpressionMaterialFunctionCall* Expression = NewMaterialExpression<UMaterialExpressionMaterialFunctionCall>(Parent);
		Expression->SetMaterialFunction(Function);

		if (Expression->FunctionInputs.Num() < Inputs.Num())
		{
			UE_LOGF(LogMDLImporter, Warning, "Function <%ls> received too many inputs - expected %d, but got %d!\n", *(Function->GetName()), Expression->FunctionInputs.Num(), Inputs.Num());
		}
		else
		{
			for (int32 i = 0; i < Inputs.Num(); i++)
			{
				if ((Inputs[i].GetConnectionType() != EConnectionType::Expression) || !Inputs[i].IsExpressionDefault())
				{
					CheckedConnect(Parent, Inputs[i], Expression->FunctionInputs[i].Input);
				}
			}
		}
		
		Expression->UpdateFromFunctionResource();
		return Expression;
	}

	UMaterialExpressionMakeMaterialAttributes* NewMaterialExpressionMakeMaterialAttributes(
	    UObject* Parent, const FMaterialExpressionConnection& BaseColor, const FMaterialExpressionConnection& Metallic,
	    const FMaterialExpressionConnection& Specular, const FMaterialExpressionConnection& Roughness,
	    const FMaterialExpressionConnection& EmissiveColor, const FMaterialExpressionConnection& Opacity,
	    const FMaterialExpressionConnection& OpacityMask, const FMaterialExpressionConnection& Normal,
	    const FMaterialExpressionConnection& WorldPositionOffset, const FMaterialExpressionConnection& SubsurfaceColor,
	    const FMaterialExpressionConnection& ClearCoat, const FMaterialExpressionConnection& ClearCoatRoughness,
	    const FMaterialExpressionConnection& AmbientOcclusion, const FMaterialExpressionConnection& Refraction,
	    const FMaterialExpressionConnection& CustomizedUVs0, const FMaterialExpressionConnection& CustomizedUVs1,
	    const FMaterialExpressionConnection& CustomizedUVs2, const FMaterialExpressionConnection& CustomizedUVs3,
	    const FMaterialExpressionConnection& CustomizedUVs4, const FMaterialExpressionConnection& CustomizedUVs5,
	    const FMaterialExpressionConnection& CustomizedUVs6, const FMaterialExpressionConnection& CustomizedUVs7,
	    const FMaterialExpressionConnection& PixelDepthOffset)
	{
		check(IsScalar(Metallic) && IsScalar(Specular) && IsScalar(Roughness) && IsScalar(Opacity) && IsScalar(OpacityMask) &&
		      IsScalar(ClearCoat) && IsScalar(ClearCoatRoughness) && IsScalar(AmbientOcclusion) &&
		      IsScalar(PixelDepthOffset));

		UMaterialExpressionMakeMaterialAttributes* Expression = NewMaterialExpression<UMaterialExpressionMakeMaterialAttributes>(Parent);

		CheckedConnect(Parent, BaseColor, Expression->BaseColor);
		CheckedConnect(Parent, Metallic, Expression->Metallic);
		CheckedConnect(Parent, Specular, Expression->Specular);
		CheckedConnect(Parent, Roughness, Expression->Roughness);
		CheckedConnect(Parent, EmissiveColor, Expression->EmissiveColor);
		CheckedConnect(Parent, Opacity, Expression->Opacity);
		CheckedConnect(Parent, OpacityMask, Expression->OpacityMask);
		CheckedConnect(Parent, Normal, Expression->Normal);
		CheckedConnect(Parent, WorldPositionOffset, Expression->WorldPositionOffset);
		CheckedConnect(Parent, SubsurfaceColor, Expression->SubsurfaceColor);
		CheckedConnect(Parent, ClearCoat, Expression->ClearCoat);
		CheckedConnect(Parent, ClearCoatRoughness, Expression->ClearCoatRoughness);
		CheckedConnect(Parent, AmbientOcclusion, Expression->AmbientOcclusion);
		CheckedConnect(Parent, Refraction, Expression->Refraction);
		CheckedConnect(Parent, CustomizedUVs0, Expression->CustomizedUVs[0]);
		CheckedConnect(Parent, CustomizedUVs1, Expression->CustomizedUVs[1]);
		CheckedConnect(Parent, CustomizedUVs2, Expression->CustomizedUVs[2]);
		CheckedConnect(Parent, CustomizedUVs3, Expression->CustomizedUVs[3]);
		CheckedConnect(Parent, CustomizedUVs4, Expression->CustomizedUVs[4]);
		CheckedConnect(Parent, CustomizedUVs5, Expression->CustomizedUVs[5]);
		CheckedConnect(Parent, CustomizedUVs6, Expression->CustomizedUVs[6]);
		CheckedConnect(Parent, CustomizedUVs7, Expression->CustomizedUVs[7]);
		CheckedConnect(Parent, PixelDepthOffset, Expression->PixelDepthOffset);
		return Expression;
	}

	UMaterialExpressionMax* NewMaterialExpressionMax(UObject* Parent, const FMaterialExpressionConnection& A, const FMaterialExpressionConnection& B)
	{
		uint32 ACount = ComponentCount(A);
		uint32 BCount = ComponentCount(B);
		check((ACount == 1) || (BCount == 1) || (ACount == BCount));

		UMaterialExpressionMax* Expression = NewMaterialExpression<UMaterialExpressionMax>(Parent);
		CheckedConnect(Parent, A, Expression->A, Expression->ConstA);
		CheckedConnect(Parent, B, Expression->B, Expression->ConstB);
		return Expression;
	}

	UMaterialExpressionMax* NewMaterialExpressionMax(UObject* Parent, const FMaterialExpressionConnection& A, const FMaterialExpressionConnection& B,
	                                                 const FMaterialExpressionConnection& C)
	{
		return NewMaterialExpressionMax(Parent, NewMaterialExpressionMax(Parent, A, B), C);
	}

	UMaterialExpressionMin* NewMaterialExpressionMin(UObject* Parent, const FMaterialExpressionConnection& A, const FMaterialExpressionConnection& B)
	{
		uint32 ACount = ComponentCount(A);
		uint32 BCount = ComponentCount(B);
		check((ACount == 1) || (BCount == 1) || (ACount == BCount));

		UMaterialExpressionMin* Expression = NewMaterialExpression<UMaterialExpressionMin>(Parent);
		CheckedConnect(Parent, A, Expression->A, Expression->ConstA);
		CheckedConnect(Parent, B, Expression->B, Expression->ConstB);
		return Expression;
	}

	UMaterialExpressionMin* NewMaterialExpressionMin(UObject* Parent, const FMaterialExpressionConnection& A, const FMaterialExpressionConnection& B,
	                                                 const FMaterialExpressionConnection& C)
	{
		return NewMaterialExpressionMin(Parent, NewMaterialExpressionMin(Parent, A, B), C);
	}

	UMaterialExpressionMultiply* NewMaterialExpressionMultiply(UObject* Parent, const FMaterialExpressionConnection& A,
	                                                           const FMaterialExpressionConnection& B)
	{
		uint32 ACount = ComponentCount(A);
		uint32 BCount = ComponentCount(B);
		UMaterialExpressionMultiply* Expression = NewMaterialExpression<UMaterialExpressionMultiply>(Parent);
		
		if (ensure((ACount == 1) || (BCount == 1) || (ACount == BCount)))
		{
			CheckedConnect(Parent, A, Expression->A, Expression->ConstA);
			CheckedConnect(Parent, B, Expression->B, Expression->ConstB);
		}

		return Expression;
	}

	UMaterialExpressionMultiply* NewMaterialExpressionMultiplyRecursive(UObject* Parent, uint32 Begin, uint32 End,
	                                                                    const TArray<FMaterialExpressionConnection>& Arguments)
	{
		uint32 Count = End - Begin;
		check(1 < Count);
		switch (Count)
		{
			case 2:
				return NewMaterialExpressionMultiply(Parent, Arguments[Begin], Arguments[Begin + 1]);
			case 3:
				return NewMaterialExpressionMultiply(Parent, NewMaterialExpressionMultiply(Parent, Arguments[Begin], Arguments[Begin + 1]),
				                                     Arguments[Begin + 2]);
			default:
			{
				uint32 Center = Begin + Count / 2;
				return NewMaterialExpressionMultiply(Parent, NewMaterialExpressionMultiplyRecursive(Parent, Begin, Center, Arguments),
				                                     NewMaterialExpressionMultiplyRecursive(Parent, Center, End, Arguments));
			}
		}
	}

	UMaterialExpressionMultiply* NewMaterialExpressionMultiply(UObject* Parent, const TArray<FMaterialExpressionConnection>& Arguments)
	{
		return NewMaterialExpressionMultiplyRecursive(Parent, 0, Arguments.Num(), Arguments);
	}

	UMaterialExpressionMultiply* NewMaterialExpressionSquare(UObject* Parent, const FMaterialExpressionConnection& A)
	{
		return NewMaterialExpressionMultiply(Parent, A, A);
	}

	UMaterialExpressionNoise* NewMaterialExpressionNoise(UObject* Parent, const FMaterialExpressionConnection& Position, int32 Quality)
	{
		uint32 Count = ComponentCount(Position);
		check((Count == 1) || (Count == 3));
		UMaterialExpressionNoise* Expression = NewMaterialExpression<UMaterialExpressionNoise>(Parent);
		CheckedConnect(Parent, Position, Expression->Position);
		Expression->Quality = Quality;
		return Expression;
	}

	UMaterialExpressionVectorNoise* NewMaterialExpressionVectorNoise(UObject* Parent, const FMaterialExpressionConnection& Position, int32 Quality)
	{
		uint32 Count = ComponentCount(Position);
		check((Count == 1) || (Count == 3));
		UMaterialExpressionVectorNoise* Expression = NewMaterialExpression<UMaterialExpressionVectorNoise>(Parent);
		CheckedConnect(Parent, Position, Expression->Position);
		Expression->Quality = Quality;
		return Expression;
	}

	UMaterialExpressionNormalize* NewMaterialExpressionNormalize(UObject* Parent, const FMaterialExpressionConnection& Input)
	{
		// digests any dimensionality on Input
		UMaterialExpressionNormalize* Expression = NewMaterialExpression<UMaterialExpressionNormalize>(Parent);
		CheckedConnect(Parent, Input, Expression->VectorInput);
		return Expression;
	}

	UMaterialExpressionOneMinus* NewMaterialExpressionOneMinus(UObject* Parent, const FMaterialExpressionConnection& Input)
	{
		// digests any dimensionality on Input
		UMaterialExpressionOneMinus* Expression = NewMaterialExpression<UMaterialExpressionOneMinus>(Parent);
		CheckedConnect(Parent, Input, Expression->Input);
		return Expression;
	}

	template <typename T>
	T* NewMaterialExpressionParameter(UObject* Parent, const FString& Name)
	{
		T* Expression              = NewMaterialExpression<T>(Parent);
		Expression->ExpressionGUID = FGuid::NewGuid();
		Expression->ParameterName  = *Name;
		return Expression;
	}

	UMaterialExpressionPower* NewMaterialExpressionPower(UObject* Parent, const FMaterialExpressionConnection& Base,
	                                                     const FMaterialExpressionConnection& Exponent)
	{
		uint32 BaseCount     = ComponentCount(Base);
		uint32 ExponentCount = ComponentCount(Exponent);
		check((BaseCount == 1) || (ExponentCount == 1) || (BaseCount == ExponentCount));

		UMaterialExpressionPower* Expression = NewMaterialExpression<UMaterialExpressionPower>(Parent);
		CheckedConnect(Parent, Base, Expression->Base);
		CheckedConnect(Parent, Exponent, Expression->Exponent, Expression->ConstExponent);
		return Expression;
	}

	UMaterialExpressionReflectionVectorWS* NewMaterialExpressionReflectionVectorWS(UObject*                             Parent,
	                                                                               const FMaterialExpressionConnection& CustomWorldNormal)
	{
		check(ComponentCount(CustomWorldNormal) != 2);
		UMaterialExpressionReflectionVectorWS* Expression = NewMaterialExpression<UMaterialExpressionReflectionVectorWS>(Parent);
		CheckedConnect(Parent, CustomWorldNormal, Expression->CustomWorldNormal);
		return Expression;
	}

	UMaterialExpressionSaturate* NewMaterialExpressionSaturate(UObject* Parent, const FMaterialExpressionConnection& Input)
	{
		// digests any dimensionality on Input
		UMaterialExpressionSaturate* Expression = NewMaterialExpression<UMaterialExpressionSaturate>(Parent);
		CheckedConnect(Parent, Input, Expression->Input);
		return Expression;
	}

	UMaterialExpressionScalarParameter* NewMaterialExpressionScalarParameter(UObject* Parent, const FString& Name, float DefaultValue)
	{
		UMaterialExpressionScalarParameter* Expression = NewMaterialExpressionParameter<UMaterialExpressionScalarParameter>(Parent, Name);
		Expression->DefaultValue                       = DefaultValue;
		return Expression;
	}

	UMaterialExpressionSine* NewMaterialExpressionSine(UObject* Parent, const FMaterialExpressionConnection& Input)
	{
		check(IsScalar(Input));
		UMaterialExpressionSine* Expression = NewMaterialExpression<UMaterialExpressionSine>(Parent);
		Expression->Period                  = 2 * PI;
		CheckedConnect(Parent, Input, Expression->Input);
		return Expression;
	}

	UMaterialExpressionSquareRoot* NewMaterialExpressionSquareRoot(UObject* Parent, const FMaterialExpressionConnection& Input)
	{
		// digests any dimensionality on Input
		UMaterialExpressionSquareRoot* Expression = NewMaterialExpression<UMaterialExpressionSquareRoot>(Parent);
		CheckedConnect(Parent, Input, Expression->Input);
		return Expression;
	}

	UMaterialExpressionStaticBool* NewMaterialExpressionStaticBool(UObject* Parent, bool Value)
	{
		UMaterialExpressionStaticBool* Expression = NewMaterialExpression<UMaterialExpressionStaticBool>(Parent);
		Expression->Value                         = Value;
		return Expression;
	}

	UMaterialExpressionStaticBoolParameter* NewMaterialExpressionStaticBoolParameter(UObject* Parent, const FString& Name, bool bDefaultValue,
	                                                                                 const FString& Group)
	{
		UMaterialExpressionStaticBoolParameter* Expression = NewMaterialExpressionParameter<UMaterialExpressionStaticBoolParameter>(Parent, Name);
		Expression->DefaultValue                           = bDefaultValue ? 1 : 0;
		Expression->Group                                  = *Group;
		return Expression;
	}

	UMaterialExpressionStaticSwitch* NewMaterialExpressionStaticSwitch(UObject* Parent, const FMaterialExpressionConnection& Value,
	                                                                   const FMaterialExpressionConnection& A, const FMaterialExpressionConnection& B)
	{
		check(IsBool(Value) &&
		      ((IsBool(A) && IsBool(B)) || (IsMaterialAttribute(A) && IsMaterialAttribute(B)) || (ComponentCount(A) == ComponentCount(B))));
		UMaterialExpressionStaticSwitch* Expression = NewMaterialExpression<UMaterialExpressionStaticSwitch>(Parent);
		CheckedConnect(Parent, A, Expression->A);
		CheckedConnect(Parent, B, Expression->B);

		// needs some special coding, as DefaultValue is not a float!
		if (Value.HasExpression())
		{
			Expression->Value.Connect(Value.GetExpressionOutputIndex(), Value.GetExpressionAndUse());
		}
		else
		{
			check(Value.GetConnectionType() == EConnectionType::Float);
			Expression->DefaultValue = !!Value.GetVectorValue()[0];
		}
		return Expression;
	}

	UMaterialExpressionSubtract* NewMaterialExpressionNegate(UObject* Parent, const FMaterialExpressionConnection& Input)
	{
		return NewMaterialExpressionSubtract(Parent, 0.0f, Input);
	}

	UMaterialExpressionSubtract* NewMaterialExpressionSubtract(UObject* Parent, const FMaterialExpressionConnection& A,
	                                                           const FMaterialExpressionConnection& B)
	{
		uint32 ACount = ComponentCount(A);
		uint32 BCount = ComponentCount(B);
		check((ACount == 1) || (BCount == 1) || (ACount == BCount));

		UMaterialExpressionSubtract* Expression = NewMaterialExpression<UMaterialExpressionSubtract>(Parent);
		CheckedConnect(Parent, A, Expression->A, Expression->ConstA);
		CheckedConnect(Parent, B, Expression->B, Expression->ConstB);
		return Expression;
	}

	UMaterialExpressionSubtract* NewMaterialExpressionSubtract(UObject* Parent, const FMaterialExpressionConnection& A,
	                                                           const FMaterialExpressionConnection& B, const FMaterialExpressionConnection& C)
	{
		return NewMaterialExpressionSubtract(Parent, NewMaterialExpressionSubtract(Parent, A, B), C);
	}

	template <typename T>
	T* NewMaterialExpressionTextureType(UObject* Parent, UTexture* Texture)
	{
		T* Expression = NewMaterialExpression<T>(Parent);

		if (Texture)
		{
			Expression->Texture = Texture;
			Expression->SamplerType = MaterialExpressionUtils::GetSamplerTypeForTexture(Texture);
		}

		return Expression;
	}

	UMaterialExpressionTangent* NewMaterialExpressionTangent(UObject* Parent, const FMaterialExpressionConnection& Input)
	{
		check(IsScalar(Input));
		UMaterialExpressionTangent* Expression = NewMaterialExpression<UMaterialExpressionTangent>(Parent);
		CheckedConnect(Parent, Input, Expression->Input);
		return Expression;
	}

	UMaterialExpressionTextureCoordinate* NewMaterialExpressionTextureCoordinate(UObject* Parent, int32 CoordinateIndex)
	{
		UMaterialExpressionTextureCoordinate* Expression = NewMaterialExpression<UMaterialExpressionTextureCoordinate>(Parent);
		Expression->CoordinateIndex                      = CoordinateIndex;
		return Expression;
	}

	UMaterialExpressionTextureObject* NewMaterialExpressionTextureObject(UObject* Parent, UTexture* Texture)
	{
		return NewMaterialExpressionTextureType<UMaterialExpressionTextureObject>(Parent, Texture);
	}

	UMaterialExpressionTextureObjectParameter* NewMaterialExpressionTextureObjectParameter(UObject* Parent, const FString& Name, UTexture* Texture)
	{
		UMaterialExpressionTextureObjectParameter* Expression =
		    NewMaterialExpressionTextureType<UMaterialExpressionTextureObjectParameter>(Parent, Texture);
		Expression->ParameterName = *Name;
		return Expression;
	}

	UMaterialExpressionTextureProperty* NewMaterialExpressionTextureProperty(UObject* Parent, const FMaterialExpressionConnection& TextureObject,
	                                                                         EMaterialExposedTextureProperty Property)
	{
		check(IsTexture(TextureObject));
		UMaterialExpressionTextureProperty* Expression = NewMaterialExpression<UMaterialExpressionTextureProperty>(Parent);
		CheckedConnect(Parent, TextureObject, Expression->TextureObject);
		Expression->Property = Property;
		return Expression;
	}

	UMaterialExpressionTextureSample* NewMaterialExpressionTextureSample(UObject* Parent, const FMaterialExpressionConnection& TextureObject,
	                                                                     const FMaterialExpressionConnection& Coordinates)
	{
		check(IsTexture(TextureObject) && ((!Coordinates.GetExpressionUnused() || ComponentCount(Coordinates) < 3)));
		UMaterialExpressionTextureSample* Expression = NewMaterialExpression<UMaterialExpressionTextureSample>(Parent);
		CheckedConnect(Parent, TextureObject, Expression->TextureObject, &Expression->Texture);
		CheckedConnect(Parent, Coordinates, Expression->Coordinates);
		return Expression;
	}

	UMaterialExpressionTransform* NewMaterialExpressionTransform(UObject* Parent, const FMaterialExpressionConnection& Input,
	                                                             EMaterialVectorCoordTransformSource SourceType,
	                                                             EMaterialVectorCoordTransform       DestinationType)
	{
		check(2 < ComponentCount(Input));
		UMaterialExpressionTransform* Expression = NewMaterialExpression<UMaterialExpressionTransform>(Parent);
		CheckedConnect(Parent, Input, Expression->Input);
		Expression->TransformSourceType = SourceType;
		Expression->TransformType       = DestinationType;
		return Expression;
	}

	UMaterialExpressionTransformPosition* NewMaterialExpressionTransformPosition(UObject* Parent, const FMaterialExpressionConnection& Input,
	                                                                             EMaterialPositionTransformSource SourceType,
	                                                                             EMaterialPositionTransformSource DestinationType)
	{
		check(2 < ComponentCount(Input));
		UMaterialExpressionTransformPosition* Expression = NewMaterialExpression<UMaterialExpressionTransformPosition>(Parent);
		CheckedConnect(Parent, Input, Expression->Input);
		Expression->TransformSourceType = SourceType;
		Expression->TransformType       = DestinationType;
		return Expression;
	}

	UMaterialExpressionTwoSidedSign* NewMaterialExpressionTwoSidedSign(UObject* Parent)
	{
		return NewMaterialExpression<UMaterialExpressionTwoSidedSign>(Parent);
	}

	UMaterialExpressionVectorNoise* NewMaterialExpressionVectorNoise(UObject* Parent, const FMaterialExpressionConnection& Position,
	                                                                 EVectorNoiseFunction NoiseFunction, int32 Quality)
	{
		check(ComponentCount(Position) != 2);
		UMaterialExpressionVectorNoise* Expression = NewMaterialExpression<UMaterialExpressionVectorNoise>(Parent);
		CheckedConnect(Parent, Position, Expression->Position);
		Expression->NoiseFunction = NoiseFunction;
		Expression->Quality       = Quality;
		return Expression;
	}

	UMaterialExpressionVectorParameter* NewMaterialExpressionVectorParameter(UObject* Parent, const FString& Name, const FLinearColor& DefaultValue)
	{
		UMaterialExpressionVectorParameter* Expression = NewMaterialExpressionParameter<UMaterialExpressionVectorParameter>(Parent, Name);
		Expression->DefaultValue                       = DefaultValue;
		return Expression;
	}

}  // namespace Generator
