// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/MaterialExpression.h"

class UTexture;

namespace Generator
{
	enum EConnectionType
	{
		Expression,
		Boolean,
		Float,
		Float2,
		Float3,
		Float4,
		Texture,
		TextureSelection
	};

	// Controls usage count of an expression
	struct FSharedMaterialExpression: FNoncopyable
	{
		UMaterialExpression* Expression;
		mutable int32 UserCount;
		FSharedMaterialExpression(UMaterialExpression* InExpression): Expression(InExpression), UserCount(0){}

		void DestroyExpression()
		{
			if (Expression)
			{
				Expression->ConditionalBeginDestroy();
				Expression = nullptr;
			}
		}
	};

	// 
	struct FMaterialExpressionHandle
	{
		TSharedPtr<FSharedMaterialExpression> Expression;

		FMaterialExpressionHandle()
		{
		}

		FMaterialExpressionHandle(UMaterialExpression* InExpression)
		{
			Expression = MakeShared<FSharedMaterialExpression>(InExpression);
		}

		UMaterialExpression* GetMaterialExpression() const
		{
			return Expression ? Expression->Expression : nullptr;
		}

		void AddUser() const
		{
			if (Expression)
			{
				Expression->UserCount++;
			}
		}

		void DestroyExpression()
		{
			if (Expression)
			{
				Expression->DestroyExpression();
			}
		}
	};

	struct FMaterialExpressionConnection
	{
		struct FData
		{
			FData();
			FData(FMaterialExpressionHandle ExpressionHandle, int32 Index, bool bIsDefault);

			bool operator!=(const FData& rhs);

			UMaterialExpression* GetMaterialExpression() const
			{
				return ExpressionHandle.GetMaterialExpression();
			}

			FMaterialExpressionHandle ExpressionHandle;
			int32                Index;
			bool                 bIsDefault;
		};

		FMaterialExpressionConnection();
		FMaterialExpressionConnection(UMaterialExpression* Expression);
		FMaterialExpressionConnection(FMaterialExpressionHandle ExpressionHandle, int32 OutputIndex = 0, bool bIsDefault = false);
		FMaterialExpressionConnection(bool bValue);
		FMaterialExpressionConnection(int Value);
		FMaterialExpressionConnection(float Value);
		FMaterialExpressionConnection(float Value0, float Value1);
		FMaterialExpressionConnection(float Value0, float Value1, float Value2);
		FMaterialExpressionConnection(float Value0, float Value1, float Value2, float Value3);
		FMaterialExpressionConnection(double Value);
		FMaterialExpressionConnection(UTexture* Texture);
		FMaterialExpressionConnection(const FData& Value, const FData& True, const FData& False);

		bool operator!=(const FMaterialExpressionConnection& rhs);

		EConnectionType GetConnectionType() const
		{
			return ConnectionType;
		}

		// Get expression without incrementing usage count
		UMaterialExpression* GetMaterialExpression() const
		{
			return ExpressionData.GetMaterialExpression();
		}

		bool HasExpression() const
		{
			return (ConnectionType == Expression) && GetMaterialExpression();
		}

		bool IsExpressionWithoutExpression() const
		{
			return (ConnectionType == Expression) && !GetMaterialExpression();
		}

		template<typename ExpressionType>
		bool IsExpressionA() const
		{
			return GetMaterialExpression()->IsA<ExpressionType>();
		}

		int32 GetExpressionOutputIndex() const
		{
			return ExpressionData.Index;
		}

		FString GetExpressionName() const
		{
			return GetMaterialExpression()->GetName();
		}

		bool IsExpressionDefault() const
		{
			return ExpressionData.bIsDefault;
		}

		void SetExpressionDefault()
		{
			ExpressionData.bIsDefault = true;
		}

		void DestroyExpression()
		{
			ExpressionData.ExpressionHandle.DestroyExpression();
		}

		// Get expression but don't increment usage count
		UMaterialExpression* GetExpressionUnused() const
		{
			return GetMaterialExpression();
		}

		// Get expression AND increment usage count
		UMaterialExpression* GetExpressionAndUse()  const
		{
			bIsUsed = true;
			ExpressionData.ExpressionHandle.AddUser();
			return GetMaterialExpression();
		}

		// Get expression AND increment usage count(used in placed where it's unclear whether it's used)
		UMaterialExpression* GetExpressionAndMaybeUse() const
		{
			bIsUsed = true;
			ExpressionData.ExpressionHandle.AddUser();
			return GetMaterialExpression();
		}

		FData GetExpressionData() const
		{
			return ExpressionData;
		}

		bool GetBoolValue() const
		{
			return bValue;
		}

		const float* GetVectorValue() const
		{
			return Values;
		}

		FData* GetTextureSelectionData()
		{
			return TextureSelectionData;
		}

		UTexture* GetTextureAndUse() const
		{
			// todo: possible to track texture usage and discard unused
			return Texture;
		}

		UTexture* GetTextureUnused() const
		{
			return Texture;
		}

	private:
		EConnectionType ConnectionType;

		FData     ExpressionData;
		FData     TextureSelectionData[3];
		union
		{
			bool      bValue;
			float     Values[4];
			UTexture* Texture;
		};
		mutable bool bIsUsed = false;
	};

	struct FMaterialExpressionConnectionList
	{
		TArray<FMaterialExpressionConnection> Connections;

		bool bIsUsed;

		FMaterialExpressionConnectionList()
			: bIsUsed(false)
		{}

		FMaterialExpressionConnectionList(std::initializer_list<FMaterialExpressionConnection> Expressions)
			: bIsUsed(false)
		{
			Connections = Expressions;
		}

		FMaterialExpressionConnectionList(std::initializer_list<UMaterialExpression*> Expressions)
			: bIsUsed(false)
		{
			for (UMaterialExpression* Expression : Expressions)
			{
				Connections.Add(Expression);
			}
		}

		bool IsUsed()
		{
			return bIsUsed;
		}

		void SetIsUsed()
		{
			bIsUsed = true;
		}

		void Reserve(int32 Size)
		{
			Connections.Reserve(Size);
		}

		void SetNum(int32 Size)
		{
			Connections.SetNum(Size);
		}

		void Empty() 
		{
			Connections.Empty();
		}

		int32 Num() const
		{
			return Connections.Num();
		}

		void Add(const FMaterialExpressionConnection& Connection)
		{
			Connections.Add(Connection);
		}

		void Push(const FMaterialExpressionConnection&& Connection)
		{
			Connections.Push(Connection);
		}

		template <typename... ArgsType>
		FORCEINLINE int32 Emplace(ArgsType&&... Args)
		{
			return Connections.Emplace(Forward<ArgsType>(Args)...);
		}


		FMaterialExpressionConnection& operator[](int32 ConnectionIndex)
		{
			return Connections[ConnectionIndex];
		}

		const 	FMaterialExpressionConnection& operator[](int32 ConnectionIndex) const
		{
			return Connections[ConnectionIndex];
		}

		void Append(const FMaterialExpressionConnectionList& Other)
		{
			Connections.Append(Other.Connections);
		}

		template <typename Predicate>
		int32 FindLastByPredicate(Predicate Pred)
		{
			return Connections.FindLastByPredicate(Pred);
		}

		operator TArray<FMaterialExpressionConnection>()
		{
			return Connections;
		}


		void Reset()
		{
			Connections.SetNum(0);
			bIsUsed = false;
		}


	};
		

	//

	inline FMaterialExpressionConnection::FMaterialExpressionConnection()
	    : ConnectionType(EConnectionType::Expression)
	    , ExpressionData(nullptr, 0, true)
	{
	}

	inline FMaterialExpressionConnection::FMaterialExpressionConnection(UMaterialExpression* Expression)
		: ConnectionType(EConnectionType::Expression)
		, ExpressionData(Expression, 0, false)
	{
	}

	inline FMaterialExpressionConnection::FMaterialExpressionConnection(FMaterialExpressionHandle ExpressionHandle,
	                                                                    int32                OutputIndex /*= 0*/,
	                                                                    bool                 bIsDefault /*= false*/)
	    : ConnectionType(EConnectionType::Expression)
	    , ExpressionData(ExpressionHandle, OutputIndex, bIsDefault)
	{
	}

	inline FMaterialExpressionConnection::FMaterialExpressionConnection(bool bValue)
	    : ConnectionType(EConnectionType::Boolean)
	    , bValue(bValue)
	{
	}

	inline FMaterialExpressionConnection::FMaterialExpressionConnection(int Value)
	    : ConnectionType(EConnectionType::Float)
	    , Values {static_cast<float>(Value), 0.0f, 0.0f, 0.0f}
	{
	}

	inline FMaterialExpressionConnection::FMaterialExpressionConnection(float Value)
	    : ConnectionType(EConnectionType::Float)
	    , Values {Value, 0.0f, 0.0f, 0.0f}
	{
	}

	inline FMaterialExpressionConnection::FMaterialExpressionConnection(float Value0, float Value1)
	    : ConnectionType(EConnectionType::Float2)
	    , Values {Value0, Value1, 0.0f, 0.0f}
	{
	}

	inline FMaterialExpressionConnection::FMaterialExpressionConnection(float Value0, float Value1, float Value2)
	    : ConnectionType(EConnectionType::Float3)
	    , Values {Value0, Value1, Value2, 0.0f}
	{
	}

	inline FMaterialExpressionConnection::FMaterialExpressionConnection(float Value0, float Value1, float Value2, float Value3)
	    : ConnectionType(EConnectionType::Float4)
	    , Values {Value0, Value1, Value2, Value3}
	{
	}

	inline FMaterialExpressionConnection::FMaterialExpressionConnection(double Value)
	    : ConnectionType(EConnectionType::Float)
	    , Values {static_cast<float>(Value), 0.0f, 0.0f, 0.0f}
	{
	}

	inline FMaterialExpressionConnection::FMaterialExpressionConnection(UTexture* Texture)
	    : ConnectionType(EConnectionType::Texture)
	    , Texture(Texture)
	{
	}

	inline FMaterialExpressionConnection::FMaterialExpressionConnection(const FData& Value, const FData& True, const FData& False)
	    : ConnectionType(EConnectionType::TextureSelection)
	{
		TextureSelectionData[0] = Value;
		TextureSelectionData[1] = True;
		TextureSelectionData[2] = False;
	}

	inline bool FMaterialExpressionConnection::operator!=(const FMaterialExpressionConnection& rhs)
	{
		return (ConnectionType != rhs.ConnectionType) ||
		       ((ConnectionType == EConnectionType::Expression) && (ExpressionData != rhs.ExpressionData)) ||
		       ((ConnectionType == EConnectionType::Boolean) && (bValue != rhs.bValue)) ||
		       ((ConnectionType == EConnectionType::Float) && (Values[0] != rhs.Values[0])) ||
		       ((ConnectionType == EConnectionType::Float2) && ((Values[0] != rhs.Values[0]) || (Values[1] != rhs.Values[1]))) ||
		       ((ConnectionType == EConnectionType::Float3) &&
		        ((Values[0] != rhs.Values[0]) || (Values[1] != rhs.Values[1]) || (Values[2] != rhs.Values[2]))) ||
		       ((ConnectionType == EConnectionType::Float4) &&
		        ((Values[0] != rhs.Values[0]) || (Values[1] != rhs.Values[1]) || (Values[2] != rhs.Values[2]) || (Values[3] != rhs.Values[3]))) ||
		       ((ConnectionType == EConnectionType::Texture) && (Texture != rhs.Texture)) ||
		       ((ConnectionType == EConnectionType::TextureSelection) &&
		        ((TextureSelectionData[0] != rhs.TextureSelectionData[0]) || (TextureSelectionData[1] != rhs.TextureSelectionData[1]) ||
		         (TextureSelectionData[2] != rhs.TextureSelectionData[2])));
	}

	inline FMaterialExpressionConnection::FData::FData()
		: Index(0)
		, bIsDefault(false)
	{
	}

	inline FMaterialExpressionConnection::FData::FData(FMaterialExpressionHandle InExpressionHandle, int32 Index, bool bIsDefault)
	    : ExpressionHandle(InExpressionHandle)
	    , Index(Index)
	    , bIsDefault(bIsDefault)
	{
	}

	inline bool FMaterialExpressionConnection::FData::operator!=(const FData& rhs)
	{
		return (ExpressionHandle.GetMaterialExpression() != rhs.ExpressionHandle.GetMaterialExpression()) || (Index != rhs.Index) || (bIsDefault != rhs.bIsDefault);
	}

}  // namespace Generator
