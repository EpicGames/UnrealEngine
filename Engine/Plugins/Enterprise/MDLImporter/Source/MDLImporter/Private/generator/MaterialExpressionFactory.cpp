// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef USE_MDLSDK

#include "MaterialExpressionFactory.h"
#include "Materials/MaterialExpressionAbs.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"

#include "Materials/MaterialExpressionAdd.h"
#include "generator/FunctionLoader.h"
#include "Materials/MaterialExpressionArccosine.h"
#include "generator/MaterialExpressions.h"
#include "Materials/MaterialExpressionArcsine.h"
#include "generator/MaterialTextureFactory.h"
#include "Materials/MaterialExpressionArctangent.h"
#include "mdl/ApiContext.h"

#include "Engine/Texture2D.h"

MDLSDK_INCLUDES_START
#include "Materials/MaterialExpressionArctangent2.h"
#include "mi/neuraylib/icompiled_material.h"
#include "Materials/MaterialExpressionBlackBody.h"
#include "mi/neuraylib/iexpression.h"
#include "Materials/MaterialExpressionCeil.h"
#include "mi/neuraylib/ifunction_definition.h"
#include "Materials/MaterialExpressionClamp.h"
#include "mi/neuraylib/imdl_factory.h"
#include "Materials/MaterialExpressionComponentMask.h"
#include "mi/neuraylib/itransaction.h"
#include "Materials/MaterialExpressionConstant.h"
#include "mi/neuraylib/istring.h"
#include "Materials/MaterialExpressionChannelMaskParameter.h"
#include "Materials/MaterialExpressionCosine.h"
#include "Materials/MaterialExpressionCrossProduct.h"
#include "Materials/MaterialExpressionDistance.h"
#include "Materials/MaterialExpressionDivide.h"
#include "Materials/MaterialExpressionDotProduct.h"
#include "Materials/MaterialExpressionFloor.h"
#include "Materials/MaterialExpressionFmod.h"
#include "Materials/MaterialExpressionFrac.h"
#include "Materials/MaterialExpressionIf.h"
#include "Materials/MaterialExpressionLinearInterpolate.h"
#include "Materials/MaterialExpressionMakeMaterialAttributes.h"
#include "Materials/MaterialExpressionMax.h"
#include "Materials/MaterialExpressionMin.h"
#include "Materials/MaterialExpressionMultiply.h"
#include "Materials/MaterialExpressionNormalize.h"
#include "Materials/MaterialExpressionPower.h"
#include "Materials/MaterialExpressionScalarParameter.h"
#include "Materials/MaterialExpressionSine.h"
#include "Materials/MaterialExpressionSquareRoot.h"
#include "Materials/MaterialExpressionStaticBool.h"
#include "Materials/MaterialExpressionStaticBoolParameter.h"
#include "Materials/MaterialExpressionStaticSwitch.h"
#include "Materials/MaterialExpressionSubtract.h"
#include "Materials/MaterialExpressionTangent.h"
#include "Materials/MaterialExpressionTextureCoordinate.h"
#include "Materials/MaterialExpressionTextureProperty.h"
#include "Materials/MaterialExpressionTextureSampleParameter.h"
#include <mi/base/iinterface.h>
#include <mi/base/types.h>
#include <mi/base/uuid.h>
#include <mi/neuraylib/itype.h>
#include <mi/neuraylib/ivalue.h>
MDLSDK_INCLUDES_END

namespace Generator
{
	namespace
	{
		int32 CalcArraySize(const mi::base::Handle<const mi::neuraylib::IExpression>& Expression)
		{
			const mi::base::Handle<const mi::neuraylib::IType_array> Type(Expression->get_type<mi::neuraylib::IType_array>());

			if (Type->get_kind() == mi::neuraylib::IType::TK_ARRAY)
			{
				if (Expression->get_kind() != mi::neuraylib::IExpression::EK_CONSTANT)
					return (int32)Type->get_size();

				auto Handle      = Expression.get_interface<const mi::neuraylib::IExpression_constant>();
				auto ValueHandle = mi::base::make_handle(Handle->get_value<const mi::neuraylib::IValue_array>());
				return (int32)ValueHandle->get_size();
			}

			return 1;
		}

		void RerouteNormal(const FMaterialExpressionConnection& From, UMaterialExpression* ToExpression)
		{
			check((From.GetConnectionType() == EConnectionType::Expression) && From.GetExpressionUnused() && ToExpression);
			if (ToExpression->IsA<UMaterialExpressionMakeMaterialAttributes>())
			{
				auto* Expression = Cast<UMaterialExpressionMakeMaterialAttributes>(ToExpression);
				Expression->Normal.Connect(From.GetExpressionOutputIndex(), From.GetExpressionAndUse());
			}
			else if (ToExpression->IsA<UMaterialExpressionMaterialFunctionCall>())
			{
				check(Cast<UMaterialExpressionMaterialFunctionCall>(ToExpression)->FunctionInputs.Last().Input.InputName == TEXT("normal"));
				auto* Expression = Cast<UMaterialExpressionMaterialFunctionCall>(ToExpression);
				Expression->FunctionInputs.Last().Input.Connect(From.GetExpressionOutputIndex(), From.GetExpressionAndUse());
			}
			else
			{
				check(ToExpression->IsA<UMaterialExpressionStaticSwitch>());
				UMaterialExpressionStaticSwitch* StaticSwitch = Cast<UMaterialExpressionStaticSwitch>(ToExpression);
				RerouteNormal(From, StaticSwitch->A.Expression);
				RerouteNormal(From, StaticSwitch->B.Expression);
			}
		}

		FString GetAssetNamePostFix(int Semantic, const mi::neuraylib::IType_list& ParameterTypes)
		{
			FString AssetNamePostfix;

			// set Postfix if needed
			switch (Semantic)
			{
				case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_COS:
				case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_LOG:
				case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_LOG2:
				case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_LOG10:
				case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_SIN:
				{
					mi::base::Handle<const mi::neuraylib::IType> Type(ParameterTypes.get_type(mi::Size(0)));
					mi::neuraylib::IType::Kind                   kind = Type->get_kind();
					switch (kind)
					{
						case mi::neuraylib::IType::TK_COLOR:
							AssetNamePostfix = TEXT("_float3");
							break;
						case mi::neuraylib::IType::TK_FLOAT:
							AssetNamePostfix = TEXT("_float");
							break;
						case mi::neuraylib::IType::TK_VECTOR:
						{
							const mi::base::Handle<const mi::neuraylib::IType_vector> VectorType(
							    Type->get_interface<const mi::neuraylib::IType_vector>());
							check(VectorType->get_size() == 3);
							check(mi::base::make_handle(VectorType->get_element_type())->get_kind() == mi::neuraylib::IType::TK_FLOAT);
							AssetNamePostfix = TEXT("_float3");
						}
						break;
						default:
							check(false);
					}
				}
				break;
			}
			return AssetNamePostfix;
		}

		UMaterialExpression* CompareStaticBool(UMaterial* Parent, UMaterialExpressionStaticBool* StaticBool, UMaterialExpression* RHS,
		                                       bool EqualCheck)
		{
			if (StaticBool->Value == EqualCheck)
			{
				// either check for equal to true, or not equal to false -> just keep the right hand side
				return RHS;
			}
			else if (RHS->IsA<UMaterialExpressionStaticBoolParameter>() || RHS->IsA<UMaterialExpressionStaticSwitch>())
			{
				// determine !RHS by an inverting StaticSwitch
				return NewMaterialExpressionStaticSwitch(Parent, RHS, NewMaterialExpressionStaticBool(Parent, false),
				                                         NewMaterialExpressionStaticBool(Parent, true));
			}
			else
			{
				// determine !RHS by an IfEqual comparing to 0.0f
				return NewMaterialExpressionIfEqual(Parent, RHS, 0.0f, 1.0f, 0.0f);
			}
		}
	}

	FMaterialExpressionFactory::FMaterialExpressionFactory(const Mdl::FApiContext& MdlContext)
	    : CurrentTransaction(nullptr)
	    , CurrentNormalExpression(nullptr)
	    , bIsGeometryExpression(false)
	    , FunctionLoader(nullptr)
	{
		MdlFactory = Mdl::Detail::GetFactory(MdlContext);
		check(MdlFactory != nullptr);
	}

	void FMaterialExpressionFactory::SetCurrentMaterial(const mi::neuraylib::IMaterial_definition& MDLMaterialDefinition,
	                                                    const mi::neuraylib::ICompiled_material&   MDLMaterial,
	                                                    mi::neuraylib::ITransaction&               MDLTransaction,
	                                                    UMaterial&                                 Material)
	{
		FBaseExpressionFactory::SetCurrentMaterial(MDLMaterialDefinition, MDLMaterial, Material);
		ConstantExpressionFactory.SetCurrentMaterial(MDLMaterialDefinition, MDLMaterial, Material);
		ParameterExpressionFactory.SetCurrentMaterial(MDLMaterialDefinition, MDLMaterial, Material);

		CurrentTransaction = &MDLTransaction;
		// Temporaries are imported on demand, as they might change after having read the geometry part
		Temporaries.SetNum(CurrentMDLMaterial->get_temporary_count());
	}

	void FMaterialExpressionFactory::CreateParameterExpressions()
	{
		ParameterExpressionFactory.CreateExpressions(*CurrentTransaction);
	}

	void FMaterialExpressionFactory::SetTextureFactory(FMaterialTextureFactory* Factory)
	{
		TextureFactory = Factory;
		ConstantExpressionFactory.SetTextureFactory(Factory);
		ParameterExpressionFactory.SetTextureFactory(Factory);
	}

	FMaterialExpressionConnectionList FMaterialExpressionFactory::CreateExpression(
	    const mi::base::Handle<const mi::neuraylib::IExpression>& MDLExpression, const FString& CallPath)
	{
		check(CurrentMaterial != nullptr);
		check(CurrentMDLMaterial != nullptr);
		check(CurrentTransaction != nullptr);
		check(MDLExpression);
		check(FunctionLoader);

		FMaterialExpressionConnectionList Outputs;
		const mi::neuraylib::IExpression::Kind kind = MDLExpression->get_kind();
		switch (kind)
		{
			case mi::neuraylib::IExpression::EK_CONSTANT:
			{
				auto Handle      = MDLExpression.get_interface<const mi::neuraylib::IExpression_constant>();
				auto ValueHandle = mi::base::make_handle(Handle->get_value());
				Outputs          = ConstantExpressionFactory.CreateExpression(*CurrentTransaction, *ValueHandle);
			}
			break;
			case mi::neuraylib::IExpression::EK_DIRECT_CALL:
			{
				auto Handle = MDLExpression.get_interface<const mi::neuraylib::IExpression_direct_call>();
				Outputs     = CreateExpressionFunctionCall(*Handle, CallPath);
			}
			break;
			case mi::neuraylib::IExpression::EK_PARAMETER:
			{
				auto Handle = MDLExpression.get_interface<const mi::neuraylib::IExpression_parameter>();
				Outputs     = ParameterExpressionFactory.GetExpression(*Handle);

				// when expression up the call stack is a normalmap expression make (every)texture used under it
				// treated as a normalmap
				// store it as a normalmap and sample it as normal
				if (bProcessingNormapMap)
				{
					if (Outputs.Num() == 1)
					{
						if (Outputs[0].GetConnectionType() == EConnectionType::Expression)
						{
							if (Outputs[0].IsExpressionA<UMaterialExpressionTextureSampleParameter>())
							{
								UMaterialExpression* Expression = Outputs[0].GetExpressionAndMaybeUse();
								UMaterialExpressionTextureSampleParameter* TextureExpression = Cast<UMaterialExpressionTextureSampleParameter>(Expression);
								if (UTexture2D* Texture = Cast<UTexture2D>(TextureExpression->Texture))
								{
									TextureExpression->SamplerType = Texture->VirtualTextureStreaming ? SAMPLERTYPE_VirtualNormal : SAMPLERTYPE_Normal;

									if (!Texture->GetPathName().StartsWith(TEXT("/Engine")))
									{
										Common::FTextureProperty Property;
										Property.bIsSRGB = false;
										Property.CompressionSettings = TC_Normalmap;
										TextureFactory->UpdateTextureSettings(Cast<UTexture2D>(TextureExpression->Texture), Property);
									}
									else
									{
										TextureExpression->Texture = LoadObject<UTexture2D>(nullptr, TEXT("/Engine/EngineMaterials/DefaultNormal.DefaultNormal"), nullptr, LOAD_None, nullptr);
									}
								}
							}

						}
					}
				}
			}
			break;
			case mi::neuraylib::IExpression::EK_TEMPORARY:
			{
				auto Handle = MDLExpression.get_interface<const mi::neuraylib::IExpression_temporary>();
				Outputs     = CreateExpressionTemporary(*Handle, CallPath);
			}
			break;
			default:
				checkf(0, TEXT("Unhandled MDL expression type %d"), (int32)kind);
				break;
		}

		return Outputs;
	}

	FMaterialExpressionConnectionList FMaterialExpressionFactory::CreateExpressionTemporary(
	    const mi::neuraylib::IExpression_temporary& MDLExpression, const FString& CallPath)
	{
		const mi::Size Index = MDLExpression.get_index();
		if (Temporaries[Index].Num() == 0)
		{
			auto Handle        = mi::base::make_handle(CurrentMDLMaterial->get_temporary(Index));
			Temporaries[Index] = CreateExpression(Handle, CallPath);
		}
		return Temporaries[Index];
	}

	FMaterialExpressionConnectionList FMaterialExpressionFactory::CreateExpressionUnary(int                                          Semantic,
	                                                                                    const FMaterialExpressionConnectionList& Inputs,
	                                                                                    const mi::neuraylib::IType&                  MDLType)
	{
		switch (Semantic)
		{
			case mi::neuraylib::IFunction_definition::DS_CONV_OPERATOR:
				check(Inputs.Num() == 1);
				check(Inputs[0].GetConnectionType() == EConnectionType::Expression);
				check(Inputs[0].IsExpressionA<UMaterialExpressionScalarParameter>() &&
				      (MDLType.get_kind() == mi::neuraylib::IType::TK_INT));
				return { Inputs };

			case mi::neuraylib::IFunction_definition::DS_LOGICAL_NOT:
			{
				check(Inputs.Num() == 1);
				check(Inputs[0].GetConnectionType() == EConnectionType::Expression);

				if (IsStatic(Inputs[0]))
				{
					return {NewMaterialExpressionStaticSwitch(CurrentMaterial, Inputs[0], NewMaterialExpressionStaticBool(CurrentMaterial, false),
					                                          NewMaterialExpressionStaticBool(CurrentMaterial, true))};
				}
				else
				{
					return {NewMaterialExpressionIfEqual(CurrentMaterial, Inputs[0], 0.0f, 1.0f, 0.0f)};
				}
			}
			case mi::neuraylib::IFunction_definition::DS_POSITIVE:
			{
				check(Inputs.Num() == 1);
				return Inputs;
			}
			case mi::neuraylib::IFunction_definition::DS_NEGATIVE:
			{
				check(Inputs.Num() == 1);
				return {NewMaterialExpressionNegate(CurrentMaterial, Inputs[0])};
			}
			case mi::neuraylib::IFunction_definition::DS_PRE_INCREMENT:
			{
				check(Inputs.Num() == 1);
				return {NewMaterialExpressionAdd(CurrentMaterial, Inputs[0], 1.0f)};
			}
			case mi::neuraylib::IFunction_definition::DS_PRE_DECREMENT:
			{
				check(Inputs.Num() == 1);
				return {NewMaterialExpressionSubtract(CurrentMaterial, Inputs[0], 1.0f)};
			}
			default:
				check(false);
				return {};
		}
	}

	FMaterialExpressionConnectionList FMaterialExpressionFactory::CreateExpressionBinary(int                                          Semantic,
	                                                                                     const FMaterialExpressionConnectionList& Inputs)
	{
		switch (Semantic)
		{
			case mi::neuraylib::IFunction_definition::DS_DIVIDE:
			{
				check(Inputs.Num() == 2);
				return {NewMaterialExpressionDivide(CurrentMaterial, Inputs[0], Inputs[1])};
			}
			case mi::neuraylib::IFunction_definition::DS_MODULO:
			{
				check(Inputs.Num() == 2);
				return {NewMaterialExpressionFmod(CurrentMaterial, Inputs[0], Inputs[1])};
			}
			case mi::neuraylib::IFunction_definition::DS_PLUS:
			{
				check(Inputs.Num() == 2);
				return {NewMaterialExpressionAdd(CurrentMaterial, Inputs[0], Inputs[1])};
			}
			case mi::neuraylib::IFunction_definition::DS_MINUS:
			{
				check(Inputs.Num() == 2);
				return {NewMaterialExpressionSubtract(CurrentMaterial, Inputs[0], Inputs[1])};
			}
			case mi::neuraylib::IFunction_definition::DS_LESS:
			{
				check(Inputs.Num() == 2);
				return {NewMaterialExpressionIfLess(CurrentMaterial, Inputs[0], Inputs[1], 1.0f, 0.0f)};
			}
			case mi::neuraylib::IFunction_definition::DS_LESS_OR_EQUAL:
			{
				check(Inputs.Num() == 2);
				return {NewMaterialExpressionIfGreater(CurrentMaterial, Inputs[1], Inputs[0], 1.0f, 0.0f)};
			}
			case mi::neuraylib::IFunction_definition::DS_GREATER_OR_EQUAL:
			{
				check(Inputs.Num() == 2);
				return {NewMaterialExpressionIfLess(CurrentMaterial, Inputs[1], Inputs[0], 1.0f, 0.0f)};
			}
			case mi::neuraylib::IFunction_definition::DS_GREATER:
			{
				check(Inputs.Num() == 2);
				return {NewMaterialExpressionIfGreater(CurrentMaterial, Inputs[0], Inputs[1], 1.0f, 0.0f)};
			}
			case mi::neuraylib::IFunction_definition::DS_EQUAL:
			{
				check(Inputs.Num() == 2);
				check(Inputs[0].GetConnectionType() == EConnectionType::Expression);
				check(Inputs[1].GetConnectionType() == EConnectionType::Expression);

				if (Inputs[0].IsExpressionA<UMaterialExpressionStaticBool>())
				{
					check(!Inputs[1].IsExpressionA<UMaterialExpressionStaticBool>());
					return {CompareStaticBool(CurrentMaterial, Cast<UMaterialExpressionStaticBool>(Inputs[0].GetExpressionAndMaybeUse()),
					                          Inputs[1].GetExpressionAndMaybeUse(), true)};
				}
				else if (Inputs[1].IsExpressionA<UMaterialExpressionStaticBool>())
				{
					return {CompareStaticBool(CurrentMaterial, Cast<UMaterialExpressionStaticBool>(Inputs[1].GetExpressionAndMaybeUse()),
					                          Inputs[0].GetExpressionAndMaybeUse(), true)};
				}
				else
				{
					return {NewMaterialExpressionIfEqual(CurrentMaterial, Inputs[0], Inputs[1], 1.0f, 0.0f)};
				}
			}
			case mi::neuraylib::IFunction_definition::DS_NOT_EQUAL:
			{
				check(Inputs.Num() == 2);
				check(Inputs[0].GetConnectionType() == EConnectionType::Expression);
				check(Inputs[1].GetConnectionType() == EConnectionType::Expression);

				if (Inputs[0].IsExpressionA<UMaterialExpressionStaticBool>())
				{
					check(!Inputs[1].IsExpressionA<UMaterialExpressionStaticBool>());
					return {CompareStaticBool(CurrentMaterial, Cast<UMaterialExpressionStaticBool>(Inputs[0].GetExpressionAndMaybeUse()),
					                          Inputs[1].GetExpressionAndMaybeUse(), false)};
				}
				else if (Inputs[1].IsExpressionA<UMaterialExpressionStaticBool>())
				{
					return {CompareStaticBool(CurrentMaterial, Cast<UMaterialExpressionStaticBool>(Inputs[1].GetExpressionAndMaybeUse()),
					                          Inputs[0].GetExpressionAndMaybeUse(), false)};
				}
				else
				{
					return {NewMaterialExpressionIfEqual(CurrentMaterial, Inputs[0], Inputs[1], 0.0f, 1.0f)};
				}
			}
			case mi::neuraylib::IFunction_definition::DS_LOGICAL_AND:
			{
				check(Inputs.Num() == 2);
				check(Inputs[0].GetConnectionType() == EConnectionType::Expression);
				check(Inputs[1].GetConnectionType() == EConnectionType::Expression);

				if (IsStatic(Inputs[0]))
				{
					if (IsStatic(Inputs[1]))
					{
						return {NewMaterialExpressionStaticSwitch(
						    CurrentMaterial, Inputs[0],
						    NewMaterialExpressionStaticSwitch(CurrentMaterial, Inputs[1], NewMaterialExpressionStaticBool(CurrentMaterial, true),
						                                      NewMaterialExpressionStaticBool(CurrentMaterial, false)),
						    NewMaterialExpressionStaticBool(CurrentMaterial, false))};
					}
					else
					{
						return {NewMaterialExpressionStaticSwitch(CurrentMaterial, Inputs[0],
						                                          NewMaterialExpressionIfEqual(CurrentMaterial, Inputs[1], 0.0f, 0.0f, 1.0f), 0.0f)};
					}
				}
				else if (IsStatic(Inputs[1]))
				{
					return {NewMaterialExpressionStaticSwitch(CurrentMaterial, Inputs[1],
					                                          NewMaterialExpressionIfEqual(CurrentMaterial, Inputs[0], 0.0f, 0.0f, 1.0f), 0.0f)};
				}
				else
				{
					return {NewMaterialExpressionIfEqual(CurrentMaterial, Inputs[0], 0.0f, 0.0f,
					                                     NewMaterialExpressionIfEqual(CurrentMaterial, Inputs[1], 0.0f, 0.0f, 1.0f))};
				}
			}
			case mi::neuraylib::IFunction_definition::DS_LOGICAL_OR:
			{
				check(Inputs.Num() == 2);
				check(Inputs[0].GetConnectionType() == EConnectionType::Expression);
				check(Inputs[1].GetConnectionType() == EConnectionType::Expression);

				if (IsStatic(Inputs[0]))
				{
					if (IsStatic(Inputs[1]))
					{
						return {NewMaterialExpressionStaticSwitch(
						    CurrentMaterial, Inputs[0], NewMaterialExpressionStaticBool(CurrentMaterial, true),
						    NewMaterialExpressionStaticSwitch(CurrentMaterial, Inputs[1], NewMaterialExpressionStaticBool(CurrentMaterial, true),
						                                      NewMaterialExpressionStaticBool(CurrentMaterial, false)))};
					}
					else
					{
						return {NewMaterialExpressionStaticSwitch(CurrentMaterial, Inputs[0], 1.0f,
						                                          NewMaterialExpressionIfEqual(CurrentMaterial, Inputs[1], 0.0f, 0.0f, 1.0f))};
					}
				}
				else if (IsStatic(Inputs[1]))
				{
					return {NewMaterialExpressionStaticSwitch(CurrentMaterial, Inputs[1], 1.0f,
					                                          NewMaterialExpressionIfEqual(CurrentMaterial, Inputs[0], 0.0f, 0.0f, 1.0f))};
				}
				else
				{
					return {NewMaterialExpressionIfEqual(CurrentMaterial, Inputs[0], 0.0f,
					                                     NewMaterialExpressionIfEqual(CurrentMaterial, Inputs[1], 0.0f, 0.0f, 1.0f), 1.0f)};
				}
				default:
					check(false);
					return {};
			}
		}
	}

	FMaterialExpressionConnectionList FMaterialExpressionFactory::CreateExpressionTernary(const FMaterialExpressionConnectionList& Inputs)
	{
		check((Inputs.Num() - 1) % 2 == 0 && Inputs.Num() >= 3);
		check(Inputs[0].GetConnectionType() == EConnectionType::Expression);

		const int32 NumOutputs = (Inputs.Num() - 1) / 2;

		FMaterialExpressionConnectionList Outputs;
		Outputs.Reserve(NumOutputs);
		for (int32 i = 1; i <= NumOutputs; i++)
		{
			if (IsTexture(Inputs[i]))
			{
				// StaticSwitch and If on Texture would fail -> store everything for evaluating later on
				check((Inputs.Num() == 3) && (Inputs[0].GetConnectionType() == EConnectionType::Expression) &&
				      (Inputs[1].GetConnectionType() == EConnectionType::Expression) && (Inputs[2].GetConnectionType() == EConnectionType::Expression));
				Outputs.Add(FMaterialExpressionConnection(Inputs[0].GetExpressionData(), Inputs[1].GetExpressionData(), Inputs[2].GetExpressionData()));
			}
			else if (IsStatic(Inputs[0]))
			{
				Outputs.Add(NewMaterialExpressionStaticSwitch(CurrentMaterial, Inputs[0], Inputs[i], Inputs[i + NumOutputs]));
			}
			else if (IsMaterialAttribute(Inputs[i]))
			{
				UMaterialFunction* SelectBSDF = FunctionLoader->Load(TEXT("mdlimporter_select_bsdf"));
				// If on MaterialAttribute would fail -> use a helper function instead
				// Note: static switch on a MaterialAttribute works! Therefore, check for MaterialAttribute after check for Static !
				check(IsMaterialAttribute(Inputs[i + NumOutputs]));
				Outputs.Add(NewMaterialExpressionFunctionCall(CurrentMaterial, SelectBSDF,
				                                              {Inputs[0], Inputs[i], Inputs[i + NumOutputs], CurrentNormalExpression}));
			}
			else
			{
				Outputs.Add(NewMaterialExpressionIfEqual(CurrentMaterial, Inputs[0], 0.0f, Inputs[i + NumOutputs], Inputs[i]));
			}
		}

		return Outputs;
	}

	FMaterialExpressionConnectionList FMaterialExpressionFactory::CreateExpressionMath(int                                          Semantic,
	                                                                                   const FMaterialExpressionConnectionList& Inputs)
	{
		switch (Semantic)
		{
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_ABS:
			{
				check(Inputs.Num() == 1);
				return {NewMaterialExpressionAbs(CurrentMaterial, Inputs[0])};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_ACOS:
			{
				check(Inputs.Num() == 1);
				return {NewMaterialExpressionArccosine(CurrentMaterial, Inputs[0])};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_ASIN:
			{
				check(Inputs.Num() == 1);
				return {NewMaterialExpressionArcsine(CurrentMaterial, Inputs[0])};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_ATAN:
			{
				check(Inputs.Num() == 1);
				return {NewMaterialExpressionArctangent(CurrentMaterial, Inputs[0])};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_ATAN2:
			{
				check(Inputs.Num() == 2);
				return {NewMaterialExpressionArctangent2(CurrentMaterial, Inputs[0], Inputs[1])};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_CEIL:
			{
				check(Inputs.Num() == 1);
				return {NewMaterialExpressionCeil(CurrentMaterial, Inputs[0])};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_CLAMP:
			{
				check(Inputs.Num() == 3);
				return {NewMaterialExpressionClamp(CurrentMaterial, Inputs[0], Inputs[1], Inputs[2])};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_CROSS:
			{
				check(Inputs.Num() == 2);
				return {NewMaterialExpressionCrossProduct(CurrentMaterial, Inputs[0], Inputs[1])};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_DEGREES:
			{
				check(Inputs.Num() == 1);
				return {NewMaterialExpressionMultiply(CurrentMaterial, Inputs[0], 180.0f / PI)};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_DISTANCE:
			{
				check(Inputs.Num() == 2);
				return {NewMaterialExpressionDistance(CurrentMaterial, Inputs[0], Inputs[1])};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_DOT:
			{
				check(Inputs.Num() == 2);
				return {NewMaterialExpressionDotProduct(CurrentMaterial, Inputs[0], Inputs[1])};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_EXP:
			{
				check(Inputs.Num() == 1);
				const float e = 2.71828f;
				return {NewMaterialExpressionPower(CurrentMaterial, e, Inputs[0])};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_EXP2:
			{
				check(Inputs.Num() == 1);
				return {NewMaterialExpressionPower(CurrentMaterial, 2.0f, Inputs[0])};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_FLOOR:
			{
				check(Inputs.Num() == 1);
				return {NewMaterialExpressionFloor(CurrentMaterial, Inputs[0])};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_FMOD:
			{
				check(Inputs.Num() == 2);
				return {NewMaterialExpressionFmod(CurrentMaterial, Inputs[0], Inputs[1])};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_FRAC:
			{
				check(Inputs.Num() == 1);
				return {NewMaterialExpressionFrac(CurrentMaterial, Inputs[0])};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_LENGTH:
			{
				check(Inputs.Num() == 1);
				return {NewMaterialExpressionSquareRoot(CurrentMaterial, NewMaterialExpressionDotProduct(CurrentMaterial, Inputs[0], Inputs[0]))};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_LERP:
			{
				check(Inputs.Num() == 3);
				return {NewMaterialExpressionLinearInterpolate(CurrentMaterial, Inputs[0], Inputs[1], Inputs[2])};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_MAX:
			{
				check(Inputs.Num() == 2);
				return {NewMaterialExpressionMax(CurrentMaterial, Inputs[0], Inputs[1])};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_MIN:
			{
				check(Inputs.Num() == 2);
				return {NewMaterialExpressionMin(CurrentMaterial, Inputs[0], Inputs[1])};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_MODF:
			{
				check(Inputs.Num() == 1);

				UMaterialExpressionFrac*     FractionalPart = NewMaterialExpressionFrac(CurrentMaterial, Inputs[0]);
				UMaterialExpressionSubtract* IntegralPart   = NewMaterialExpressionSubtract(CurrentMaterial, Inputs[0], FractionalPart);

				return {IntegralPart, FractionalPart};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_NORMALIZE:
			{
				check(Inputs.Num() == 1);
				return {NewMaterialExpressionNormalize(CurrentMaterial, Inputs[0])};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_POW:
			{
				check(Inputs.Num() == 2);
				return {NewMaterialExpressionPower(CurrentMaterial, Inputs[0], Inputs[1])};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_RADIANS:
			{
				check(Inputs.Num() == 1);
				return {NewMaterialExpressionMultiply(CurrentMaterial, Inputs[0], PI / 180.0f)};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_ROUND:
			{
				check(Inputs.Num() == 1);
				return {NewMaterialExpressionFloor(CurrentMaterial, NewMaterialExpressionAdd(CurrentMaterial, Inputs[0], 0.5f))};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_RSQRT:
			{
				check(Inputs.Num() == 1);
				return {NewMaterialExpressionDivide(CurrentMaterial, 1.0f, NewMaterialExpressionSquareRoot(CurrentMaterial, Inputs[0]))};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_SATURATE:
			{
				check(Inputs.Num() == 1);
				return {NewMaterialExpressionClamp(CurrentMaterial, Inputs[0], 0.0f, 1.0f)};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_SIGN:
			{
				check(Inputs.Num() == 1);
				return {NewMaterialExpressionIf(CurrentMaterial, Inputs[0], 0.0f, -1.0f, 0.0f, 1.0f)};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_SINCOS:
			{
				check(Inputs.Num() == 1);
				return {NewMaterialExpressionSine(CurrentMaterial, Inputs[0]), NewMaterialExpressionCosine(CurrentMaterial, Inputs[0])};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_SQRT:
			{
				check(Inputs.Num() == 1);
				return {NewMaterialExpressionSquareRoot(CurrentMaterial, Inputs[0])};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_TAN:
			{
				check(Inputs.Num() == 1);
				return {NewMaterialExpressionTangent(CurrentMaterial, Inputs[0])};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_BLACKBODY:
			{
				check(Inputs.Num() == 1);
				return {NewMaterialExpressionBlackBody(CurrentMaterial, Inputs[0])};
			}
			default:
				check(false);
				return {};
		}
	}

	FMaterialExpressionConnectionList FMaterialExpressionFactory::CreateExpressionDAG(int                                          Semantic,
	                                                                                  const FMaterialExpressionConnectionList& Inputs,
	                                                                                  const FString&                               FunctionName)
	{
		switch (Semantic)
		{
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DAG_FIELD_ACCESS:
			{
				if (FunctionName == TEXT("::base::anisotropy_return.roughness_u(::base::anisotropy_return)"))
				{
					check((Inputs.Num() == 0) || (Inputs.Num() == 3));
					return (Inputs.Num() == 0) ? Inputs : FMaterialExpressionConnectionList({Inputs[0]});
				}
				else if (FunctionName == TEXT("::base::anisotropy_return.roughness_v(::base::anisotropy_return)"))
				{
					check((Inputs.Num() == 0) || (Inputs.Num() == 3));
					return (Inputs.Num() == 0) ? Inputs : FMaterialExpressionConnectionList({Inputs[1]});
				}
				else if (FunctionName == TEXT("::base::anisotropy_return.tangent_u(::base::anisotropy_return)"))
				{
					check((Inputs.Num() == 0) || (Inputs.Num() == 3));
					return (Inputs.Num() == 0) ? Inputs : FMaterialExpressionConnectionList({Inputs[2]});
				}
				else if (FunctionName == TEXT("::base::texture_return.tint(::base::texture_return)"))
				{
					check((Inputs.Num() == 0) || (Inputs.Num() == 2));
					return (Inputs.Num() == 0) ? Inputs : FMaterialExpressionConnectionList({Inputs[0]});
				}
				else
				{
					unsigned int crc32 = FCrc::StrCrc32(*FunctionName);
					switch (crc32)
					{
						case 84791706:    // material_surface.scattering(material_surface)
						case 146468741:   // material.thin_walled(material)
						case 1818454712:  // ::base::texture_coordinate_info.position(::base::texture_coordinate_info)
						case 4090769780:  // ::base::texture_coordinate_info.position
							check(1 <= Inputs.Num());
							return {Inputs[0]};
						case 820916979:   // ::base::texture_return.mono(::base::texture_return)
						case 3402110799:  // ::base::texture_coordinate_info.tangent_u
							check(2 <= Inputs.Num());
							return {Inputs[1]};
						case 3256380617:  // ::base::texture_coordinate_info.tangent_v
							check(3 <= Inputs.Num());
							return {Inputs[2]};
						case 2291824070:  // material_surface.emission(material_surface)
							check(4 <= Inputs.Num());
							return {Inputs[1], Inputs[2], Inputs[3]};
						case 2487874440:  // material.surface(material)
							check(Inputs.Num() == 16);
							return {Inputs[1], Inputs[2], Inputs[3], Inputs[4]};
						case 1789602102:  // material.ior(material)
							check(Inputs.Num() == 16);
							return {Inputs[9]};
						case 2121329778:  // material.volume(material)
							check(Inputs.Num() == 16);
							return {Inputs[10], Inputs[11], Inputs[12]};
					}
					check(false);
				}
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DAG_ARRAY_CONSTRUCTOR:
			{
				return Inputs;
			}
			case mi::neuraylib::IFunction_definition::DS_ARRAY_INDEX:
			{
				check(Inputs.Num() == 2);
				check(Inputs[0].GetConnectionType() == EConnectionType::Expression);
				check(Inputs[1].GetConnectionType() == EConnectionType::Expression);
				check(Inputs[1].IsExpressionA<UMaterialExpressionConstant>());
				check(FunctionName == TEXT("operator[](<0>[],int)") || FunctionName == TEXT("operator[](%3C0%3E[],int)"));

				const int32 Index = (int32)Cast<UMaterialExpressionConstant>(Inputs[1].GetExpressionUnused())->R;
				CurrentMaterial->GetExpressionCollection().RemoveExpression(Inputs[1].GetExpressionUnused());

				return {NewMaterialExpressionComponentMask(CurrentMaterial, Inputs[0], 1 << Index)};
			}
			default:
				check(false);
				return {};
		}
	}

	FMaterialExpressionConnectionList FMaterialExpressionFactory::CreateExpressionOther(int                                          Semantic,
	                                                                                    const FMaterialExpressionConnectionList& Inputs)
	{
		switch (Semantic)
		{
				// state intrinsics
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_STATE_NORMAL:
			{
				check(Inputs.Num() == 0);
				return {CurrentNormalExpression};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_STATE_ROUNDED_CORNER_NORMAL:
			{
				// just do something for rounded corner normal... we can't do that!
				check(Inputs.Num() == 3);
				return {CurrentNormalExpression};
			}

			// tex intrinsics
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_TEX_WIDTH:
			{
				check(Inputs.Num() == 1);
				return {NewMaterialExpressionComponentMask(CurrentMaterial,
				                                           NewMaterialExpressionTextureProperty(CurrentMaterial, Inputs[0], TMTM_TextureSize), 1)};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_TEX_HEIGHT:
			{
				check(Inputs.Num() == 1);
				return {NewMaterialExpressionComponentMask(CurrentMaterial,
				                                           NewMaterialExpressionTextureProperty(CurrentMaterial, Inputs[0], TMTM_TextureSize), 2)};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_TEX_DEPTH:
			{
				// Unreal doesn't know about 3D textures ?? Does that mean, the depth is always 1 ?
				check(Inputs.Num() == 1);
				return {NewMaterialExpressionConstant(CurrentMaterial, 1.0f)};
			}
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_TEX_TEXTURE_ISVALID:
			{
				// in Unreal, a texture is always valid, so just make a static bool <true>
				return {NewMaterialExpressionStaticBool(CurrentMaterial, true)};
			}
			default:
				check(false);
				return {};
		}
	}

	void FMaterialExpressionFactory::HandleNormal(int Semantic, FMaterialExpressionConnectionList& Inputs)
	{
		// Special distribution function handling, step one: add a normal to the bsdfs
		switch (Semantic)
		{
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_DIFFUSE_REFLECTION_BSDF:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_DIFFUSE_TRANSMISSION_BSDF:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_SPECULAR_BSDF:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_SIMPLE_GLOSSY_BSDF:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_BACKSCATTERING_GLOSSY_REFLECTION_BSDF:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_MEASURED_BSDF:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_NORMALIZED_MIX:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_THIN_FILM:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_TINT:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_DIRECTIONAL_FACTOR:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_MEASURED_CURVE_FACTOR:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_MICROFACET_BECKMANN_SMITH_BSDF:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_MICROFACET_GGX_SMITH_BSDF:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_MICROFACET_BECKMANN_VCAVITIES_BSDF:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_MICROFACET_GGX_VCAVITIES_BSDF:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_WARD_GEISLER_MORODER_BSDF:
				Inputs.Push(CurrentNormalExpression);
				break;

				// These DFs need some special handling to determine if the CurrentNormalExpression is to be pushed!
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_CLAMPED_MIX:  // only if mixing BSDFs
				check(false);
				break;
		}

		// weighted_layer, fresnel_layer, custom_curve_layer, and measured_curve_layer reroute their normal to their layer argument
		switch (Semantic)
		{
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_WEIGHTED_LAYER:
				check(Inputs[1].HasExpression());
				check(Inputs[3].HasExpression());
				if (Inputs[1].IsExpressionA<UMaterialExpressionStaticSwitch>())
				{
					UMaterialExpressionStaticSwitch* StaticSwitch = Cast<UMaterialExpressionStaticSwitch>(Inputs[1].GetExpressionAndMaybeUse());
					check(StaticSwitch->A.Expression && StaticSwitch->B.Expression && StaticSwitch->Value.Expression);
					RerouteNormal(Inputs[3], StaticSwitch->A.Expression);
					RerouteNormal(Inputs[3], StaticSwitch->B.Expression);
				}
				else
				{
					RerouteNormal(Inputs[3], Inputs[1].GetExpressionAndMaybeUse());
				}
				break;
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_FRESNEL_LAYER:
				check(Inputs[2].HasExpression());
				check(Inputs[3].HasExpression());
				check(Inputs[4].HasExpression());
				RerouteNormal(Inputs[4], Inputs[2].GetExpressionAndMaybeUse());
				break;
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_CUSTOM_CURVE_LAYER:
				check(Inputs[4].GetConnectionType() == EConnectionType::Expression);
				check(Inputs[5].HasExpression());
				check(Inputs[6].HasExpression());
				check(Inputs[4].GetExpressionUnused() ||
				      (Inputs[6].IsExpressionA<UMaterialExpressionMaterialFunctionCall>() &&
				       Cast<UMaterialExpressionMaterialFunctionCall>(Inputs[6].GetExpressionUnused())->MaterialFunction->GetName() ==
				           TEXT("mdl_state_normal")));
				if (Inputs[4].GetExpressionAndMaybeUse())
				{
					RerouteNormal(Inputs[6], Inputs[4].GetExpressionAndMaybeUse());
				}
				break;
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_MEASURED_CURVE_LAYER:
				check(Inputs[2].HasExpression());
				check(Inputs[3].HasExpression());
				check(Inputs[4].HasExpression());
				RerouteNormal(Inputs[4], Inputs[2].GetExpressionAndMaybeUse());
				break;
		}

		// fresnel_layer and custom_curve_layer use the clear coat shading model
		switch (Semantic)
		{
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_FRESNEL_LAYER:
				check(Inputs[4].GetConnectionType() == EConnectionType::Expression);
				SetClearCoatNormal(Inputs[3], Inputs[4].GetExpressionAndMaybeUse());
				break;
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_CUSTOM_CURVE_LAYER:
				check(Inputs[6].GetConnectionType() == EConnectionType::Expression);
				SetClearCoatNormal(Inputs[5], Inputs[6].GetExpressionAndMaybeUse());
				break;
		}
	}

	bool FMaterialExpressionFactory::CreateFunctionCallInputs(const mi::neuraylib::IFunction_definition&    InFunctionDefinition,
	                                                          const mi::neuraylib::IExpression_direct_call& InMDLFunctionCall,
	                                                          const FString& InCallPath, int32& OutArraySize,
	                                                          FMaterialExpressionConnectionList& OutInputs)
	{
		const mi::base::Handle<const mi::neuraylib::IExpression_list> DefaultArguments(InFunctionDefinition.get_defaults());
		const mi::base::Handle<const mi::neuraylib::IExpression_list> Arguments(InMDLFunctionCall.get_arguments());
		const mi::base::Handle<const mi::neuraylib::IType_list>       ParameterTypes(InFunctionDefinition.get_parameter_types());

		OutArraySize = 0;
		for (mi::Size i = 0; i < Arguments->get_size(); i++)
		{
			const mi::base::Handle<const mi::neuraylib::IExpression> Argument(Arguments->get_expression(i));
			mi::base::Handle<const mi::neuraylib::IType>             ParameterType(ParameterTypes->get_type(i));

			// Use argument type instead of parameter type in case of arrays so to have immediate size information
			if (ParameterType == nullptr || ParameterType->get_kind() == mi::neuraylib::IType::TK_ARRAY)
			{
				ParameterType = mi::base::make_handle(Argument->get_type());
			}

			mi::neuraylib::IType::Kind ParameterKind = mi::base::make_handle(ParameterType->skip_all_type_aliases())->get_kind();
			if (ParameterKind == mi::neuraylib::IType::TK_STRING)
				continue;  // ignore parameter

			// Apply additional processing if parameter type is an array
			if (ParameterKind == mi::neuraylib::IType::TK_ARRAY)
			{
				check(0 == OutArraySize);
				OutArraySize = CalcArraySize(Argument);
				check(0 < OutArraySize);
			}

			const FString                     ArgumentName        = ANSI_TO_TCHAR(Arguments->get_name(i));
			FMaterialExpressionConnectionList ArgumentExpressions = CreateExpression(Argument, InCallPath + TEXT(".") + ArgumentName);
			if (ArgumentExpressions.Num() == 0)
			{
				return false;
			}

			// mark default arguments as default
			const mi::base::Handle<const mi::neuraylib::IExpression> DefaultArgument(
			    DefaultArguments->get_expression(InFunctionDefinition.get_parameter_name(i)));

			mi::base::Handle<mi::neuraylib::IExpression_factory> ExpressionFactory(MdlFactory->create_expression_factory(CurrentTransaction));
			if (ExpressionFactory->compare(Argument.get(), DefaultArgument.get()) == 0)
			{
				for (int32 k = 0; k < ArgumentExpressions.Num(); k++)
				{
					check(ArgumentExpressions[k].GetConnectionType() == EConnectionType::Expression);
					ArgumentExpressions[k].SetExpressionDefault();
				}
			}

			OutInputs.Append(ArgumentExpressions);
		}
		return true;
	}

	FMaterialExpressionConnectionList FMaterialExpressionFactory::CreateExpressionFunctionCall(
	    const mi::neuraylib::IExpression_direct_call& MDLFunctionCall, const FString& CallPath)
	{
		const char* FName = MDLFunctionCall.get_definition();

		const mi::base::Handle<const mi::neuraylib::IFunction_definition> FunctionDefinition(
		    CurrentTransaction->access<mi::neuraylib::IFunction_definition>(FName));
		const FString FunctionName(ANSI_TO_TCHAR(FunctionDefinition->get_mdl_name()));

		bool IsProcessingNormalmap = FunctionName.StartsWith(TEXT("::base::tangent_space_normal_texture"));
		if (IsProcessingNormalmap)
		{
			SetProcessingNormapMap(true);
			ConstantExpressionFactory.SetProcessingNormapMap(IsProcessingNormalmap);
			ParameterExpressionFactory.SetProcessingNormapMap(IsProcessingNormalmap);
		}

		int32                             ArraySize;
		FMaterialExpressionConnectionList Inputs;
		if (!CreateFunctionCallInputs(*FunctionDefinition, MDLFunctionCall, CallPath, ArraySize, Inputs))
			return {};  // function has no arguments

		if (IsProcessingNormalmap)
		{
			SetProcessingNormapMap(false);
			ConstantExpressionFactory.SetProcessingNormapMap(false);
			ParameterExpressionFactory.SetProcessingNormapMap(false);
		}

		mi::neuraylib::IFunction_definition::Semantics Semantic = FunctionDefinition->get_semantic();
		HandleNormal(Semantic, Inputs);

		if (FunctionName == "::base::texture_coordinate_info(float3,float3,float3)")
		{
			const mi::base::Handle<const mi::neuraylib::IType_list> ParameterTypes(FunctionDefinition->get_parameter_types());

			const FString AssetNamePostfix = GetAssetNamePostFix(Semantic, *ParameterTypes);
			return MakeFunctionCall(CallPath, FunctionName, ArraySize, AssetNamePostfix, Inputs);
		}

		// general material function and operator handling
		switch (Semantic)
		{
			// handle all the functions
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_COS:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_LOG:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_LOG2:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_LOG10:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_SIN:
				check(Inputs.Num() == 1);
			case mi::neuraylib::IFunction_definition::DS_UNKNOWN:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_ANISOTROPIC_VDF:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_BACKSCATTERING_GLOSSY_REFLECTION_BSDF:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_CUSTOM_CURVE_LAYER:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_DIFFUSE_EDF:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_DIFFUSE_REFLECTION_BSDF:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_DIFFUSE_TRANSMISSION_BSDF:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_DIRECTIONAL_FACTOR:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_FRESNEL_LAYER:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_LIGHT_PROFILE_MAXIMUM:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_LIGHT_PROFILE_POWER:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_MEASURED_BSDF:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_MEASURED_CURVE_FACTOR:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_MEASURED_EDF:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_MICROFACET_BECKMANN_SMITH_BSDF:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_MICROFACET_BECKMANN_VCAVITIES_BSDF:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_MICROFACET_GGX_SMITH_BSDF:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_MICROFACET_GGX_VCAVITIES_BSDF:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_NORMALIZED_MIX:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_SPECULAR_BSDF:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_SIMPLE_GLOSSY_BSDF:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_SPOT_EDF:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_THIN_FILM:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_TINT:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_WARD_GEISLER_MORODER_BSDF:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_WEIGHTED_LAYER:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_AVERAGE:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_LUMINANCE:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_MAX_VALUE:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_MIN_VALUE:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_STATE_ANIMATION_TIME:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_STATE_DIRECTION:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_STATE_GEOMETRY_NORMAL:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_STATE_GEOMETRY_TANGENT_U:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_STATE_GEOMETRY_TANGENT_V:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_STATE_METERS_PER_SCENE_UNIT:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_STATE_OBJECT_ID:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_STATE_POSITION:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_STATE_SCENE_UNITS_PER_METER:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_STATE_TANGENT_SPACE:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_STATE_TEXTURE_COORDINATE:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_STATE_TEXTURE_SPACE_MAX:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_STATE_TEXTURE_TANGENT_U:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_STATE_TEXTURE_TANGENT_V:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_STATE_TRANSFORM_POINT:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_STATE_TRANSFORM_VECTOR:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_TEX_LOOKUP_COLOR:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_TEX_LOOKUP_FLOAT2:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_TEX_LOOKUP_FLOAT3:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_TEX_LOOKUP_FLOAT4:
			{
				const mi::base::Handle<const mi::neuraylib::IType_list> ParameterTypes(FunctionDefinition->get_parameter_types());

				const FString AssetNamePostfix = GetAssetNamePostFix(Semantic, *ParameterTypes);
				return MakeFunctionCall(CallPath, FunctionName, ArraySize, AssetNamePostfix, Inputs);
			}

			case mi::neuraylib::IFunction_definition::DS_CONV_CONSTRUCTOR:
			case mi::neuraylib::IFunction_definition::DS_ELEM_CONSTRUCTOR:
			case mi::neuraylib::IFunction_definition::DS_MATRIX_ELEM_CONSTRUCTOR:
			case mi::neuraylib::IFunction_definition::DS_MATRIX_DIAG_CONSTRUCTOR:
			{
				const mi::base::Handle<const mi::neuraylib::IType> ReturnType(FunctionDefinition->get_return_type());
				return CreateExpressionConstructorCall(*ReturnType, Inputs);
			}

				// unary operators
			case mi::neuraylib::IFunction_definition::DS_CONV_OPERATOR:
			case mi::neuraylib::IFunction_definition::DS_LOGICAL_NOT:
			case mi::neuraylib::IFunction_definition::DS_POSITIVE:
			case mi::neuraylib::IFunction_definition::DS_NEGATIVE:
			case mi::neuraylib::IFunction_definition::DS_PRE_INCREMENT:
			case mi::neuraylib::IFunction_definition::DS_PRE_DECREMENT:
			{
				const mi::base::Handle<const mi::neuraylib::IType> ReturnType(FunctionDefinition->get_return_type());
				return CreateExpressionUnary(Semantic, Inputs, *ReturnType);
			}

				// binary operators
			case mi::neuraylib::IFunction_definition::DS_MULTIPLY:
				// TODO: Matrix * Vector and Matrix * Matrix for all matrix sizes
				if (Inputs.Num() == 8)
				{
					const mi::base::Handle<const mi::neuraylib::IType_list> ParameterTypes(FunctionDefinition->get_parameter_types());

					const FString AssetNamePostfix = GetAssetNamePostFix(Semantic, *ParameterTypes);
					return MakeFunctionCall(CallPath, FunctionName, ArraySize, AssetNamePostfix, Inputs);
				}
				else
				{
					check(Inputs.Num() == 2);

					return {NewMaterialExpressionMultiply(CurrentMaterial, Inputs[0], Inputs[1])};
				}
			case mi::neuraylib::IFunction_definition::DS_DIVIDE:
			case mi::neuraylib::IFunction_definition::DS_MODULO:
			case mi::neuraylib::IFunction_definition::DS_PLUS:
			case mi::neuraylib::IFunction_definition::DS_MINUS:
			case mi::neuraylib::IFunction_definition::DS_LESS:
			case mi::neuraylib::IFunction_definition::DS_LESS_OR_EQUAL:
			case mi::neuraylib::IFunction_definition::DS_GREATER_OR_EQUAL:
			case mi::neuraylib::IFunction_definition::DS_GREATER:
			case mi::neuraylib::IFunction_definition::DS_EQUAL:
			case mi::neuraylib::IFunction_definition::DS_NOT_EQUAL:
			case mi::neuraylib::IFunction_definition::DS_LOGICAL_AND:
			case mi::neuraylib::IFunction_definition::DS_LOGICAL_OR:
				return CreateExpressionBinary(Semantic, Inputs);

			case mi::neuraylib::IFunction_definition::DS_TERNARY:
				return CreateExpressionTernary(Inputs);

				// math intrinsics
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_ABS:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_ACOS:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_ASIN:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_ATAN:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_ATAN2:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_CEIL:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_CLAMP:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_CROSS:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_DEGREES:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_DISTANCE:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_DOT:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_EXP:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_EXP2:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_FLOOR:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_FMOD:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_FRAC:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_LENGTH:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_LERP:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_MAX:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_MIN:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_MODF:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_NORMALIZE:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_POW:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_RADIANS:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_ROUND:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_RSQRT:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_SATURATE:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_SIGN:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_SINCOS:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_SQRT:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_TAN:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_BLACKBODY:
				return CreateExpressionMath(Semantic, Inputs);

				// state intrinsics
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_STATE_NORMAL:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_STATE_ROUNDED_CORNER_NORMAL:

				// tex intrinsics
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_TEX_WIDTH:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_TEX_HEIGHT:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_TEX_DEPTH:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_TEX_TEXTURE_ISVALID:
				return CreateExpressionOther(Semantic, Inputs);

				// DAG intrinsics
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DAG_FIELD_ACCESS:
			case mi::neuraylib::IFunction_definition::DS_INTRINSIC_DAG_ARRAY_CONSTRUCTOR:
			case mi::neuraylib::IFunction_definition::DS_ARRAY_INDEX:
				return CreateExpressionDAG(Semantic, Inputs, FunctionName);

			default:
				check(false);  // needs to be handled, if ever encountered!
				return {};
		}
	}

	FMaterialExpressionConnectionList FMaterialExpressionFactory::MakeFunctionCall(const FString& InCallPath, const FString& InFunctionName,
	                                                                               int32 InArraySize, const FString& InAssetNamePostfix,
	                                                                               FMaterialExpressionConnectionList& OutInputs)
	{
		FString AssetName(InFunctionName);
		if (AssetName == TEXT("::state::texture_coordinate(int)"))
		{
			// create a texture coodrinate expression with the correct index
			check(OutInputs.Num() == 1);

			UMaterialExpression* InputExpression = OutInputs[0].GetExpressionUnused();
			int32 CoordinateIndex = 0;
			if (UMaterialExpressionConstant* Index = Cast<UMaterialExpressionConstant>(InputExpression))
			{
				CoordinateIndex = Index->R;
				CurrentMaterial->GetExpressionCollection().RemoveExpression(Index);
			}
			else if (UMaterialExpressionScalarParameter* ScalarExpression = Cast<UMaterialExpressionScalarParameter>(InputExpression))
			{
				CoordinateIndex = ScalarExpression->DefaultValue;
				CurrentMaterial->GetExpressionCollection().RemoveExpression(ScalarExpression);
			}
			else
			{
				
				FString Name = InputExpression ? InputExpression->GetClass()->GetName() : "<null>";
				LogMessages.Emplace(MDLImporterLogging::EMessageSeverity::Warning, TEXT("Not supported expression for texture coordinate :") + Name);
			}
			OutInputs[0] = { NewMaterialExpressionTextureCoordinate(CurrentMaterial, CoordinateIndex) };
			AssetName = TEXT("_state_texture_coordinate");
		}
		else if (AssetName == TEXT("::state::texture_tangent_u(int)") || AssetName == TEXT("::state::texture_tangent_v(int)") ||
		         AssetName == TEXT("::state::geometry_tangent_u(int)") || AssetName == TEXT("::state::geometry_tangent_v(int)"))
		{
			// ignore the index parameter
			check(OutInputs.Num() == 1);

			OutInputs.Empty();

			AssetName = AssetName.Left(AssetName.Find(TEXT("("))).Replace(TEXT("::"), TEXT("_"));
		}
		else if (AssetName == TEXT("::base::coordinate_source(::base::texture_coordinate_system,int)"))
		{
			check(OutInputs.Num() == 2);
			if (bIsGeometryExpression)
			{  // TODO: in the geometry expression path, we don't support the full functionality of base::coordinate_source -> fall back to just the
				// base::texture_coordinate_info constructor
				OutInputs[0] = FMaterialExpressionConnection();
			}
			else
			{
				UMaterialExpressionConstant* Index = Cast<UMaterialExpressionConstant>(OutInputs[0].GetExpressionUnused());
			}
			OutInputs.SetNum(1); // base::coordinate_source will use only the first param(the coordinate_system)
			AssetName = TEXT("_base_coordinate_source");
		}
		else if (AssetName == TEXT("operator*(float4x4,float4x4)"))
		{
			AssetName = TEXT("_math_multiply_float4x4_float4x4");
		}
		else if (AssetName.Contains(TEXT("::distilling_support")))
		{
			AssetName =
			    AssetName.Replace(TEXT("("), TEXT("_")).Replace(TEXT(")"), TEXT("")).Replace(TEXT(","), TEXT("_")).Replace(TEXT("::"), TEXT("_"));
		}
		else
		{
			AssetName = AssetName.Left(AssetName.Find(TEXT("("))).Replace(TEXT("::"), TEXT("_"));
		}
		AssetName = TEXT("mdl") + AssetName + InAssetNamePostfix;

		FMaterialExpressionConnectionList Outputs;

		UMaterialFunction* Function = FunctionLoader->Load(AssetName, InArraySize);
		if (!Function)
			return Outputs;

		int32 TextureSelectionIndex = OutInputs.FindLastByPredicate(
		    [](FMaterialExpressionConnection const& MEC) { return MEC.GetConnectionType() == EConnectionType::TextureSelection; });
		if (TextureSelectionIndex != INDEX_NONE)
		{
			FMaterialExpressionConnection::FData Value = OutInputs[TextureSelectionIndex].GetTextureSelectionData()[0];
			FMaterialExpressionConnection::FData True  = OutInputs[TextureSelectionIndex].GetTextureSelectionData()[1];
			FMaterialExpressionConnection::FData False = OutInputs[TextureSelectionIndex].GetTextureSelectionData()[2];
			OutInputs[TextureSelectionIndex]           = {True.GetMaterialExpression(), True.Index, True.bIsDefault};
			check(OutInputs.FindLastByPredicate([](FMaterialExpressionConnection const& MEC) {
				return MEC.GetConnectionType() == EConnectionType::TextureSelection;
			}) == INDEX_NONE);
			UMaterialExpressionMaterialFunctionCall* TrueCall  = NewMaterialExpressionFunctionCall(CurrentMaterial, Function, OutInputs);
			OutInputs[TextureSelectionIndex]                   = {False.GetMaterialExpression(), False.Index, False.bIsDefault};
			UMaterialExpressionMaterialFunctionCall* FalseCall = NewMaterialExpressionFunctionCall(CurrentMaterial, Function, OutInputs);
			check(TrueCall->Outputs.Num() == FalseCall->Outputs.Num());

			FMaterialExpressionConnection ValueConnection(Value.GetMaterialExpression(), Value.Index, Value.bIsDefault);
			Outputs.Reserve(TrueCall->Outputs.Num());
			for (int32 i = 0; i < TrueCall->Outputs.Num(); i++)
			{
				if (IsStatic(ValueConnection))
				{
					Outputs.Add(NewMaterialExpressionStaticSwitch(CurrentMaterial, ValueConnection, {TrueCall, i}, {FalseCall, i}));
				}
				else
				{
					Outputs.Add(NewMaterialExpressionIfEqual(CurrentMaterial, ValueConnection, 0.0f, {FalseCall, i}, {TrueCall, i}));
				}
			}
		}
		else
		{
			UMaterialExpressionMaterialFunctionCall* FunctionCall = NewMaterialExpressionFunctionCall(CurrentMaterial, Function, OutInputs);
			Outputs.Reserve(FunctionCall->Outputs.Num());
			for (int32 i = 0; i < FunctionCall->Outputs.Num(); i++)
			{
				Outputs.Add(FMaterialExpressionConnection(FunctionCall, i));
			}
		}

		return Outputs;
	}

	FMaterialExpressionConnectionList FMaterialExpressionFactory::CreateExpressionConstructorCall(const mi::neuraylib::IType& MDLType,
	                                                                                              const FMaterialExpressionConnectionList& Inputs)
	{
		UMaterialFunction* MakeFloat2 = &FunctionLoader->Get(ECommonFunction::MakeFloat2);
		UMaterialFunction* MakeFloat3 = &FunctionLoader->Get(ECommonFunction::MakeFloat3);
		UMaterialFunction* MakeFloat4 = &FunctionLoader->Get(ECommonFunction::MakeFloat4);

		mi::neuraylib::IType::Kind Kind = MDLType.get_kind();

		switch (Kind)
		{
			case mi::neuraylib::IType::TK_FLOAT:
			{
				if (!ensure(Inputs[0].GetConnectionType() == EConnectionType::Expression))
				{
					break;
				}

				if (Inputs.Num() == 1)
				{
					if ((Inputs[0].IsExpressionA<UMaterialExpressionStaticBool>() ||
					     Inputs[0].IsExpressionA<UMaterialExpressionStaticBoolParameter>()))
					{
						return {NewMaterialExpressionStaticSwitch(CurrentMaterial, Inputs[0], 1.0f, 0.0f)};
					}
					if (Inputs[0].IsExpressionA<UMaterialExpressionScalarParameter>())
					{
						return Inputs;
					}
				}
				break;
			}
			case mi::neuraylib::IType::TK_VECTOR:
			{
				const mi::base::Handle<const mi::neuraylib::IType_vector> Type(MDLType.get_interface<const mi::neuraylib::IType_vector>());

				int32 VectorSize = (int32)Type->get_size();

				if (Inputs.Num() == 1)
				{
					if (IsScalar(Inputs[0]))
					{
						return { NewMaterialExpressionFunctionCall(CurrentMaterial, MakeFloat3, {Inputs[0], Inputs[0], Inputs[0]}) };
					}
					if (Inputs[0].IsExpressionA<UMaterialExpressionVectorParameter>())
					{
						return Inputs;
					}
				}
				else if (Inputs.Num() == VectorSize)
				{
					switch (VectorSize)
					{
						case 2:
						{
							if (ensure(IsScalar(Inputs[0]) && IsScalar(Inputs[1])))
							{
								return { NewMaterialExpressionFunctionCall(CurrentMaterial, MakeFloat2, {Inputs[0], Inputs[1]}) };
							}
							break;
						}
						case 3:
						{
							if (ensure(IsScalar(Inputs[0]) && IsScalar(Inputs[1]) && IsScalar(Inputs[2])))
							{
								return { NewMaterialExpressionFunctionCall(CurrentMaterial, MakeFloat3, {Inputs[0], Inputs[1], Inputs[2]}) };
							}
							break;
						}
						case 4:
						{
							if (ensure(IsScalar(Inputs[0]) && IsScalar(Inputs[1]) && IsScalar(Inputs[2]) && IsScalar(Inputs[3])))
							{
								return { NewMaterialExpressionFunctionCall(CurrentMaterial, MakeFloat4, {Inputs[0], Inputs[1], Inputs[2], Inputs[3]}) };
							}
							break;
						}
					}
				}
				break;
			}
			case mi::neuraylib::IType::TK_MATRIX:
			{
				const mi::base::Handle<const mi::neuraylib::IType_matrix> MatrixType(MDLType.get_interface<const mi::neuraylib::IType_matrix>());
				const mi::base::Handle<const mi::neuraylib::IType_vector> ElementType(MatrixType->get_element_type());
				const int32 NumRows = (int32)ElementType->get_size(), NumColumns = (int32)MatrixType->get_size();

				if (Inputs.Num() == 1)  // DS_MATRIX_DIAG_CONSTRUCTOR
				{
					UMaterialExpressionConstant* Zero = NewMaterialExpressionConstant(CurrentMaterial, 0.0f);

					FMaterialExpressionConnectionList Outputs;
					Outputs.Reserve(NumColumns);

					for (int32 i = 0; i < NumColumns; i++)
					{
						switch (NumRows)
						{
							case 2:
								Outputs.Add(NewMaterialExpressionFunctionCall(CurrentMaterial, MakeFloat2,
								                                              {i == 0 ? Inputs[0] : Zero, i == 1 ? Inputs[0] : Zero}));
								break;
							case 3:
								Outputs.Add(NewMaterialExpressionFunctionCall(
								    CurrentMaterial, MakeFloat3, {i == 0 ? Inputs[0] : Zero, i == 1 ? Inputs[0] : Zero, i == 2 ? Inputs[0] : Zero}));
								break;
							case 4:
								Outputs.Add(NewMaterialExpressionFunctionCall(
								    CurrentMaterial, MakeFloat4,
								    {i == 0 ? Inputs[0] : Zero, i == 1 ? Inputs[0] : Zero, i == 2 ? Inputs[0] : Zero, i == 3 ? Inputs[0] : Zero}));
								break;
						}
					}

					return Outputs;
				}
				else if (Inputs.Num() == NumColumns)
				{
					return Inputs;
				}
				else if (Inputs.Num() == NumColumns * NumRows)  // DS_MATRIX_ELEM_CONSTRUCTOR
				{
					FMaterialExpressionConnectionList Outputs;
					Outputs.Reserve(NumColumns);

					for (int32 i = 0; i < NumColumns; i++)
					{
						switch (NumRows)
						{
							case 2:
								check(IsScalar(Inputs[i * NumColumns]) && IsScalar(Inputs[i * NumColumns + 1]));
								Outputs.Add(NewMaterialExpressionFunctionCall(CurrentMaterial, MakeFloat2,
								                                              {Inputs[i * NumColumns], Inputs[i * NumColumns + 1]}));
								break;
							case 3:
								check(IsScalar(Inputs[i * NumColumns]) && IsScalar(Inputs[i * NumColumns + 1]) &&
								      IsScalar(Inputs[i * NumColumns + 2]));
								Outputs.Add(NewMaterialExpressionFunctionCall(
								    CurrentMaterial, MakeFloat3, {Inputs[i * NumColumns], Inputs[i * NumColumns + 1], Inputs[i * NumColumns + 2]}));
								break;
							case 4:
								check(IsScalar(Inputs[i * NumColumns]) && IsScalar(Inputs[i * NumColumns + 1]) &&
								      IsScalar(Inputs[i * NumColumns + 2]) && IsScalar(Inputs[i * NumColumns + 3]));
								Outputs.Add(NewMaterialExpressionFunctionCall(
								    CurrentMaterial, MakeFloat4,
								    {Inputs[i * NumColumns], Inputs[i * NumColumns + 1], Inputs[i * NumColumns + 2], Inputs[i * NumColumns + 3]}));
								break;
						}
					}

					return Outputs;
				}
				break;
			}
			case mi::neuraylib::IType::TK_COLOR:
			{
				switch (Inputs.Num())
				{
					case 1:
						if (IsVector3(Inputs[0]))
						{
							return {Inputs[0]};
						}
						else
						{
							check(IsScalar(Inputs[0]));
							return {NewMaterialExpressionFunctionCall(CurrentMaterial, MakeFloat3, {Inputs[0], Inputs[0], Inputs[0]})};
						}
					case 3:
						check(IsScalar(Inputs[0]) && IsScalar(Inputs[1]) && IsScalar(Inputs[2]));
						return {NewMaterialExpressionFunctionCall(CurrentMaterial, MakeFloat3, {Inputs[0], Inputs[1], Inputs[2]})};
				}
				break;
			}
			case mi::neuraylib::IType::TK_STRUCT:
			{
				return Inputs;
			}
		}


		ensure(false);
		// We couldn't handle constructor call, print out the type that it was called for
		mi::base::Handle<mi::neuraylib::IExpression_factory> ExpressionFactory(MdlFactory->create_expression_factory(CurrentTransaction));
		mi::base::Handle<mi::neuraylib::IType_factory> TypeFactory(MdlFactory->create_type_factory(CurrentTransaction));
		mi::base::Handle<const mi::IString> TypeStr(TypeFactory->dump(&MDLType, 1));
		FString TypeString = ANSI_TO_TCHAR(TypeStr->get_c_str());

		LogMessages.Emplace(MDLImporterLogging::EMessageSeverity::Warning,
			FString::Printf(TEXT("Can't construct '%s' from %d Inputs"), *TypeString, Inputs.Num()));
		
		return Inputs;
	}

	void FMaterialExpressionFactory::SetClearCoatNormal(const FMaterialExpressionConnection& Base, const UMaterialExpression* Normal)
	{
		check(Base.GetConnectionType() == EConnectionType::Expression);
		UMaterialExpression* BaseNormal = nullptr;
		if (Base.IsExpressionA<UMaterialExpressionIf>())
		{
			UMaterialExpressionIf* If = Cast<UMaterialExpressionIf>(Base.GetExpressionUnused());
			check(If->ALessThanB.Expression->IsA<UMaterialExpressionMaterialFunctionCall>() &&
			      If->AEqualsB.Expression->IsA<UMaterialExpressionMaterialFunctionCall>() &&
			      If->AGreaterThanB.Expression->IsA<UMaterialExpressionMaterialFunctionCall>());
			UMaterialExpression* BaseNormalALessThanB =
			    Cast<UMaterialExpressionMaterialFunctionCall>(If->ALessThanB.Expression)->FunctionInputs.Last().Input.Expression;
			UMaterialExpression* BaseNormalAEqualsB =
			    Cast<UMaterialExpressionMaterialFunctionCall>(If->AEqualsB.Expression)->FunctionInputs.Last().Input.Expression;
			UMaterialExpression* BaseNormalAGreaterThanB =
			    Cast<UMaterialExpressionMaterialFunctionCall>(If->AGreaterThanB.Expression)->FunctionInputs.Last().Input.Expression;
			if ((BaseNormalALessThanB && (BaseNormalALessThanB != Normal)) || (BaseNormalAEqualsB && (BaseNormalAEqualsB != Normal)) ||
			    (BaseNormalAGreaterThanB && (BaseNormalAGreaterThanB != Normal)))
			{
				BaseNormal = NewMaterialExpressionIf(CurrentMaterial, {If->A.Expression, If->A.OutputIndex}, {If->B.Expression, If->B.OutputIndex},
				                                     BaseNormalALessThanB, BaseNormalAEqualsB, BaseNormalAGreaterThanB);
			}
		}
		else if (Base.IsExpressionA<UMaterialExpressionStaticSwitch>())
		{
			UMaterialExpressionStaticSwitch* StaticSwitch = Cast<UMaterialExpressionStaticSwitch>(Base.GetExpressionUnused());
			check(StaticSwitch->A.Expression->IsA<UMaterialExpressionMaterialFunctionCall>() &&
			      StaticSwitch->B.Expression->IsA<UMaterialExpressionMaterialFunctionCall>());
			UMaterialExpression* BaseNormalA =
			    Cast<UMaterialExpressionMaterialFunctionCall>(StaticSwitch->A.Expression)->FunctionInputs.Last().Input.Expression;
			UMaterialExpression* BaseNormalB =
			    Cast<UMaterialExpressionMaterialFunctionCall>(StaticSwitch->B.Expression)->FunctionInputs.Last().Input.Expression;
			if ((BaseNormalA && (BaseNormalA != Normal)) || (BaseNormalB && (BaseNormalB != Normal)))
			{
				BaseNormal = NewMaterialExpressionStaticSwitch(CurrentMaterial, {StaticSwitch->Value.Expression, StaticSwitch->Value.OutputIndex},
				                                               BaseNormalA, BaseNormalB);
			}
		}
		else
		{
			check(Base.IsExpressionA<UMaterialExpressionMakeMaterialAttributes>() ||
			      Base.IsExpressionA<UMaterialExpressionMaterialFunctionCall>());
			BaseNormal = Base.IsExpressionA<UMaterialExpressionMakeMaterialAttributes>()
			                 ? Cast<UMaterialExpressionMakeMaterialAttributes>(Base.GetExpressionAndUse())->Normal.Expression
			                 : Cast<UMaterialExpressionMaterialFunctionCall>(Base.GetExpressionAndUse())->FunctionInputs.Last().Input.Expression;
		}
		if (BaseNormal && (BaseNormal != Normal))
		{
			NewMaterialExpressionClearCoatNormalCustomOutput(CurrentMaterial, BaseNormal);
		}
	}
}  // namespace Generator

#endif
