// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "generator/ConstantExpressionFactory.h"
#include "generator/ParameterExpressionFactory.h"


class UMaterialFunction;
class UMaterialExpression;
namespace mi
{
	namespace neuraylib
	{
		class IExpression;
		class IExpression_temporary;
		class IExpression_direct_call;
		class ITransaction;
		class IType;
		class IFunction_definition;
		class IMdl_factory;
	}
	namespace base
	{
		template <class Interface>
		class Handle;
	}
}
namespace Mdl
{
	class FApiContext;
}

namespace Generator
{
	class FFunctionLoader;
	class FMaterialTextureFactory;

	class FMaterialExpressionFactory : public FBaseExpressionFactory
	{
	public:
		explicit FMaterialExpressionFactory(const Mdl::FApiContext& MdlContext);

		void SetCurrentMaterial(const mi::neuraylib::IMaterial_definition& MDLMaterialDefinition,
		                        const mi::neuraylib::ICompiled_material&   MDLMaterial,
		                        mi::neuraylib::ITransaction&               MDLTransaction,
		                        UMaterial&                                 Material);

		void CreateParameterExpressions();

		void SetCurrentNormal(UMaterialExpression* NormalExpression, bool bIsGeometryExpression);

		void SetFunctionLoader(FFunctionLoader* FunctionLoader);
		void SetTextureFactory(FMaterialTextureFactory* Factory);

		FMaterialExpressionConnectionList CreateExpression(const mi::base::Handle<const mi::neuraylib::IExpression>& MDLExpression,
		                                                   const FString&                                            CallPath);

		void Cleanup(bool bOnlyTemporaries = false);

		// remove redundant material expressions
		void CleanupMaterialExpressions();

		TArray<MDLImporterLogging::FLogMessage> GetLogMessages();

	private:
		FMaterialExpressionConnectionList CreateExpressionTemporary(const mi::neuraylib::IExpression_temporary& MDLExpression,
		                                                            const FString&                              CallPath);

		FMaterialExpressionConnectionList CreateExpressionFunctionCall(const mi::neuraylib::IExpression_direct_call& MDLCall,
		                                                               const FString&                                CallPath);

		FMaterialExpressionConnectionList CreateExpressionConstructorCall(const mi::neuraylib::IType&                  MDLType,
		                                                                  const FMaterialExpressionConnectionList& Inputs);

		bool CreateFunctionCallInputs(const mi::neuraylib::IFunction_definition&    InFunctionDefinition,
		                              const mi::neuraylib::IExpression_direct_call& InMDLFunctionCall,
		                              const FString&                                InCallPath,
		                              int32&                                        OutArraySize,
		                              FMaterialExpressionConnectionList&            OutInputs);

		FMaterialExpressionConnectionList MakeFunctionCall(const FString&                         InCallPath,      //
		                                                   const FString&                         InFunctionName,  //
		                                                   int32                                  InArraySize,     //
		                                                   const FString&                         InAssetNamePostfix,
		                                                   FMaterialExpressionConnectionList& OutInputs);

		FMaterialExpressionConnectionList CreateExpressionUnary(int                                          Semantic,
		                                                        const FMaterialExpressionConnectionList& Inputs,
		                                                        const mi::neuraylib::IType&                  MDLType);

		FMaterialExpressionConnectionList CreateExpressionBinary(int                                          Semantic,  //
		                                                         const FMaterialExpressionConnectionList& Inputs);

		FMaterialExpressionConnectionList CreateExpressionTernary(const FMaterialExpressionConnectionList& Inputs);

		FMaterialExpressionConnectionList CreateExpressionMath(int                                          Semantic,  //
		                                                       const FMaterialExpressionConnectionList& Inputs);

		FMaterialExpressionConnectionList CreateExpressionDAG(int                                          Semantic,
		                                                      const FMaterialExpressionConnectionList& Inputs,
		                                                      const FString&                               FunctionName);

		FMaterialExpressionConnectionList CreateExpressionOther(int Semantic, const FMaterialExpressionConnectionList& Inputs);

		void HandleNormal(int Semantic, FMaterialExpressionConnectionList& Inputs);
		void SetClearCoatNormal(const FMaterialExpressionConnection& Base, const UMaterialExpression* Normal);

	private:
		FConstantExpressionFactory  ConstantExpressionFactory;
		FParameterExpressionFactory ParameterExpressionFactory;

		FMaterialTextureFactory* TextureFactory;

		mi::neuraylib::IMdl_factory* MdlFactory;
		mi::neuraylib::ITransaction* CurrentTransaction;
		UMaterialExpression*         CurrentNormalExpression;
		bool                         bIsGeometryExpression;

		FFunctionLoader* FunctionLoader;

		TArray<FMaterialExpressionConnectionList> Temporaries;
	};

	inline void FMaterialExpressionFactory::SetCurrentNormal(UMaterialExpression* InNormalExpression, bool InIsGeometryExpression)
	{
		ensure(InNormalExpression);
		CurrentNormalExpression = InNormalExpression;
		bIsGeometryExpression   = InIsGeometryExpression;
	}

	inline void FMaterialExpressionFactory::SetFunctionLoader(FFunctionLoader* InFunctionLoader)
	{
		FunctionLoader = InFunctionLoader;
	}

	inline void FMaterialExpressionFactory::Cleanup(bool bOnlyTemporaries /*= false*/)
	{
		if (bOnlyTemporaries)
		{
			for (FMaterialExpressionConnectionList& Temporary : Temporaries)
			{
				Temporary.Empty();
			}
			return;
		}
		FBaseExpressionFactory::Cleanup();
		ConstantExpressionFactory.Cleanup();
		ParameterExpressionFactory.Cleanup();

		CurrentTransaction      = nullptr;
		CurrentNormalExpression = nullptr;
		bIsGeometryExpression   = false;
		Temporaries.Empty();
	}

	inline void FMaterialExpressionFactory::CleanupMaterialExpressions()
	{
		ParameterExpressionFactory.CleanupMaterialExpressions();
		ConstantExpressionFactory.CleanupMaterialExpressions();

	}

	inline TArray<MDLImporterLogging::FLogMessage> FMaterialExpressionFactory::GetLogMessages()
	{
		TArray<MDLImporterLogging::FLogMessage> Messages;
		Swap(Messages, LogMessages);
		Messages.Append(ConstantExpressionFactory.GetLogMessages());
		Messages.Append(ParameterExpressionFactory.GetLogMessages());
		return Messages;
	}
}  // namespace Generator
