// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialExpressionConnection.h"

#include "Materials/Material.h"
#include "Materials/MaterialFunction.h"

class UMaterialExpressionAbs;
class UMaterialExpressionAdd;
class UMaterialExpressionAppendVector;
class UMaterialExpressionArccosine;
class UMaterialExpressionArcsine;
class UMaterialExpressionArctangent2;
class UMaterialExpressionArctangent;
class UMaterialExpressionBlackBody;
class UMaterialExpressionBreakMaterialAttributes;
class UMaterialExpressionCeil;
class UMaterialExpressionClamp;
class UMaterialExpressionClearCoatNormalCustomOutput;
class UMaterialExpressionComponentMask;
class UMaterialExpressionConstant2Vector;
class UMaterialExpressionConstant3Vector;
class UMaterialExpressionConstant4Vector;
class UMaterialExpressionConstant;
class UMaterialExpressionCosine;
class UMaterialExpressionCrossProduct;
class UMaterialExpressionDistance;
class UMaterialExpressionDivide;
class UMaterialExpressionDotProduct;
class UMaterialExpressionFloor;
class UMaterialExpressionFmod;
class UMaterialExpressionFrac;
class UMaterialExpressionFresnel;
class UMaterialExpressionFunctionInput;
class UMaterialExpressionFunctionOutput;
class UMaterialExpressionIf;
class UMaterialExpressionLinearInterpolate;
class UMaterialExpressionLogarithm2;
class UMaterialExpressionMakeMaterialAttributes;
class UMaterialExpressionMaterialFunctionCall;
class UMaterialExpressionMax;
class UMaterialExpressionMin;
class UMaterialExpressionMultiply;
class UMaterialExpressionNoise;
class UMaterialExpressionNormalize;
class UMaterialExpressionOneMinus;
class UMaterialExpressionPower;
class UMaterialExpressionReflectionVectorWS;
class UMaterialExpressionSaturate;
class UMaterialExpressionScalarParameter;
class UMaterialExpressionSine;
class UMaterialExpressionSquareRoot;
class UMaterialExpressionStaticBool;
class UMaterialExpressionStaticBoolParameter;
class UMaterialExpressionStaticSwitch;
class UMaterialExpressionSubtract;
class UMaterialExpressionTangent;
class UMaterialExpressionTextureCoordinate;
class UMaterialExpressionTextureObject;
class UMaterialExpressionTextureObjectParameter;
class UMaterialExpressionTextureProperty;
class UMaterialExpressionTextureSample;
class UMaterialExpressionTransform;
class UMaterialExpressionTransformPosition;
class UMaterialExpressionTwoSidedSign;
class UMaterialExpressionVectorNoise;
class UMaterialExpressionVectorParameter;
enum EFunctionInputType : int;
enum EMaterialExposedTextureProperty : int;
enum EMaterialPositionTransformSource : int;
enum EMaterialVectorCoordTransform : int;
enum EMaterialVectorCoordTransformSource : int;
enum EVectorNoiseFunction : int;

namespace Generator
{
	template <typename T>
	T* NewMaterialExpression(UObject* Parent);

	template <typename T>
	void Connect(T& Target, const FMaterialExpressionConnection& Connection);

	void SetMaterialExpressionGroup(const FString& GroupName, UMaterialExpression* ParameterExpression);

	bool IsBool(const FMaterialExpressionConnection& Input);
	bool IsMaterialAttribute(const FMaterialExpressionConnection& Input);
	bool IsScalar(const FMaterialExpressionConnection& Input);
	bool IsStatic(const FMaterialExpressionConnection& Input);
	bool IsTexture(const FMaterialExpressionConnection& Input);
	bool IsVector3(const FMaterialExpressionConnection& Input);

	// Abs
	UMaterialExpressionAbs* NewMaterialExpressionAbs(UObject* Parent, const FMaterialExpressionConnection& Input);

	// Add
	UMaterialExpressionAdd* NewMaterialExpressionAdd(UObject* Parent, const FMaterialExpressionConnection& A, const FMaterialExpressionConnection& B);
	UMaterialExpressionAdd* NewMaterialExpressionAdd(UObject* Parent, const TArray<FMaterialExpressionConnection>& Arguments);

	// AppendVector
	UMaterialExpressionAppendVector* NewMaterialExpressionAppendVector(UObject*                             Parent,
	                                                                   const FMaterialExpressionConnection& A,
	                                                                   const FMaterialExpressionConnection& B);
	UMaterialExpressionAppendVector* NewMaterialExpressionAppendVector(UObject*                             Parent,
	                                                                   const FMaterialExpressionConnection& A,
	                                                                   const FMaterialExpressionConnection& B,
	                                                                   const FMaterialExpressionConnection& C);

	// Arccosine
	UMaterialExpressionArccosine* NewMaterialExpressionArccosine(UObject* Parent, const FMaterialExpressionConnection& Input);

	// Arcsine
	UMaterialExpressionArcsine* NewMaterialExpressionArcsine(UObject* Parent, const FMaterialExpressionConnection& Input);

	// Arctangent
	UMaterialExpressionArctangent* NewMaterialExpressionArctangent(UObject* Parent, const FMaterialExpressionConnection& Input);

	// Arctangent2
	UMaterialExpressionArctangent2* NewMaterialExpressionArctangent2(UObject*                             Parent,
	                                                                 const FMaterialExpressionConnection& Y,
	                                                                 const FMaterialExpressionConnection& X);

	// BlackBody
	UMaterialExpressionBlackBody* NewMaterialExpressionBlackBody(UObject* Parent, const FMaterialExpressionConnection& Temperature);

	// BreakMaterialAttributes
	UMaterialExpressionBreakMaterialAttributes* NewMaterialExpressionBreakMaterialAttributes(UObject*                             Parent,
	                                                                                         const FMaterialExpressionConnection& MaterialAttributes);

	// Ceil
	UMaterialExpressionCeil* NewMaterialExpressionCeil(UObject* Parent, const FMaterialExpressionConnection& Input);

	// Clamp
	UMaterialExpressionClamp* NewMaterialExpressionClamp(UObject*                             Parent,
	                                                     const FMaterialExpressionConnection& Input,
	                                                     const FMaterialExpressionConnection& Min,
	                                                     const FMaterialExpressionConnection& Max);

	// ComponentMask
	UMaterialExpressionComponentMask* NewMaterialExpressionComponentMask(UObject* Parent, const FMaterialExpressionConnection& Input, uint32 Mask);

	// ClearCoatNormalCustomOutput
	UMaterialExpressionClearCoatNormalCustomOutput* NewMaterialExpressionClearCoatNormalCustomOutput(UObject*                             Parent,
	                                                                                                 const FMaterialExpressionConnection& Input);

	// Constant
	UMaterialExpressionConstant*        NewMaterialExpressionConstant(UObject* Parent, float X);
	UMaterialExpressionConstant2Vector* NewMaterialExpressionConstant(UObject* Parent, float X, float Y);
	UMaterialExpressionConstant3Vector* NewMaterialExpressionConstant(UObject* Parent, float X, float Y, float Z);
	UMaterialExpressionConstant4Vector* NewMaterialExpressionConstant(UObject* Parent, float X, float Y, float Z, float W);

	// Cosine
	UMaterialExpressionCosine* NewMaterialExpressionCosine(UObject* Parent, const FMaterialExpressionConnection& Input);

	// CrossProduct
	UMaterialExpressionCrossProduct* NewMaterialExpressionCrossProduct(UObject*                             Parent,
	                                                                   const FMaterialExpressionConnection& A,
	                                                                   const FMaterialExpressionConnection& B);

	// Distance
	UMaterialExpressionDistance* NewMaterialExpressionDistance(UObject*                             Parent,
	                                                           const FMaterialExpressionConnection& A,
	                                                           const FMaterialExpressionConnection& B);

	// Divide
	UMaterialExpressionDivide* NewMaterialExpressionDivide(UObject*                             Parent,
	                                                       const FMaterialExpressionConnection& A,
	                                                       const FMaterialExpressionConnection& B);

	// DotProduct
	UMaterialExpressionDotProduct* NewMaterialExpressionDotProduct(UObject*                             Parent,
	                                                               const FMaterialExpressionConnection& A,
	                                                               const FMaterialExpressionConnection& B);

	// Floor
	UMaterialExpressionFloor* NewMaterialExpressionFloor(UObject* Parent, const FMaterialExpressionConnection& Input);

	// Fmod
	UMaterialExpressionFmod* NewMaterialExpressionFmod(UObject*                             Parent,
	                                                   const FMaterialExpressionConnection& A,
	                                                   const FMaterialExpressionConnection& B);

	// Frac
	UMaterialExpressionFrac* NewMaterialExpressionFrac(UObject* Parent, const FMaterialExpressionConnection& Input);

	// Fresnel
	UMaterialExpressionFresnel* NewMaterialExpressionFresnel(UObject*                             Parent,
	                                                         const FMaterialExpressionConnection& Exponent,
	                                                         const FMaterialExpressionConnection& BaseReflectFraction,
	                                                         const FMaterialExpressionConnection& Normal);

	// FunctionInput
	UMaterialExpressionFunctionInput* NewMaterialExpressionFunctionInput(UObject* Parent, const FString& Name, EFunctionInputType Type);
	UMaterialExpressionFunctionInput* NewMaterialExpressionFunctionInput(UObject*                             Parent,
	                                                                     const FString&                       Name,
	                                                                     EFunctionInputType                   Type,
	                                                                     const FMaterialExpressionConnection& DefaultExpression);

	// FunctionOutput
	UMaterialExpressionFunctionOutput* NewMaterialExpressionFunctionOutput(UObject*                             Parent,
	                                                                       const FString&                       Name,
	                                                                       const FMaterialExpressionConnection& Output);

	// If
	UMaterialExpressionIf* NewMaterialExpressionIf(UObject*                             Parent,
	                                               const FMaterialExpressionConnection& A,
	                                               const FMaterialExpressionConnection& B,
	                                               const FMaterialExpressionConnection& Less,
	                                               const FMaterialExpressionConnection& Equal,
	                                               const FMaterialExpressionConnection& Greater);
	UMaterialExpressionIf* NewMaterialExpressionIfEqual(UObject*                             Parent,
	                                                    const FMaterialExpressionConnection& A,
	                                                    const FMaterialExpressionConnection& Bx,
	                                                    const FMaterialExpressionConnection& Yes,
	                                                    const FMaterialExpressionConnection& No);
	UMaterialExpressionIf* NewMaterialExpressionIfGreater(UObject*                             Parent,
	                                                      const FMaterialExpressionConnection& A,
	                                                      const FMaterialExpressionConnection& B,
	                                                      const FMaterialExpressionConnection& Yes,
	                                                      const FMaterialExpressionConnection& No);
	UMaterialExpressionIf* NewMaterialExpressionIfLess(UObject*                             Parent,
	                                                   const FMaterialExpressionConnection& A,
	                                                   const FMaterialExpressionConnection& B,
	                                                   const FMaterialExpressionConnection& Yes,
	                                                   const FMaterialExpressionConnection& No);
	UMaterialExpressionIf* NewMaterialExpressionSwitch(UObject*                                     Parent,
	                                                   const FMaterialExpressionConnection&         Switch,
	                                                   const TArray<FMaterialExpressionConnection>& Cases);

	// LinearInterpolate
	UMaterialExpressionLinearInterpolate* NewMaterialExpressionLinearInterpolate(UObject*                             Parent,
	                                                                             const FMaterialExpressionConnection& A,
	                                                                             const FMaterialExpressionConnection& B,
	                                                                             const FMaterialExpressionConnection& Alpha);
	UMaterialExpression*                  NewMaterialExpressionLinearInterpolate(UObject*                                     Function,
	                                                                             const TArray<FMaterialExpressionConnection>& Values,
	                                                                             const FMaterialExpressionConnection&         Alpha);

	// Logarithm2
	UMaterialExpressionLogarithm2* NewMaterialExpressionLogarithm2(UObject* Parent, const FMaterialExpressionConnection& X);

	// MaterialFunctionCall
	UMaterialExpressionMaterialFunctionCall* NewMaterialExpressionFunctionCall(UObject*                                     Parent,
	                                                                           UMaterialFunction*                           Function,
	                                                                           const TArray<FMaterialExpressionConnection>& Inputs);

	// MakeMaterialAttributes
	// @note Forcing ClearCoat to 0.0f per default
	UMaterialExpressionMakeMaterialAttributes* NewMaterialExpressionMakeMaterialAttributes(
	    UObject* Parent, const FMaterialExpressionConnection& BaseColor = {}, const FMaterialExpressionConnection& Metallic = {},
	    const FMaterialExpressionConnection& Specular = {}, const FMaterialExpressionConnection& Roughness = {},
	    const FMaterialExpressionConnection& EmissiveColor = {}, const FMaterialExpressionConnection& Opacity = {},
	    const FMaterialExpressionConnection& OpacityMask = {}, const FMaterialExpressionConnection& Normal = {},
	    const FMaterialExpressionConnection& WorldPositionOffset = {}, const FMaterialExpressionConnection& SubsurfaceColor = {},
	    const FMaterialExpressionConnection& ClearCoat = 0.0f, const FMaterialExpressionConnection& ClearCoatRoughness = {},
	    const FMaterialExpressionConnection& AmbientOcclusion = {}, const FMaterialExpressionConnection& Refraction = {},
	    const FMaterialExpressionConnection& CustomizedUVs0 = {}, const FMaterialExpressionConnection& CustomizedUVs1 = {},
	    const FMaterialExpressionConnection& CustomizedUVs2 = {}, const FMaterialExpressionConnection& CustomizedUVs3 = {},
	    const FMaterialExpressionConnection& CustomizedUVs4 = {}, const FMaterialExpressionConnection& CustomizedUVs5 = {},
	    const FMaterialExpressionConnection& CustomizedUVs6 = {}, const FMaterialExpressionConnection& CustomizedUVs7 = {},
	    const FMaterialExpressionConnection& PixelDepthOffset = {});

	// Max
	UMaterialExpressionMax* NewMaterialExpressionMax(UObject* Parent, const FMaterialExpressionConnection& A, const FMaterialExpressionConnection& B);

	UMaterialExpressionMax* NewMaterialExpressionMax(UObject*                             Parent,
	                                                 const FMaterialExpressionConnection& A,
	                                                 const FMaterialExpressionConnection& B,
	                                                 const FMaterialExpressionConnection& C);

	// Min
	UMaterialExpressionMin* NewMaterialExpressionMin(UObject* Parent, const FMaterialExpressionConnection& A, const FMaterialExpressionConnection& B);
	UMaterialExpressionMin* NewMaterialExpressionMin(UObject*                             Parent,
	                                                 const FMaterialExpressionConnection& A,
	                                                 const FMaterialExpressionConnection& B,
	                                                 const FMaterialExpressionConnection& C);

	// Multiply
	UMaterialExpressionMultiply* NewMaterialExpressionMultiply(UObject*                             Parent,
	                                                           const FMaterialExpressionConnection& A,
	                                                           const FMaterialExpressionConnection& B);
	UMaterialExpressionMultiply* NewMaterialExpressionMultiply(UObject* Parent, const TArray<FMaterialExpressionConnection>& Arguments);
	UMaterialExpressionMultiply* NewMaterialExpressionSquare(UObject* Parent, const FMaterialExpressionConnection& A);

	// Negate
	UMaterialExpressionSubtract* NewMaterialExpressionNegate(UObject* Parent, const FMaterialExpressionConnection& Input);

	// Noise
	UMaterialExpressionNoise* NewMaterialExpressionNoise(UObject* Parent, const FMaterialExpressionConnection& Position, int32 Quality);

	// Vector Noise
	UMaterialExpressionVectorNoise* NewMaterialExpressionVectorNoise(UObject* Parent, const FMaterialExpressionConnection& Position, int32 Quality);

	// Normalize
	UMaterialExpressionNormalize* NewMaterialExpressionNormalize(UObject* Parent, const FMaterialExpressionConnection& Input);

	// OneMinus
	UMaterialExpressionOneMinus* NewMaterialExpressionOneMinus(UObject* Parent, const FMaterialExpressionConnection& Input);

	// Power
	UMaterialExpressionPower* NewMaterialExpressionPower(UObject*                             Parent,
	                                                     const FMaterialExpressionConnection& Base,
	                                                     const FMaterialExpressionConnection& Exponent);

	// ReflectionVectorWS
	UMaterialExpressionReflectionVectorWS* NewMaterialExpressionReflectionVectorWS(UObject*                             Parent,
	                                                                               const FMaterialExpressionConnection& CustomWorldNormal);

	// Saturate
	UMaterialExpressionSaturate* NewMaterialExpressionSaturate(UObject* Parent, const FMaterialExpressionConnection& Input);

	// ScalarParameter
	UMaterialExpressionScalarParameter* NewMaterialExpressionScalarParameter(UObject* Parent, const FString& Name, float DefaultValue);

	// Sine
	UMaterialExpressionSine* NewMaterialExpressionSine(UObject* Parent, const FMaterialExpressionConnection& Input);

	// SquareRoot
	UMaterialExpressionSquareRoot* NewMaterialExpressionSquareRoot(UObject* Parent, const FMaterialExpressionConnection& Input);

	// StaticBool
	UMaterialExpressionStaticBool* NewMaterialExpressionStaticBool(UObject* Parent, bool bValue);

	// StaticBoolParameter
	UMaterialExpressionStaticBoolParameter* NewMaterialExpressionStaticBoolParameter(UObject*       Parent,
	                                                                                 const FString& Name,
	                                                                                 bool           bDefaultValue,
	                                                                                 const FString& Group = TEXT(""));

	// StaticSwitch
	UMaterialExpressionStaticSwitch* NewMaterialExpressionStaticSwitch(UObject*                             Parent,
	                                                                   const FMaterialExpressionConnection& Value,
	                                                                   const FMaterialExpressionConnection& True,
	                                                                   const FMaterialExpressionConnection& False);

	// Subtract
	UMaterialExpressionSubtract* NewMaterialExpressionSubtract(UObject*                             Parent,
	                                                           const FMaterialExpressionConnection& A,
	                                                           const FMaterialExpressionConnection& B);
	UMaterialExpressionSubtract* NewMaterialExpressionSubtract(UObject*                             Parent,
	                                                           const FMaterialExpressionConnection& A,
	                                                           const FMaterialExpressionConnection& B,
	                                                           const FMaterialExpressionConnection& C);

	// Tangent
	UMaterialExpressionTangent* NewMaterialExpressionTangent(UObject* Parent, const FMaterialExpressionConnection& Input);

	// TextureCoordinate
	UMaterialExpressionTextureCoordinate* NewMaterialExpressionTextureCoordinate(UObject* Parent, int32 CoordinateIndex);

	// TextureObject
	UMaterialExpressionTextureObject* NewMaterialExpressionTextureObject(UObject* Parent, UTexture* Texture);

	// TextureObjectParameter
	UMaterialExpressionTextureObjectParameter* NewMaterialExpressionTextureObjectParameter(UObject* Parent, const FString& Name, UTexture* Texture);

	// TextureProperty
	UMaterialExpressionTextureProperty* NewMaterialExpressionTextureProperty(UObject*                             Parent,
	                                                                         const FMaterialExpressionConnection& TextureObject,
	                                                                         EMaterialExposedTextureProperty      Property);

	// TextureSample
	UMaterialExpressionTextureSample* NewMaterialExpressionTextureSample(UObject*                             Parent,
	                                                                     const FMaterialExpressionConnection& TextureObject,
	                                                                     const FMaterialExpressionConnection& Coordinates);

	// Transform
	UMaterialExpressionTransform* NewMaterialExpressionTransform(UObject*                             Parent,
	                                                             const FMaterialExpressionConnection& Input,
	                                                             EMaterialVectorCoordTransformSource  SourceType,
	                                                             EMaterialVectorCoordTransform        DestinationType);

	// TransformPosition
	UMaterialExpressionTransformPosition* NewMaterialExpressionTransformPosition(UObject*                             Parent,
	                                                                             const FMaterialExpressionConnection& Input,
	                                                                             EMaterialPositionTransformSource     SourceType,
	                                                                             EMaterialPositionTransformSource     DestinationType);

	// TwoSidedSign
	UMaterialExpressionTwoSidedSign* NewMaterialExpressionTwoSidedSign(UObject* Parent);

	// VectorNoise
	UMaterialExpressionVectorNoise* NewMaterialExpressionVectorNoise(UObject* Parent, const FMaterialExpressionConnection& Position,
	                                                                 EVectorNoiseFunction NoiseFunction, int32 Quality);

	// VectorParameter
	UMaterialExpressionVectorParameter* NewMaterialExpressionVectorParameter(UObject* Parent, const FString& Name, const FLinearColor& DefaultValue);

	//

	template <typename T>
	inline void Connect(T& Target, const FMaterialExpressionConnection& Connection)
	{
		if (UMaterialExpression* Expression = Connection.GetExpressionAndUse())
		{
			Target.Connect(Connection.GetExpressionOutputIndex(), Expression);
		}
	}

	template <typename T>
	inline T* NewMaterialExpression(UObject* Parent)
	{
		check(Parent != nullptr);

		T* Expression                      = NewObject<T>(Parent);
		Expression->MaterialExpressionGuid = FGuid::NewGuid();
		Expression->bCollapsed             = true;

		if (Parent->IsA<UMaterial>())
		{
			Cast<UMaterial>(Parent)->GetExpressionCollection().AddExpression(Expression);
		}
		else if (Parent->IsA<UMaterialFunction>())
		{
			Cast<UMaterialFunction>(Parent)->GetExpressionCollection().AddExpression(Expression);
		}

		return Expression;
	}

}  // namespace Generator
