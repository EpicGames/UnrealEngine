// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialTextureFactory.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "EditorFramework/AssetImportData.h"
#include "Engine/Texture2D.h"
#include "Factories/TextureFactory.h"
#include "IAssetTools.h"
#include "Misc/Paths.h"
#include "ObjectTools.h"
#include "PackageTools.h"

namespace Generator
{
	FMaterialTextureFactory::FMaterialTextureFactory()
	    : Factory(nullptr)
	{
	}

	UTexture2D* FMaterialTextureFactory::CreateTexture(UObject* ParentPackage, const Common::FTextureProperty& Property, FTextureSourcePtr& Source,
	                                                   EObjectFlags Flags, TArray<MDLImporterLogging::FLogMessage>* InLogMessages /*= nullptr*/)
	{
		TFunction<void(const FString&)> LogWarning = [InLogMessages](const FString& Message) -> void
		{
			if (InLogMessages)
			{
				InLogMessages->Emplace(MDLImporterLogging::EMessageSeverity::Warning, TEXT("Texture creation failed: ") + Message);
			}
		};

		FString TextureName = AssetPrefix + TEXT("_") + ObjectTools::SanitizeObjectName(FPaths::GetBaseFilename(Property.Path));
		if (TextureName == TEXT("_"))
		{
			LogWarning(TEXT("Texture does not have a name"));
			return nullptr;
		}

		if (!ensure(ParentPackage))
		{
			LogWarning(TEXT("Invalid package for texture ") + Property.Path);
			return nullptr;
		}

		// save texture settings if texture exists
		Factory->SuppressImportOverwriteDialog();
		UpdateTextureFactorySettings(Factory, Property);

		// The package of the material is passed as the parent package
		// Therefore we need to create a package for the texture asset itself
		// This was unnoticed so far because the ContentBrowser was displaying packages
		// with multiple assets as separate entries in folders. This has been removed for 5.3. 
		const FString PackagePath = FPaths::GetPath(ParentPackage->GetPathName()) / TextureName;
		UPackage* Package = CreatePackage(*UPackageTools::SanitizePackageName(*PackagePath));

		// check for asset collision
		{
			UObject* Asset = LoadObject<UObject>(Package, *TextureName, nullptr, LOAD_NoWarn);
			if (Asset)
			{
				if (ensure((Asset->IsA<UTexture2D>())))
				{
					return Cast<UTexture2D>(Asset);  // already loaded
				}

				// Make sure the asset name is unique
				FString AssetName;
				FString NewPackagePath;
				IAssetTools::Get().CreateUniqueAssetName(PackagePath, FString(), /*out*/ NewPackagePath, /*out*/ AssetName);

				TextureName = AssetName;

				Package->MarkAsGarbage();
				Package = CreatePackage(*NewPackagePath);
			}
		}


		UTexture2D* Texture = nullptr;
		const FString Extension = FPaths::GetExtension(Property.Path);
		if (Extension == TEXT("png") || Extension == TEXT("jpg") || Extension == TEXT("jpeg"))
		{
			bool bOperationCanceled = false;
			Texture = static_cast<UTexture2D*>(Factory->FactoryCreateFile(UTexture2D::StaticClass(), Package, *TextureName, Flags,
			                                                              Property.Path, TEXT("MDL"), nullptr, bOperationCanceled));
			Texture->AssetImportData->Update(Property.Path);
			FAssetRegistryModule::AssetCreated(Texture);
			Texture->MarkPackageDirty();
		}
		else
		{
			if (!ensure(Source))
			{
				LogWarning(TEXT("Not supported texture format '") + FPaths::GetExtension(Property.Path) + TEXT("' for ") + Property.Path);
				return nullptr;
			}

			if (!ensure(Source->GetWidth() > 4 && Source->GetHeight() > 4))
			{
				LogWarning(TEXT("Not supported texture size for ") + Property.Path);
				return nullptr;
			}

			Texture = Factory->CreateTexture2D(Package, *TextureName, Flags);
			Texture->Source.Init(*Source);
			Texture->Source.Compress();

			delete Source;
			Source = nullptr;
		}

		if (Texture)
		{
			UpdateTextureSettings(Texture, Property, InLogMessages);
		}

		return Texture;
	}

	void FMaterialTextureFactory::UpdateTextureFactorySettings(UTextureFactory* TextureFactory, const Common::FTextureProperty& Property)
	{
		// Set the settings on the factory so that the texture gets built with the correct settings when the factory builds it
		TextureFactory->bUseHashAsGuid = true; // Use texture source data hash as DDC guid
		TextureFactory->MipGenSettings = Property.MipGenSettings;
		TextureFactory->NoAlpha = Property.bCompressionNoAlpha;
		TextureFactory->CompressionSettings = Property.CompressionSettings;
		TextureFactory->LODGroup = Property.LODGroup;
		TextureFactory->bFlipNormalMapGreenChannel = Property.bFlipGreenChannel;
		TextureFactory->ColorSpaceMode = Property.bIsSRGB ? ETextureSourceColorSpace::SRGB : ETextureSourceColorSpace::Linear;
	}

	void FMaterialTextureFactory::UpdateTextureSettings(UTexture2D* Texture, const Common::FTextureProperty& Property, TArray<MDLImporterLogging::FLogMessage>* LogMessages /*= nullptr*/)
	{
		TextureMipGenSettings MipGenSettings = Property.MipGenSettings;

		if (!Texture->Source.AreAllBlocksPowerOfTwo())
		{
			MipGenSettings = TMGS_NoMipmaps;

			if (LogMessages)
			{
				LogMessages->Emplace(
					MDLImporterLogging::EMessageSeverity::Warning,
					FString::Printf(TEXT("Texture %s does not have power of two dimensions and therefore no mipmaps will be generated"),
						*Texture->GetName()));
			}
		}

		const bool bIsSRGB = Texture->IsNormalMap() ? false : Property.bIsSRGB;

		const bool bSettingsChanged =
			(Texture->MipGenSettings != MipGenSettings || Texture->CompressionNoAlpha != Property.bCompressionNoAlpha ||
				Texture->CompressionSettings != Property.CompressionSettings || Texture->Filter != Property.Filter ||
				Texture->AddressY != Property.Address || Texture->AddressX != Property.Address ||
				Texture->SRGB != bIsSRGB || Texture->bFlipGreenChannel != Property.bFlipGreenChannel);

		if (bSettingsChanged)
		{
			Texture->PreEditChange(nullptr);
			Texture->MipGenSettings = MipGenSettings;
			Texture->CompressionNoAlpha = Property.bCompressionNoAlpha;
			Texture->CompressionSettings = Property.CompressionSettings;
			Texture->Filter = Property.Filter;
			Texture->AddressY = Property.Address;
			Texture->AddressX = Property.Address;
			Texture->LODGroup = Property.LODGroup;
			Texture->SRGB = bIsSRGB;
			Texture->bFlipGreenChannel = Property.bFlipGreenChannel;
			Texture->PostEditChange();
		}
	}

}  // namespace Generator
