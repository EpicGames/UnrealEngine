// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "common/Logging.h"
#include "common/TextureProperty.h"


class UTextureFactory;
class UObject;
class UTexture2D;
struct FImage;

namespace Generator
{
	class FMaterialTextureFactory
	{
	public:
		typedef FImage* FTextureSourcePtr;

		FMaterialTextureFactory();

		void SetFactory(UTextureFactory* Factory);
		void SetDefaultProperties(const Common::FTextureProperty& Properties);
		void SetAssetPrefix(const FString& Suffix);

		UTexture2D* CreateTexture(UObject* ParentPackage, const Common::FTextureProperty& Property, FTextureSourcePtr& Source, EObjectFlags Flags,
		                          TArray<MDLImporterLogging::FLogMessage>* InLogMessages = nullptr);
		UTexture2D* CreateTexture(UObject* ParentPackage, const Common::FTextureProperty& Property, EObjectFlags Flags,
		                          TArray<MDLImporterLogging::FLogMessage>* InLogMessages = nullptr);
		UTexture2D* CreateTexture(UObject* ParentPackage, const FString& FilePath, EObjectFlags Flags,
		                          TArray<MDLImporterLogging::FLogMessage>* InLogMessages = nullptr);

		void UpdateTextureFactorySettings(UTextureFactory* TextureFactory, const Common::FTextureProperty& Property);
		void UpdateTextureSettings(UTexture2D* Texture, const Common::FTextureProperty& Property, TArray<MDLImporterLogging::FLogMessage>* LogMessages = nullptr);

	private:
		UTextureFactory*         Factory;
		Common::FTextureProperty DefaultProperties;
		FString                  AssetPrefix;
	};

	inline void FMaterialTextureFactory::SetFactory(UTextureFactory* InFactory)
	{
		Factory = InFactory;
	}

	inline void FMaterialTextureFactory::SetDefaultProperties(const Common::FTextureProperty& Properties)
	{
		DefaultProperties = Properties;
	}

	inline void FMaterialTextureFactory::SetAssetPrefix(const FString& Prefix)
	{
		AssetPrefix = Prefix;
	}

	inline UTexture2D* FMaterialTextureFactory::CreateTexture(UObject* ParentPackage, const Common::FTextureProperty& Property, EObjectFlags Flags,
	                                                          TArray<MDLImporterLogging::FLogMessage>* InLogMessages /*= nullptr*/)
	{
		FTextureSourcePtr None = nullptr;
		return CreateTexture(ParentPackage, Property, None, Flags, InLogMessages);
	}

	inline UTexture2D* FMaterialTextureFactory::CreateTexture(UObject* ParentPackage, const FString& FilePath, EObjectFlags Flags,
	                                                          TArray<MDLImporterLogging::FLogMessage>* InLogMessages /*= nullptr*/)
	{
		Common::FTextureProperty Property = DefaultProperties;
		Property.Path                     = FilePath;
		FTextureSourcePtr None            = nullptr;
		return CreateTexture(ParentPackage, Property, None, Flags, InLogMessages);
	}
}  // namespace Generator
