// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef USE_MDLSDK

#include "ParameterExpressionFactory.h"

#include "Materials/MaterialExpressionAppendVector.h"
#include "generator/MaterialExpressions.h"
#include "Materials/MaterialExpressionComponentMask.h"
#include "generator/MaterialTextureFactory.h"
#include "Materials/MaterialExpressionMakeMaterialAttributes.h"
#include "mdl/MdlSdkDefines.h"
#include "Materials/MaterialExpressionScalarParameter.h"
#include "mdl/Utility.h"

#include "Engine/Texture2D.h"
#include "Materials/MaterialExpressionStaticBoolParameter.h"
#include "Misc/Paths.h"

MDLSDK_INCLUDES_START
#include "Materials/MaterialExpressionTextureObjectParameter.h"
#include "mi/neuraylib/icompiled_material.h"
#include "Materials/MaterialExpressionVectorParameter.h"
#include "mi/neuraylib/iexpression.h"
#include "generator/MaterialExpressionConnection.h"
#include "mi/neuraylib/imaterial_definition.h"
#include "mi/neuraylib/itexture.h"
#include "mi/neuraylib/itransaction.h"
#include "mi/neuraylib/ivalue.h"
#include <mi/base/handle.h>
#include <mi/base/iinterface.h>
#include <mi/base/types.h>
#include <mi/base/uuid.h>
#include <mi/neuraylib/itype.h>
MDLSDK_INCLUDES_END

namespace Generator
{
	namespace
	{
		UMaterialExpression* ImportColorParameter(const FString& Name, const mi::neuraylib::IValue& Value, UObject* Parent)
		{
			auto Color = mi::base::make_handle(Value.get_interface<const mi::neuraylib::IValue_color>());
			return NewMaterialExpressionVectorParameter(Parent, Name,
			                                            FLinearColor(mi::base::make_handle(Color->get_value(0))->get_value(),
			                                                         mi::base::make_handle(Color->get_value(1))->get_value(),
			                                                         mi::base::make_handle(Color->get_value(2))->get_value()));
		}

		UMaterialExpression* ImportVectorParameter(const FString& Name, const mi::neuraylib::IValue& Value, UObject* Parent)
		{
			auto Vector = mi::base::make_handle(Value.get_interface<const mi::neuraylib::IValue_vector>());

			FLinearColor DefaultValue(EForceInit::ForceInitToZero);
			for (mi::Size Index = 0; Index < Vector->get_size(); Index++)
			{
				mi::base::Handle<const mi::neuraylib::IValue> ElementValue(Vector->get_value(Index));
				check(ElementValue->get_kind() == mi::neuraylib::IValue::VK_FLOAT);
				DefaultValue.Component(Index) = ElementValue.get_interface<const mi::neuraylib::IValue_float>()->get_value();
			}
			UMaterialExpressionVectorParameter* Parameter = NewMaterialExpressionVectorParameter(Parent, Name, DefaultValue);
			switch (Vector->get_size())
			{
				case 2:
					return NewMaterialExpressionComponentMask(Parent, Parameter, 3);
				case 3:
					return Parameter;
				case 4:
					return NewMaterialExpressionAppendVector(Parent, {Parameter, 0}, {Parameter, 4});
				default:
					check(false);
					return nullptr;
			}
		}

		template <typename MdlType, typename ReturnType>
		ReturnType GetValue(const mi::neuraylib::IValue& Value)
		{
			auto Handle = mi::base::make_handle(Value.get_interface<const MdlType>());
			return Handle->get_value();
		}

		template <typename ReturnType, typename MDLType>
		ReturnType GetExpressionConstant(const mi::base::Handle<const mi::neuraylib::IExpression>& Expression)
		{
			check(Expression->get_kind() == mi::neuraylib::IExpression::EK_CONSTANT);

			mi::base::Handle<const mi::neuraylib::IValue> Value(Expression.get_interface<const mi::neuraylib::IExpression_constant>()->get_value());
			check(Value.get_interface<const MDLType>());

			return Value.get_interface<const MDLType>()->get_value();
		}

		template <typename ReturnType, typename MDLType>
		TArray<ReturnType> GetExpressionConstant(const mi::base::Handle<const mi::neuraylib::IExpression_list>& ExpressionList)
		{
			TArray<ReturnType> ReturnValues;
			for (mi::Size Index = 0, Size = ExpressionList->get_size(); Index < Size; Index++)
			{
				ReturnValues.Add(GetExpressionConstant<ReturnType, MDLType>(mi::base::make_handle(ExpressionList->get_expression(Index))));
			}
			return ReturnValues;
		}

		UMaterialExpression* GetParameterExpression(UMaterialExpression* Expression)
		{
			if (Expression->IsA<UMaterialExpressionAppendVector>())
			{
				UMaterialExpressionAppendVector* AppendVector = Cast<UMaterialExpressionAppendVector>(Expression);
				check(AppendVector->A.Expression == AppendVector->B.Expression);
				return AppendVector->A.Expression;
			}
			else if (Expression->IsA<UMaterialExpressionComponentMask>())
			{
				return GetParameterExpression(Cast<UMaterialExpressionComponentMask>(Expression)->Input.Expression);
			}
			else
			{
				check(Expression->IsA<UMaterialExpressionParameter>() || Expression->IsA<UMaterialExpressionTextureSampleParameter>());
				return Expression;
			}
		}

		bool IsParameterIgnored(const FString& AnnotationName)
		{
			return (AnnotationName == TEXT("::alg::base::annotations::gamma_type(::tex::gamma_mode)")) ||
			       (AnnotationName == TEXT("::alg::base::annotations::sampler_usage(string,string)")) ||
			       (AnnotationName == TEXT("::alg::base::annotations::visible_by_default(bool)")) ||
			       (AnnotationName == TEXT("::anno::author(string)")) || (AnnotationName == TEXT("::anno::contributor(string)")) ||
			       (AnnotationName == TEXT("::anno::copyright_notice(string)")) || (AnnotationName == TEXT("::anno::create(int,int,int,string)")) ||
			       (AnnotationName == TEXT("::anno::dependency(string,int,int,int,string)")) || (AnnotationName == TEXT("::anno::deprecated()")) ||
			       (AnnotationName == TEXT("::anno::deprecated(string)")) || (AnnotationName == TEXT("::anno::hard_range(double2,double2)")) ||
			       (AnnotationName == TEXT("::anno::hard_range(double3,double3)")) ||
			       (AnnotationName == TEXT("::anno::hard_range(double4,double4)")) || (AnnotationName == TEXT("::anno::hard_range(float2,float2)")) ||
			       (AnnotationName == TEXT("::anno::hard_range(float3,float3)")) || (AnnotationName == TEXT("::anno::hard_range(float4,float4)")) ||
			       (AnnotationName == TEXT("::anno::hard_range(int2,int2)")) || (AnnotationName == TEXT("::anno::hard_range(int3,int3)")) ||
			       (AnnotationName == TEXT("::anno::hard_range(int4,int4)")) || (AnnotationName == TEXT("::anno::hidden()")) ||
			       (AnnotationName == TEXT("::anno::key_words(string[])")) || (AnnotationName == TEXT("::anno::modified(int,int,int,string)")) ||
			       (AnnotationName == TEXT("::anno::in_group(string,string)")) || (AnnotationName == TEXT("::anno:in_group(string,string,string)")) ||
			       (AnnotationName == TEXT("::anno::soft_range(double2,double2)")) ||
			       (AnnotationName == TEXT("::anno::soft_range(double3,double3)")) ||
			       (AnnotationName == TEXT("::anno::soft_range(double4,double4)")) || (AnnotationName == TEXT("::anno::soft_range(float2,float2)")) ||
			       (AnnotationName == TEXT("::anno::soft_range(float3,float3)")) || (AnnotationName == TEXT("::anno::soft_range(float4,float4)")) ||
			       (AnnotationName == TEXT("::anno::soft_range(int2,int2)")) || (AnnotationName == TEXT("::anno::soft_range(int3,int3)")) ||
			       (AnnotationName == TEXT("::anno::soft_range(int4,int4)")) || (AnnotationName == TEXT("::anno::unused()")) ||
			       (AnnotationName == TEXT("::anno::unused(string)")) || (AnnotationName == TEXT("::anno::version(int,int,int,string)")) ||
			       (AnnotationName == TEXT("::core_definitions::ui_position(int)")) || (AnnotationName == TEXT("::ifm::enableIf(string,string)")) ||
			       (AnnotationName == TEXT("::ifm::mayaName(string)")) || (AnnotationName == TEXT("::ifm::state(string)")) ||
			       (AnnotationName == TEXT("::ifm::uiProperty(string,string,string)")) ||
			       (AnnotationName == TEXT("::ifm::visibleIf(string,string)")) || (AnnotationName == TEXT("::mdl::Lw::ColourShader()")) ||
			       (AnnotationName == TEXT("::mdl::Lw::DisabledIfFalse(string[N])")) ||
			       (AnnotationName == TEXT("::mdl::Lw::DisabledIfTrue(string[N])")) || (AnnotationName == TEXT("::mdl::Lw::FloatShader()")) ||
			       (AnnotationName == TEXT("::mdl::Lw::MaxValue(float)")) || (AnnotationName == TEXT("::mdl::Lw::MinValue(float)")) ||
			       (AnnotationName == TEXT("::mdl::Lw::UserLevel(int)"));
		}

		void ImportAnnotationBlock(const mi::neuraylib::IAnnotation_block&       AnnotationBlock,      //
		                           mi::neuraylib::IValue::Kind                   Kind,                 //
		                           bool                                          bUseDisplayName,      //
		                           UMaterialExpression*                          ParameterExpression,  //
		                           TMap<FString, TArray<UMaterialExpression*> >& NamedParameterExpressions,
		                           TArray<MDLImporterLogging::FLogMessage>& LogMessages)
		{
			bool    bHasGroupAnnotation = false;
			FString DisplayName;
			for (mi::Size AnnotationBlockIndex = 0; AnnotationBlockIndex < AnnotationBlock.get_size(); AnnotationBlockIndex++)
			{
				mi::base::Handle<const mi::neuraylib::IAnnotation> Annotation(AnnotationBlock.get_annotation(AnnotationBlockIndex));

				const FString AnnotationName = ANSI_TO_TCHAR(Annotation->get_name());
				if (AnnotationName == TEXT("::anno::description(string)"))
				{
					TArray<FString> Descriptions =
					    GetExpressionConstant<FString, mi::neuraylib::IValue_string>(mi::base::make_handle(Annotation->get_arguments()));
					check(Descriptions.Num() == 1);
					ParameterExpression->Desc = Descriptions[0];
				}
				else if (AnnotationName == TEXT("::anno::display_name(string)"))
				{
					if (bUseDisplayName)
					{
						TArray<FString> DisplayNames =
						    GetExpressionConstant<FString, mi::neuraylib::IValue_string>(mi::base::make_handle(Annotation->get_arguments()));
						check(DisplayNames.Num() == 1);
						DisplayName = DisplayNames[0];
						ParameterExpression->SetParameterName(*DisplayName);
						NamedParameterExpressions.FindOrAdd(*DisplayName).Add(ParameterExpression);
					}
				}
				else if ((AnnotationName == TEXT("::anno::hard_range(double,double)")) ||
				         (AnnotationName == TEXT("::anno::soft_range(double,double)")))
				{
					check(Kind == mi::neuraylib::IValue::VK_DOUBLE);
					check(ParameterExpression->IsA<UMaterialExpressionScalarParameter>());
					TArray<double> Range =
					    GetExpressionConstant<double, mi::neuraylib::IValue_double>(mi::base::make_handle(Annotation->get_arguments()));
					check(Range.Num() == 2);
					UMaterialExpressionScalarParameter* ScalarParameter = Cast<UMaterialExpressionScalarParameter>(ParameterExpression);
					ScalarParameter->SliderMin                          = Range[0];
					ScalarParameter->SliderMax                          = Range[1];
				}
				else if ((AnnotationName == TEXT("::anno::hard_range(float,float)")) || (AnnotationName == TEXT("::anno::soft_range(float,float)")))
				{
					check(Kind == mi::neuraylib::IValue::VK_FLOAT);
					check(ParameterExpression->IsA<UMaterialExpressionScalarParameter>());
					TArray<float> Range =
					    GetExpressionConstant<float, mi::neuraylib::IValue_float>(mi::base::make_handle(Annotation->get_arguments()));
					check(Range.Num() == 2);
					UMaterialExpressionScalarParameter* ScalarParameter = Cast<UMaterialExpressionScalarParameter>(ParameterExpression);
					ScalarParameter->SliderMin                          = Range[0];
					ScalarParameter->SliderMax                          = Range[1];
				}
				else if ((AnnotationName == TEXT("::anno::hard_range(int,int)")) || (AnnotationName == TEXT("::anno::soft_range(int,int)")))
				{
					check(ParameterExpression->IsA<UMaterialExpressionScalarParameter>());
					TArray<float> Range;
					if (Kind == mi::neuraylib::IValue::VK_FLOAT)
					{
						Range = GetExpressionConstant<float, mi::neuraylib::IValue_float>(mi::base::make_handle(Annotation->get_arguments()));
					}
					else
					{
						check(Kind == mi::neuraylib::IValue::VK_INT);
						TArray<int> IntRange =
						    GetExpressionConstant<int, mi::neuraylib::IValue_int>(mi::base::make_handle(Annotation->get_arguments()));
						for (int32 I = 0; I < IntRange.Num(); I++)
						{
							Range.Add(IntRange[I]);
						}
					}
					check(Range.Num() == 2);
					UMaterialExpressionScalarParameter* ScalarParameter = Cast<UMaterialExpressionScalarParameter>(ParameterExpression);
					ScalarParameter->SliderMin                          = Range[0];
					ScalarParameter->SliderMax                          = Range[1];
				}
				else if (AnnotationName == TEXT("::anno::in_group(string)"))
				{
					bHasGroupAnnotation = true;

					TArray<FString> InGroups =
					    GetExpressionConstant<FString, mi::neuraylib::IValue_string>(mi::base::make_handle(Annotation->get_arguments()));
					check(InGroups.Num() == 1);
					SetMaterialExpressionGroup(InGroups[0], ParameterExpression);
				}
				else if (AnnotationName.StartsWith(TEXT("::nvidia::core_definitions::dimension(")))
				{
					// Skip annotation which we can't apply, to avoid confusing warning for the user
				}
				else
				{
					LogMessages.Emplace(MDLImporterLogging::EMessageSeverity::Warning, TEXT("Parameter Ignored: ") + AnnotationName);
				}
			}
			DisplayName = DisplayName.ToLower();
			if (DisplayName.Find(TEXT("texture tiling")) != INDEX_NONE ||      //
			    DisplayName.Find(TEXT("texture repetition")) != INDEX_NONE ||  //
			    DisplayName.Find(TEXT("texture scale")) != INDEX_NONE ||       //
			    DisplayName.Find(TEXT("scaling factor")) != INDEX_NONE)
			{
				SetMaterialExpressionGroup(TEXT("Other"), ParameterExpression);
				ParameterExpression->SetParameterName(TEXT("Tiling Factor"));
			}
			else if (!bHasGroupAnnotation)
			{
				SetMaterialExpressionGroup(TEXT("Main"), ParameterExpression);
			}
		}
	}

	FParameterExpressionFactory::FParameterExpressionFactory()
	    : TextureFactory(nullptr)
	{
	}

	void FParameterExpressionFactory::CreateExpressions(mi::neuraylib::ITransaction& Transaction)
	{
		check(CurrentMDLMaterial);
		check(CurrentMDLMaterialDefinition);
		check(TextureFactory);

		mi::base::Handle<const mi::neuraylib::IAnnotation_list> AnnotationList(CurrentMDLMaterialDefinition->get_parameter_annotations());

		mi::Size ParameterCount = CurrentMDLMaterial->get_parameter_count();
		Parameters.SetNum(CurrentMDLMaterial->get_parameter_count());

		TMap<FString, TArray<UMaterialExpression*> > NamedParameterExpressions;
		for (mi::Size ParameterIndex = 0; ParameterIndex < Parameters.Num(); ParameterIndex++)
		{
			FMaterialExpressionConnectionList& CurrentParameter = Parameters[ParameterIndex];
			CurrentParameter.Reset();

			const auto                        Handle = mi::base::make_handle(CurrentMDLMaterial->get_argument(ParameterIndex));
			const FString                     Name   = ANSI_TO_TCHAR(CurrentMDLMaterial->get_parameter_name(ParameterIndex));
			const mi::neuraylib::IValue::Kind Kind   = Handle->get_kind();
			ImportParameter(Name, *Handle, Transaction, CurrentParameter);

			if (CurrentParameter.Num() != 1)
				continue;

			// only for single-valued parameters, we can meaningfully set any annotations
			check(CurrentParameter[0].GetConnectionType() == EConnectionType::Expression);
			UMaterialExpression* ParameterExpression = GetParameterExpression(CurrentParameter[0].GetExpressionAndMaybeUse());

			mi::base::Handle<const mi::neuraylib::IAnnotation_block> AnnotationBlock(
			    AnnotationList->get_annotation_block(CurrentMDLMaterial->get_parameter_name(ParameterIndex)));
			if (AnnotationBlock)
			{
				ImportAnnotationBlock(*AnnotationBlock, Kind, true, ParameterExpression, NamedParameterExpressions, LogMessages);
			}
			else
			{
				SetMaterialExpressionGroup(TEXT("Main"), ParameterExpression);
			}
		}

		for (auto& Elem : NamedParameterExpressions)
		{
			if (Elem.Value.Num() > 1)
			{
				for (int32 i = 0; i < Elem.Value.Num(); i++)
				{
					Elem.Value[i]->SetParameterName(*(Elem.Value[i]->GetParameterName().ToString() + TEXT(" ") + FString::FromInt(i + 1)));
				}
			}
		}
	}

	const FMaterialExpressionConnectionList& FParameterExpressionFactory::GetExpression(
	    const mi::neuraylib::IExpression_parameter& MDLExpression) 
	{
		const mi::Size Index = MDLExpression.get_index();
		check(Index < Parameters.Num());
		Parameters[Index].SetIsUsed();
		return Parameters[Index];
	}

	void FParameterExpressionFactory::ImportParameter(const FString& Name, const mi::neuraylib::IValue& Value,
	                                                  mi::neuraylib::ITransaction& Transaction, FMaterialExpressionConnectionList& Parameter)
	{
		mi::base::Handle<const mi::neuraylib::IType> Type(Value.get_type());
		const mi::neuraylib::IValue::Kind            Kind = Value.get_kind();
		switch (Kind)
		{
			case mi::neuraylib::IValue::VK_BOOL:
				Parameter.Add(NewMaterialExpressionStaticBoolParameter(CurrentMaterial, Name, GetValue<mi::neuraylib::IValue_bool, bool>(Value)));
				break;
			case mi::neuraylib::IValue::VK_INT:
				Parameter.Add(NewMaterialExpressionScalarParameter(CurrentMaterial, Name, GetValue<mi::neuraylib::IValue_int, int>(Value)));
				break;
			case mi::neuraylib::IValue::VK_ENUM:
				Parameter.Add(NewMaterialExpressionScalarParameter(CurrentMaterial, Name, GetValue<mi::neuraylib::IValue_enum, int>(Value)));
				break;
			case mi::neuraylib::IValue::VK_STRING:  // skip string parameters
				break;
			case mi::neuraylib::IValue::VK_FLOAT:
				Parameter.Add(NewMaterialExpressionScalarParameter(CurrentMaterial, Name, GetValue<mi::neuraylib::IValue_float, float>(Value)));
				break;
			case mi::neuraylib::IValue::VK_DOUBLE:
				Parameter.Add(NewMaterialExpressionScalarParameter(CurrentMaterial, Name, GetValue<mi::neuraylib::IValue_double, double>(Value)));
				break;
			case mi::neuraylib::IValue::VK_VECTOR:
				Parameter.Add(ImportVectorParameter(Name, Value, CurrentMaterial));
				break;
			case mi::neuraylib::IValue::VK_MATRIX:
			{
				auto MatrixValue = mi::base::make_handle(Value.get_interface<const mi::neuraylib::IValue_matrix>());
				for (mi::Size Index = 0; Index < MatrixValue->get_size(); Index++)
				{
					auto Handle = mi::base::make_handle(MatrixValue->get_value(Index));
					ImportParameter(Name + TEXT("_") + FString::FromInt(Index), *Handle, Transaction, Parameter);
				}
			}
			break;
			case mi::neuraylib::IValue::VK_COLOR:
				Parameter.Add(ImportColorParameter(Name, Value, CurrentMaterial));
				break;
			case mi::neuraylib::IValue::VK_ARRAY:
			{
				auto ArrayValue = mi::base::make_handle(Value.get_interface<const mi::neuraylib::IValue_array>());
				for (mi::Size Index = 0; Index < ArrayValue->get_size(); Index++)
				{
					auto Handle = mi::base::make_handle(ArrayValue->get_value(Index));
					ImportParameter(Name + TEXT("_") + FString::FromInt(Index), *Handle, Transaction, Parameter);
				}
			}
			break;
			case mi::neuraylib::IValue::VK_STRUCT:
			{
				const mi::base::Handle<const mi::neuraylib::IValue_struct> StructValue(Value.get_interface<const mi::neuraylib::IValue_struct>());
				for (mi::Size Index = 0; Index < StructValue->get_size(); Index++)
				{
					auto Handle = mi::base::make_handle(StructValue->get_value(Index));
					ImportParameter(Name + TEXT("_") + mi::base::make_handle(StructValue->get_type())->get_field_name(Index), *Handle, Transaction,
					                Parameter);
				}
			}
			break;
			case mi::neuraylib::IValue::VK_INVALID_DF:
				Parameter.Add(NewMaterialExpressionMakeMaterialAttributes(CurrentMaterial));
				break;
			case mi::neuraylib::IValue::VK_TEXTURE:
			{
				auto Handle = mi::base::make_handle(Value.get_interface<const mi::neuraylib::IValue_texture>());
				const mi::base::Handle<const mi::neuraylib::ITexture> MDLTexture(Transaction.access<mi::neuraylib::ITexture>(Handle->get_value()));

				UTexture2D* Texture = nullptr;

				if (MDLTexture)
				{
					Common::FTextureProperty Property;
					Property.Path    = Mdl::Util::GetTextureFileName(MDLTexture.get());
					float Gamma      = MDLTexture->get_effective_gamma(0, 0);
					Property.bIsSRGB = Gamma != 1.0;
					if (!FPaths::GetExtension(Property.Path).IsEmpty())
					{
						Texture = TextureFactory->CreateTexture(CurrentMaterial->GetOuter(), Property, CurrentMaterial->GetFlags(), &LogMessages);
					}
				}

				Parameter.Add(NewMaterialExpressionTextureObjectParameter(CurrentMaterial, Name, Texture));
			}
			break;
			case mi::neuraylib::IValue::VK_LIGHT_PROFILE:
			{
				auto Handle = mi::base::make_handle(Value.get_interface<const mi::neuraylib::IValue_light_profile>());

				LogMessages.Emplace(MDLImporterLogging::EMessageSeverity::Warning, TEXT("Light profiles aren't supported."));

				// TODO: check for this, never encountered some real light profile
				check(Handle->get_value() == nullptr);
			}
			break;
			case mi::neuraylib::IValue::VK_BSDF_MEASUREMENT:
			{
				auto Handle = mi::base::make_handle(Value.get_interface<const mi::neuraylib::IValue_bsdf_measurement>());

				LogMessages.Emplace(MDLImporterLogging::EMessageSeverity::Error, TEXT("Measured BSDF or BTF textures aren't supported."));
			}
			break;
			default:
				check(false);
		}
	}

	void FParameterExpressionFactory::Cleanup()
	{
		FBaseExpressionFactory::Cleanup();
	}

	void FParameterExpressionFactory::CleanupMaterialExpressions()
	{
		for (FMaterialExpressionConnectionList& Parameter : Parameters)
		{
			if (!Parameter.IsUsed())
			{
				for (FMaterialExpressionConnection& Connection : Parameter.Connections)
				{
					if (Connection.GetConnectionType() == EConnectionType::Expression)
					{
						CurrentMaterial->GetExpressionCollection().RemoveExpression(Connection.GetExpressionUnused());
						Connection.DestroyExpression();
					}
				}
			}
		}
	}

}  // namespace Generator
#endif
