// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "generator/BaseExpressionFactory.h"

namespace Generator { struct FMaterialExpressionConnectionList; }

namespace mi
{
	namespace neuraylib
	{
		class IExpression_parameter;
		class ITransaction;
		class IValue;
	}
}

namespace Generator
{
	class FMaterialTextureFactory;

	class FParameterExpressionFactory : public FBaseExpressionFactory
	{
	public:
		FParameterExpressionFactory();

		void SetTextureFactory(FMaterialTextureFactory* Factory);

		void CreateExpressions(mi::neuraylib::ITransaction& Transaction);

		const FMaterialExpressionConnectionList& GetExpression(const mi::neuraylib::IExpression_parameter& MDLExpression);

		void Cleanup();

		void CleanupMaterialExpressions();
	private:
		void ImportParameter(const FString& Name, const mi::neuraylib::IValue& Value, mi::neuraylib::ITransaction& Transaction,
		                     FMaterialExpressionConnectionList& Parameter);

	private:
		TArray<FMaterialExpressionConnectionList> Parameters;
		FMaterialTextureFactory*                  TextureFactory;
	};

	inline void FParameterExpressionFactory::SetTextureFactory(FMaterialTextureFactory* Factory)
	{
		TextureFactory = Factory;
	}

}  // namespace Generator
