// Copyright Epic Games, Inc. All Rights Reserved.

#include "BakedMaterialFactory.h"

#include "Materials/MaterialExpressionMultiply.h"
#include "material/MapConnecter.h"

#include "Materials/MaterialExpressionClearCoatNormalCustomOutput.h"
#include "Materials/MaterialExpressionScalarParameter.h"
#include "Materials/MaterialExpressionVectorParameter.h"


namespace Mat
{
	namespace
	{
		UMaterialExpression* ExpressionMakeFLoat2(UMaterialExpression* Expression, Generator::FFunctionLoader& FunctionLoader, UMaterial& Material,
		                                          int Index1 = 0, int Index2 = 0

		)
		{
			Generator::FMaterialExpressionConnectionList Inputs;
			Inputs.Emplace(Expression, Index1);
			Inputs.Emplace(Expression, Index2);
			UMaterialFunction* MakeFloat = &FunctionLoader.Get(Generator::ECommonFunction::MakeFloat2);
			return Generator::NewMaterialExpressionFunctionCall(&Material, MakeFloat, Inputs);
		}

		UMaterialExpression* ExpressionMakeFLoat2(UMaterialExpression* ExpressionX, UMaterialExpression* ExpressionY, Generator::FFunctionLoader& FunctionLoader, UMaterial& Material

		)
		{
			Generator::FMaterialExpressionConnectionList Inputs;
			Inputs.Emplace(ExpressionX, 0);
			Inputs.Emplace(ExpressionY, 0);
			UMaterialFunction* MakeFloat = &FunctionLoader.Get(Generator::ECommonFunction::MakeFloat2);
			return Generator::NewMaterialExpressionFunctionCall(&Material, MakeFloat, Inputs);
		}

		UMaterialExpression* ExpressionMakeFLoat3(UMaterialExpression* Expression, Generator::FFunctionLoader& FunctionLoader, UMaterial& Material)
		{
			Generator::FMaterialExpressionConnectionList Inputs;
			Inputs.Emplace(Expression, 0);
			Inputs.Emplace(Expression, 0);
			Inputs.Emplace(Expression, 0);
			UMaterialFunction* MakeFloat = &FunctionLoader.Get(Generator::ECommonFunction::MakeFloat3);
			return Generator::NewMaterialExpressionFunctionCall(&Material, MakeFloat, Inputs);
		}
	}

	FBakedMaterialFactory::FBakedMaterialFactory(Generator::FFunctionLoader& FunctionLoader)
	    : FunctionLoader(FunctionLoader)
	{
	}

	void FBakedMaterialFactory::Create(const Mdl::FMaterial& MdlMaterial, const Mat::FParameterMap& Parameters, UMaterial& Material) const
	{
		UMaterialEditorOnlyData& MaterialEditorOnly = *Material.GetEditorOnlyData();

		// get under clear coat output
		UMaterialExpressionClearCoatNormalCustomOutput* UnderClearCoat = nullptr;
		{
			const TObjectPtr<UMaterialExpression>* Found = Material.GetExpressions().FindByPredicate(
			    [](const UMaterialExpression* Expr) { return Expr->IsA<UMaterialExpressionClearCoatNormalCustomOutput>(); });
			if (Found)
				UnderClearCoat = Cast<UMaterialExpressionClearCoatNormalCustomOutput>(*Found);
		};
		// get the tiling parameter, if not present create one
		UMaterialExpression* Tiling = nullptr;
		{
			EMaterialParameter Maps[] = {
			    EMaterialParameter::BaseColorMap,       EMaterialParameter::ClearCoatNormalMap, EMaterialParameter::ClearCoatRoughnessMap,
			    EMaterialParameter::ClearCoatWeightMap, EMaterialParameter::DisplacementMap,    EMaterialParameter::EmissionColorMap,
			    EMaterialParameter::MetallicMap,        EMaterialParameter::NormalMap,          EMaterialParameter::RoughnessMap,
			    EMaterialParameter::SpecularMap,        EMaterialParameter::SubSurfaceColorMap};
			for (EMaterialParameter MapType : Maps)
			{
				if (Parameters.Contains(MapType))
				{
					Tiling = GetTilingParameter(FunctionLoader, Material);
					break;
				}
			}
			if (Tiling)
				Generator::SetMaterialExpressionGroup(TEXT("Other"), Tiling);
		}

		Mat::FMapConnecter MapConnecter(Parameters, FunctionLoader, Tiling, Material);

		// color
		MapConnecter.ConnectParameterMap(MaterialEditorOnly.BaseColor, TEXT("Color"), EMaterialParameter::BaseColor);

		// brdf
		MapConnecter.ConnectParameterMap(MaterialEditorOnly.Metallic, TEXT("BRDF"), EMaterialParameter::Metallic, false);
		MapConnecter.ConnectParameterMap(MaterialEditorOnly.Specular, TEXT("BRDF"), EMaterialParameter::Specular, false);
		MapConnecter.ConnectParameterMap(MaterialEditorOnly.Roughness, TEXT("BRDF"), EMaterialParameter::Roughness, false);

		// clear coat
		if (UnderClearCoat)
		{
			check(Material.GetShadingModels().HasShadingModel(EMaterialShadingModel::MSM_ClearCoat));
			MapConnecter.ConnectParameterMap(MaterialEditorOnly.ClearCoat, TEXT("Clear Coat"), EMaterialParameter::ClearCoatWeight, false);
			MapConnecter.ConnectParameterMap(MaterialEditorOnly.ClearCoatRoughness, TEXT("Clear Coat"), EMaterialParameter::ClearCoatRoughness, false);

			MapConnecter.ConnectNormalMap(MaterialEditorOnly.Normal, TEXT("Clear Coat"), EMaterialParameter::ClearCoatNormalMap);
			MapConnecter.ConnectNormalMap(UnderClearCoat->Input, TEXT("Normal"), EMaterialParameter::NormalMap);
		}
		else
		{
			MapConnecter.ConnectNormalMap(MaterialEditorOnly.Normal, TEXT("Normal"), EMaterialParameter::NormalMap);

			MapConnecter.DeleteExpressionMap(EMaterialParameter::ClearCoatWeight);
			MapConnecter.DeleteExpressionMap(EMaterialParameter::ClearCoatRoughness);
			MapConnecter.DeleteExpression(EMaterialParameter::ClearCoatNormalMap);
			MapConnecter.DeleteExpression(EMaterialParameter::ClearCoatNormalStrength);
		}

		// subsurface
		if (Material.GetShadingModels().HasShadingModel(EMaterialShadingModel::MSM_Subsurface))
			MapConnecter.ConnectParameterMap(MaterialEditorOnly.SubsurfaceColor, TEXT("Color"), EMaterialParameter::SubSurfaceColor);
		else
			MapConnecter.DeleteExpressionMap(EMaterialParameter::SubSurfaceColor);

		// emission
		if (Parameters.Contains(EMaterialParameter::EmissionColorMap))
		{
			UMaterialExpression* StrengthParameter = ExpressionMakeFLoat3(Parameters[EMaterialParameter::EmissionStrength], FunctionLoader, Material);
			MapConnecter.ConnectParameterMap(MaterialEditorOnly.EmissiveColor, TEXT("Emission"), EMaterialParameter::EmissionColor, true, StrengthParameter);
		}
		else if (Parameters.Contains(EMaterialParameter::EmissionColor))
		{
			UMaterialExpression* ColorParameter    = Parameters[EMaterialParameter::EmissionColor];
			UMaterialExpression* StrengthParameter = Parameters[EMaterialParameter::EmissionStrength];

			Generator::Connect(MaterialEditorOnly.EmissiveColor, Generator::NewMaterialExpressionMultiply(&Material, {ColorParameter, StrengthParameter}));
			Generator::SetMaterialExpressionGroup(TEXT("Emission"), ColorParameter);
			Generator::SetMaterialExpressionGroup(TEXT("Emission"), StrengthParameter);
		}
		else
		{
			MapConnecter.DeleteExpression(EMaterialParameter::EmissionStrength);
		}

		// never used
		MapConnecter.DeleteExpressionMap(EMaterialParameter::Opacity);
		MapConnecter.DeleteExpression(EMaterialParameter::IOR);
		MapConnecter.DeleteExpression(EMaterialParameter::AbsorptionColor);
	}

	UMaterialExpression* FBakedMaterialFactory::GetTilingParameter(Generator::FFunctionLoader& FunctionLoader, UMaterial& Material)
	{
		FString               Name;

		const TObjectPtr<UMaterialExpression>* Found = Material.GetExpressions().FindByPredicate(
			[&Name](UMaterialExpression* Expression)  //
		{
			if (UMaterialExpressionScalarParameter* ScalarParameter = Cast<UMaterialExpressionScalarParameter>(Expression))
			{
				Name = ScalarParameter->ParameterName.ToString().ToLower();
			}
			else if (UMaterialExpressionVectorParameter* VectorParameter = Cast<UMaterialExpressionVectorParameter>(Expression))
			{
				Name = VectorParameter->ParameterName.ToString().ToLower();
			}
			else
				Name.Empty();

			return Name.Find(TEXT("Tiling Factor")) != INDEX_NONE;
		});

		const TObjectPtr<UMaterialExpression>* FoundU = Material.GetExpressions().FindByPredicate(
		    [&Name](UMaterialExpression* Expression)  //
		    {
			    if (UMaterialExpressionScalarParameter* ScalarParameter = Cast<UMaterialExpressionScalarParameter>(Expression))
			    {
				    Name = ScalarParameter->ParameterName.ToString().ToLower();
			    }
			    else if (UMaterialExpressionVectorParameter* VectorParameter = Cast<UMaterialExpressionVectorParameter>(Expression))
			    {
				    Name = VectorParameter->ParameterName.ToString().ToLower();
			    }
			    else
				    Name.Empty();

			    return Name.Find(TEXT("U Tiling")) != INDEX_NONE;
		    });

		const TObjectPtr<UMaterialExpression>* FoundV = Material.GetExpressions().FindByPredicate(
			[&Name](UMaterialExpression* Expression)  //
		{
			if (UMaterialExpressionScalarParameter* ScalarParameter = Cast<UMaterialExpressionScalarParameter>(Expression))
			{
				Name = ScalarParameter->ParameterName.ToString().ToLower();
			}
			else if (UMaterialExpressionVectorParameter* VectorParameter = Cast<UMaterialExpressionVectorParameter>(Expression))
			{
				Name = VectorParameter->ParameterName.ToString().ToLower();
			}
			else
				Name.Empty();

			return Name.Find(TEXT("V Tiling")) != INDEX_NONE;
		});

		UMaterialExpression* Result = NULL;

		if (FoundU && FoundV)
		{
			Result = ExpressionMakeFLoat2(*FoundU, *FoundV, FunctionLoader, Material);
		}

		if (Found)
		{
			UMaterialExpression* Expression = *Found;
			check(Expression);
			UMaterialExpression* TilingFactorFloat2;
			if (Expression->IsA<UMaterialExpressionScalarParameter>())
				TilingFactorFloat2 = ExpressionMakeFLoat2(Expression, FunctionLoader, Material);
			else
				TilingFactorFloat2 = ExpressionMakeFLoat2(Expression, FunctionLoader, Material, 0, 1);

			if (!Result)
			{
				Result = TilingFactorFloat2;
			}
			else
			{
				Result = Generator::NewMaterialExpressionMultiply(&Material, TilingFactorFloat2, Result);
			}

		}

		if (!Result)
		{
			UMaterialExpression* Tiling = Generator::NewMaterialExpressionScalarParameter(&Material, TEXT("Tiling Factor"), 1.f);
			Generator::SetMaterialExpressionGroup(TEXT("Other"), Tiling);
			Result = ExpressionMakeFLoat2(Tiling, FunctionLoader, Material);
		}

		return Result;
	}
}  // namespace Mat
