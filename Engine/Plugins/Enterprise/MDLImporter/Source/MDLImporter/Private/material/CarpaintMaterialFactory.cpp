// Copyright Epic Games, Inc. All Rights Reserved.

#include "CarpaintMaterialFactory.h"

#include "Materials/MaterialExpressionAdd.h"
#include "generator/FunctionLoader.h"
#include "Materials/MaterialExpressionConstant.h"
#include "generator/MaterialExpressions.h"
#include "Materials/MaterialExpressionConstant3Vector.h"
#include "mdl/Material.h"

#include "Materials/MaterialExpressionClearCoatNormalCustomOutput.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialExpressionSaturate.h"
#include "Materials/MaterialExpressionScalarParameter.h"
#include "Materials/MaterialExpressionTextureObjectParameter.h"
#include "Materials/MaterialExpressionVectorParameter.h"

namespace Mat
{
	namespace
	{
		template <typename TargetMap>
		void ConnectParameter(TargetMap& Target, const FString& GroupName, UMaterialExpression* Expression)
		{
			if (!Expression)
				return;

			Generator::Connect(Target, Expression);
			Generator::SetMaterialExpressionGroup(GroupName, Expression);
		}

		void SetTextureParameterName(const FString& Name, UMaterialExpression* Expression)
		{
			Cast<UMaterialExpressionTextureObjectParameter>(Expression)->SetParameterName(*Name);
		}

		void AddFunctionInput(Generator::FMaterialExpressionConnectionList& Inputs, const FString& GroupName, UMaterialExpression* Expression)
		{
			check(Expression);

			Inputs.Emplace(Expression);
			Generator::SetMaterialExpressionGroup(GroupName, Expression);
		}

		UMaterialExpression* GetScalarParameter(UMaterial& OutMaterial, const FString& Name, const FString& GroupName, float Value)
		{
			UMaterialExpression* Parameter = Generator::NewMaterialExpressionScalarParameter(&OutMaterial, Name, Value);
			Generator::SetMaterialExpressionGroup(GroupName, Parameter);
			return Parameter;
		}

		UMaterialExpression* GetNormalMapParameter(UMaterial& Material)
		{
			UTexture* Texture = LoadObject<UTexture>(nullptr, TEXT("/Engine/EngineMaterials/FlatNormal.FlatNormal"));
			check(Texture);
			UMaterialExpression* Parameter = Generator::NewMaterialExpressionTextureObjectParameter(&Material, "Normal Map", Texture);
			Generator::SetMaterialExpressionGroup(TEXT("Other"), Parameter);
			return Parameter;
		}
	}

	FCarpaintMaterialFactory::FCarpaintMaterialFactory(Generator::FFunctionLoader& FunctionLoader)
	    : FunctionLoader(FunctionLoader)
	{
		Inputs.Reserve(15);
	}

	void FCarpaintMaterialFactory::Create(const Mdl::FMaterial& MdlMaterial, const Mat::FParameterMap& Parameters, UMaterial& Material) const
	{
		check(MdlMaterial.Carpaint.bEnabled);

		// get under clear coat output
		UMaterialExpressionClearCoatNormalCustomOutput* UnderClearCoat;
		{
			auto Found = Material.GetExpressions().FindByPredicate(
			    [](const UMaterialExpression* Expr) { return Expr->IsA<UMaterialExpressionClearCoatNormalCustomOutput>(); });
			check(Found);
			UnderClearCoat = Cast<UMaterialExpressionClearCoatNormalCustomOutput>(*Found);
		};

		UMaterialExpression* Tiling = Parameters[EMaterialParameter::TilingU];  // Tiling should be always square for CarPaint
		Generator::SetMaterialExpressionGroup(TEXT("Other"), Tiling);

		UMaterialFunction*   MapCall;
		UMaterialExpression* Expression;

		// change some parameter names
		SetTextureParameterName(TEXT("Color Table"), Parameters[EMaterialParameter::BaseColorMap]);

		UMaterialEditorOnlyData& MaterialEditorOnly = *Material.GetEditorOnlyData();

		// clear coat
		ConnectParameter(MaterialEditorOnly.ClearCoat, TEXT("Clear Coat"), Parameters[EMaterialParameter::ClearCoatWeight]);
		ConnectParameter(MaterialEditorOnly.ClearCoatRoughness, TEXT("Clear Coat"), Parameters[EMaterialParameter::ClearCoatRoughness]);
		// light direction
		Expression = Generator::NewMaterialExpressionVectorParameter(&Material, TEXT("Light Direction"), FLinearColor(0.5f, 0.25f, 1.f));
		Generator::SetMaterialExpressionGroup(TEXT("Other"), Expression);
		MapCall                          = &FunctionLoader.Get(Generator::ECommonFunction::AngularDirection);
		UMaterialExpression* ThetaAngles = Generator::NewMaterialExpressionFunctionCall(&Material, MapCall, {Expression});

		// carpaint flakes
		TTuple<UMaterialExpression*, int> FlakesColor = CreateFlakes(MdlMaterial, Parameters, ThetaAngles, Tiling, Material);

		// carpaint color
		Inputs.Empty();
		MapCall = &FunctionLoader.Get(Generator::ECommonFunction::CarColorTable);
		Inputs.Emplace(ThetaAngles);
		AddFunctionInput(Inputs, TEXT("Color Shift"), Parameters[EMaterialParameter::BaseColorMap]);
		Expression = Generator::NewMaterialExpressionVectorParameter(&Material, TEXT("Color Intensity"), MdlMaterial.BaseColor.Value);
		Generator::SetMaterialExpressionGroup(TEXT("Color Shift"), Expression);
		AddFunctionInput(Inputs, TEXT("Color Shift"), Expression);
		if (FlakesColor.Get<0>())
			Inputs.Emplace(Generator::FMaterialExpressionConnection(FlakesColor.Get<0>(), FlakesColor.Get<1>()));
		else
			Inputs.Emplace(Generator::NewMaterialExpressionConstant(&Material, 0.f));
		Generator::Connect(MaterialEditorOnly.BaseColor, Generator::NewMaterialExpressionFunctionCall(&Material, MapCall, Inputs));

		MapCall = &FunctionLoader.Get(Generator::ECommonFunction::NormalMap);

		// clearcoat normal
		if (Parameters.Contains(EMaterialParameter::ClearCoatNormalMap))
		{
			Inputs.Empty();
			AddFunctionInput(Inputs, TEXT("Clear Coat"), Parameters[EMaterialParameter::ClearCoatNormalMap]);
			Inputs.Emplace(Tiling);
			AddFunctionInput(Inputs, TEXT("Clear Coat"), Parameters[EMaterialParameter::ClearCoatNormalStrength]);
			Generator::Connect(MaterialEditorOnly.Normal, Generator::NewMaterialExpressionFunctionCall(&Material, MapCall, Inputs));
		}

		// under-clearcoat normal
		Inputs.Empty();
		AddFunctionInput(Inputs, TEXT("Normal"), GetNormalMapParameter(Material));
		Inputs.Emplace(Tiling);
		AddFunctionInput(Inputs, TEXT("Normal"), GetScalarParameter(Material, TEXT("Normal Strength"), TEXT("Normal"), MdlMaterial.Normal.Strength));
		Generator::Connect(UnderClearCoat->Input, Generator::NewMaterialExpressionFunctionCall(&Material, MapCall, Inputs));

		// if it has flakes then properties are already connected
		if (!FlakesColor.Get<0>())
		{
			// brdf
			ConnectParameter(MaterialEditorOnly.Metallic, TEXT("BRDF"), Parameters[EMaterialParameter::Metallic]);
			ConnectParameter(MaterialEditorOnly.Specular, TEXT("BRDF"), Parameters[EMaterialParameter::Specular]);
			ConnectParameter(MaterialEditorOnly.Roughness, TEXT("BRDF"), Parameters[EMaterialParameter::Roughness]);
		}
	}

	TTuple<UMaterialExpression*, int> FCarpaintMaterialFactory::CreateFlakes(const Mdl::FMaterial& MdlMaterial, const FParameterMap& Parameters,
	                                                                         UMaterialExpression* ThetaAngles, UMaterialExpression* Tiling,
	                                                                         UMaterial& Material) const
	{
		TTuple<UMaterialExpression*, int> ColorConnection = MakeTuple<UMaterialExpression*, int>(nullptr, 0);
		if (!MdlMaterial.Carpaint.Flakes.Depth)
			return ColorConnection;

		const auto& CarpaintProperties = MdlMaterial.Carpaint;
		Inputs.Empty();
		Inputs.Emplace(Generator::NewMaterialExpressionConstant(&Material, CarpaintProperties.FlakesColorValue[0], CarpaintProperties.FlakesColorValue[1], CarpaintProperties.FlakesColorValue[2]));
		Inputs.Emplace(ThetaAngles);
		Inputs.Emplace(Parameters[EMaterialParameter::CarFlakesMap]);
		Inputs.Emplace(Generator::NewMaterialExpressionConstant(&Material, CarpaintProperties.Flakes.Depth));
		Inputs.Emplace(Generator::NewMaterialExpressionConstant(&Material, CarpaintProperties.Flakes.Size));
		Inputs.Emplace(Parameters[EMaterialParameter::CarFlakesLut]);
		Inputs.Emplace(Generator::NewMaterialExpressionConstant(&Material, CarpaintProperties.ThetaFiLUT.TexelSize));
		Inputs.Emplace(Generator::NewMaterialExpressionConstant(&Material, CarpaintProperties.NumThetaF));
		Inputs.Emplace(Generator::NewMaterialExpressionConstant(&Material, CarpaintProperties.NumThetaI));
		Inputs.Emplace(Generator::NewMaterialExpressionConstant(&Material, CarpaintProperties.MaxThetaI));
		Inputs.Emplace(Tiling);

		UMaterialFunction*                       Function     = &FunctionLoader.Get(Generator::ECommonFunction::CarFlakes);
		UMaterialExpressionMaterialFunctionCall* FunctionCall = Generator::NewMaterialExpressionFunctionCall(&Material, Function, Inputs);
		check(FunctionCall->FunctionOutputs.Num() == 4);

		Generator::SetMaterialExpressionGroup(TEXT("Flakes"), Parameters[EMaterialParameter::CarFlakesMap]);

		UMaterialEditorOnlyData& MaterialEditorOnly = *Material.GetEditorOnlyData();

		// brdf
		Inputs.Empty();
		Inputs.Emplace(FunctionCall, 1);
		Inputs.Emplace(Parameters[EMaterialParameter::Metallic]);
		Generator::Connect(MaterialEditorOnly.Metallic,
		                   Generator::NewMaterialExpressionSaturate(&Material, Generator::NewMaterialExpressionAdd(&Material, Inputs)));
		Inputs.Empty();
		Inputs.Emplace(FunctionCall, 3);
		Inputs.Emplace(Parameters[EMaterialParameter::Roughness]);
		Generator::Connect(MaterialEditorOnly.Roughness,
		                   Generator::NewMaterialExpressionSaturate(&Material, Generator::NewMaterialExpressionAdd(&Material, Inputs)));
		Inputs.Empty();
		Inputs.Emplace(FunctionCall, 2);
		Inputs.Emplace(Parameters[EMaterialParameter::Specular]);
		Generator::Connect(MaterialEditorOnly.Specular,
		                   Generator::NewMaterialExpressionSaturate(&Material, Generator::NewMaterialExpressionAdd(&Material, Inputs)));

		Generator::SetMaterialExpressionGroup(TEXT("BRDF"), Parameters[EMaterialParameter::Metallic]);
		Generator::SetMaterialExpressionGroup(TEXT("BRDF"), Parameters[EMaterialParameter::Roughness]);
		Generator::SetMaterialExpressionGroup(TEXT("BRDF"), Parameters[EMaterialParameter::Specular]);

		ColorConnection.Get<0>() = FunctionCall;
		return ColorConnection;
	}
}  // namespace Mat
