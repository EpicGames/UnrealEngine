// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "generator/MaterialExpressionConnection.h"
#include "material/MaterialFactory.h"


namespace Generator
{
	class FFunctionLoader;
}
namespace Mat
{
	class FCarpaintMaterialFactory : public IMaterialFactory
	{
	public:
		FCarpaintMaterialFactory(Generator::FFunctionLoader& FunctionLoader);

		virtual void Create(const Mdl::FMaterial& MdlMaterial, const Mat::FParameterMap& Parameters, UMaterial& Material) const override;

	private:
		TTuple<UMaterialExpression*, int> CreateFlakes(const Mdl::FMaterial&     MdlMaterial,
		                                               const Mat::FParameterMap& Parameters,
		                                               UMaterialExpression*      ThetaAngles,
		                                               UMaterialExpression*      Tiling,
		                                               UMaterial&                OutMaterial) const;

	private:
		mutable Generator::FMaterialExpressionConnectionList Inputs;
		Generator::FFunctionLoader&                          FunctionLoader;
	};
}
