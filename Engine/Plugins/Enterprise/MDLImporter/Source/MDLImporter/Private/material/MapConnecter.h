// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "generator/FunctionLoader.h"
#include "generator/MaterialExpressionConnection.h"
#include "generator/MaterialExpressions.h"
#include "material/MaterialFactory.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"

namespace Mat
{
	struct FMapConnecter
	{
		FMapConnecter(const FParameterMap& Parameters, Generator::FFunctionLoader& FunctionLoader, UMaterialExpression* Tiling, UMaterial& Material)
		    : Parameters(Parameters)
		    , FunctionLoader(FunctionLoader)
		    , Tiling(Tiling)
		    , Material(Material)
		{
			Inputs.Reserve(5);
		}

		void DeleteExpression(EMaterialParameter Parameter)
		{
			if (!Parameters.Contains(Parameter))
				return;

			UMaterialExpression* Expression = Parameters[Parameter];
			Material.GetExpressionCollection().RemoveExpression(Expression);
		}

		void DeleteExpressionMap(EMaterialParameter Parameter)
		{
			DeleteExpression(Parameter);

			const EMaterialParameter ParameterTexture = (EMaterialParameter)(int(Parameter) + 1);
			DeleteExpression(ParameterTexture);
		}

		template <typename TargetMap>
		void ConnectParameter(TargetMap& Target, const FString& GroupName, EMaterialParameter Parameter)
		{
			if (!Parameters.Contains(Parameter))
				return;

			UMaterialExpression* Expression = Parameters[Parameter];
			Generator::Connect(Target, Expression);
			Generator::SetMaterialExpressionGroup(GroupName, Expression);
		}

		UMaterialExpression* CreateMapCall(const FString& GroupName, EMaterialParameter Parameter, bool bIsColorMap,
		                                   UMaterialExpression* ColorExpression)
		{
			UMaterialFunction* MapCall =
			    &FunctionLoader.Get(bIsColorMap ? Generator::ECommonFunction::ColorMap : Generator::ECommonFunction::GrayscaleMap);

			Inputs.Empty();
			Inputs.Emplace(Parameters[Parameter]);
			Inputs.Emplace(Tiling, 0);
			Inputs.Emplace(ColorExpression, 0);
			Generator::SetMaterialExpressionGroup(GroupName, Inputs[0].GetExpressionUnused());

			return Generator::NewMaterialExpressionFunctionCall(&Material, MapCall, Inputs);
		}

		UMaterialExpression* CreateParameterMap(const FString& GroupName, EMaterialParameter Parameter, bool bIsColorMap = true,
		                                        UMaterialExpression* ColorExpression = nullptr)
		{
			const EMaterialParameter ParameterTexture = (EMaterialParameter)(int(Parameter) + 1);

			UMaterialExpression* Expression = nullptr;
			if (Parameters.Contains(ParameterTexture))
			{
				Expression = CreateMapCall(GroupName, ParameterTexture, bIsColorMap, ColorExpression);
			}
			else if (Parameters.Contains(Parameter))
			{
				Expression = Parameters[Parameter];
				Generator::SetMaterialExpressionGroup(GroupName, Expression);
			}

			return Expression;
		}

		template <typename TargetMap>
		UMaterialExpression* ConnectParameterMap(TargetMap& Target, const FString& GroupName, EMaterialParameter Parameter, bool bIsColorMap = true,
		                                         UMaterialExpression* ColorExpression = nullptr)
		{
			UMaterialExpression* Expression = CreateParameterMap(GroupName, Parameter, bIsColorMap, ColorExpression);

			if (Expression)
				Generator::Connect(Target, Expression);
			return Expression;
		}

		template <typename TargetMap>
		UMaterialExpression* ConnectNormalMap(TargetMap& Target, const FString& GroupName, EMaterialParameter Parameter)
		{
			if (!Parameters.Contains(Parameter))
				return nullptr;

			const EMaterialParameter ParameterStrength = (EMaterialParameter)(int(Parameter) + 1);
			Inputs.Empty();
			Inputs.Emplace(Parameters[Parameter]);
			Inputs.Emplace(Tiling, 0);
			Inputs.Emplace(Parameters[ParameterStrength]);
			Generator::SetMaterialExpressionGroup(GroupName, Inputs[0].GetExpressionAndUse());
			Generator::SetMaterialExpressionGroup(GroupName, Inputs[2].GetExpressionAndUse());

			UMaterialFunction*   MapCall    = &FunctionLoader.Get(Generator::ECommonFunction::NormalMap);
			UMaterialExpression* Expression = Generator::NewMaterialExpressionFunctionCall(&Material, MapCall, Inputs);
			Generator::Connect(Target, Expression);

			return Expression;
		}

	private:
		const FParameterMap&                         Parameters;
		Generator::FFunctionLoader&                  FunctionLoader;
		UMaterialExpression*                         Tiling;
		UMaterial&                                   Material;
		Generator::FMaterialExpressionConnectionList Inputs;
	};
}  // namespace Mat
