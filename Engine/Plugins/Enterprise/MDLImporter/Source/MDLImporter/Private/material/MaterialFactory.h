// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"

class UMaterial;

class UMaterialExpression;
namespace Mdl
{
	struct FMaterial;
}
namespace Mat
{
	enum class EMaterialParameter
	{
		BaseColor = 0,
		BaseColorMap,
		SubSurfaceColor,
		SubSurfaceColorMap,
		EmissionColor,
		EmissionColorMap,
		EmissionStrength,
		Roughness,
		RoughnessMap,
		Metallic,
		MetallicMap,
		Specular,
		SpecularMap,
		IOR,
		AbsorptionColor,
		Opacity,
		OpacityMap,
		NormalMap,
		NormalStrength,
		DisplacementMap,
		DisplacementStrength,
		ClearCoatWeight,
		ClearCoatWeightMap,
		ClearCoatRoughness,
		ClearCoatRoughnessMap,
		ClearCoatNormalMap,
		ClearCoatNormalStrength,
		CarFlakesMap,
		CarFlakesLut,
		Tiling,
		TilingU,
		TilingV
	};

	using FParameterMap = TMap<EMaterialParameter, UMaterialExpression*>;

	class IMaterialFactory
	{
	public:
		virtual ~IMaterialFactory() {};

		virtual void Create(const Mdl::FMaterial& MdlMaterial, const Mat::FParameterMap& Parameters, UMaterial& Material) const = 0;
	};
}
