// Copyright Epic Games, Inc. All Rights Reserved.

#include "TranslucentMaterialFactory.h"

#include "Materials/MaterialExpressionMultiply.h"
#include "material/BakedMaterialFactory.h"
#include "Materials/MaterialExpressionVectorParameter.h"
#include "material/MapConnecter.h"

namespace Mat
{
	FTranslucentMaterialFactory::FTranslucentMaterialFactory(Generator::FFunctionLoader& FunctionLoader)
	    : FunctionLoader(FunctionLoader)
	{
	}

	void FTranslucentMaterialFactory::Create(const Mdl::FMaterial& MdlMaterial, const Mat::FParameterMap& Parameters, UMaterial& Material) const
	{
		// get the tiling parameter, if not present create one
		UMaterialExpression* Tiling = nullptr;
		{
			EMaterialParameter Maps[] = {EMaterialParameter::BaseColorMap,   EMaterialParameter::MetallicMap,  EMaterialParameter::NormalMap,
			                             EMaterialParameter::OpacityMap,     EMaterialParameter::RoughnessMap, EMaterialParameter::SpecularMap,
			                             EMaterialParameter::SubSurfaceColor};
			for (EMaterialParameter MapType : Maps)
			{
				if (Parameters.Contains(MapType))
				{
					Tiling = FBakedMaterialFactory::GetTilingParameter(FunctionLoader, Material);
					break;
				}
			}
			if (Tiling)
				Generator::SetMaterialExpressionGroup(TEXT("Other"), Tiling);
		}

		Mat::FMapConnecter   MapConnecter(Parameters, FunctionLoader, Tiling, Material);
		UMaterialFunction*   MapCall    = nullptr;
		UMaterialExpression* Expression = nullptr;

		UMaterialEditorOnlyData& MaterialEditorOnly = *Material.GetEditorOnlyData();

		// normal
		UMaterialExpression* NormalExpression = MapConnecter.ConnectNormalMap(MaterialEditorOnly.Normal, TEXT("Normal"), EMaterialParameter::NormalMap);

		// brdf
		MapConnecter.ConnectParameterMap(MaterialEditorOnly.Metallic, TEXT("BRDF"), EMaterialParameter::Metallic, false);
		MapConnecter.ConnectParameterMap(MaterialEditorOnly.Specular, TEXT("BRDF"), EMaterialParameter::Specular, false);
		MapConnecter.ConnectParameterMap(MaterialEditorOnly.Roughness, TEXT("BRDF"), EMaterialParameter::Roughness, false);

		if (Material.BlendMode == BLEND_Masked && Parameters.Contains(EMaterialParameter::OpacityMap))
		{
			// masked

			// color
			MapConnecter.ConnectParameterMap(MaterialEditorOnly.BaseColor, TEXT("Color"), EMaterialParameter::BaseColor);

			UMaterialExpression* Opacity = MapConnecter.CreateMapCall(TEXT("Opacity"), EMaterialParameter::OpacityMap, false, nullptr);

			MapCall = &FunctionLoader.Get(Generator::ECommonFunction::DitherTemporalAA);
			Generator::Connect(MaterialEditorOnly.OpacityMask, Generator::NewMaterialExpressionFunctionCall(&Material, MapCall, {{Opacity, 0}, {nullptr, 0}}));
			MapConnecter.DeleteExpression(EMaterialParameter::AbsorptionColor);
			MapConnecter.DeleteExpression(EMaterialParameter::IOR);
		}
		else if (Material.BlendMode == BLEND_Translucent)
		{
			// translucency

			// color
			Expression = MapConnecter.CreateParameterMap(TEXT("Color"), EMaterialParameter::BaseColor);
			Expression = Generator::NewMaterialExpressionMultiply(&Material, {Expression, Parameters[EMaterialParameter::Opacity]});
			Generator::Connect(MaterialEditorOnly.BaseColor, Expression);

			// path length expression
			Expression = Generator::NewMaterialExpressionVectorParameter(&Material, TEXT("Local Bounds Adjustment"), FLinearColor(0.f, 0.f, 1.f));
			Generator::SetMaterialExpressionGroup(TEXT("Translucency"), Expression);
			MapCall = &FunctionLoader.Get(Generator::ECommonFunction::EstimateObjectThickness);
			UMaterialExpression* PathLengthExpression =
			    Generator::NewMaterialExpressionFunctionCall(&Material, MapCall, {{Expression, 0}, {Expression, 4}});

			// volume absorption color
			Expression = Generator::NewMaterialExpressionVectorParameter(&Material, TEXT("Surface Transmission Color"), FLinearColor(1.f, 1.f, 1.f));
			Generator::SetMaterialExpressionGroup(TEXT("Translucency"), Expression);
			Generator::SetMaterialExpressionGroup(TEXT("Translucency"), Parameters[EMaterialParameter::AbsorptionColor]);
			Generator::SetMaterialExpressionGroup(TEXT("Translucency"), Parameters[EMaterialParameter::IOR]);
			MapCall = &FunctionLoader.Get(Generator::ECommonFunction::VolumeAbsorptionColor);
			UMaterialExpression* VolumeAbsorptionExpression =
			    Generator::NewMaterialExpressionFunctionCall(&Material, MapCall,
			                                                 {{Parameters[EMaterialParameter::AbsorptionColor], 0},  //
			                                                  {Expression, 0},                                       //
			                                                  {Parameters[EMaterialParameter::IOR], 0},              //
			                                                  {PathLengthExpression, 0},                             //
			                                                  {NormalExpression, 0}});
			Generator::Connect(MaterialEditorOnly.EmissiveColor, VolumeAbsorptionExpression);

			// opacity
			MapCall = &FunctionLoader.Get(Generator::ECommonFunction::TranslucentOpacity);
			Generator::Connect(MaterialEditorOnly.Opacity, Generator::NewMaterialExpressionFunctionCall(&Material, MapCall, {}));
		}
	}
}  // namespace Mat
