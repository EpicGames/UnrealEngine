// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "material/MaterialFactory.h"


namespace Generator
{
	class FFunctionLoader;
}
namespace Mat
{
	class FTranslucentMaterialFactory : public IMaterialFactory
	{
	public:
		FTranslucentMaterialFactory(Generator::FFunctionLoader& FunctionLoader);

		virtual void Create(const Mdl::FMaterial& MdlMaterial, const Mat::FParameterMap& Parameters, UMaterial& Material) const override;

	private:
		Generator::FFunctionLoader& FunctionLoader;
	};
}
