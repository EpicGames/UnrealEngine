// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef USE_MDLSDK

#include "ApiContext.h"

#include "Common.h"
#include "MaterialCollection.h"
#include "MaterialDistiller.h"
#include "MdlSdkDefines.h"

#include "Misc/Paths.h"

MDLSDK_INCLUDES_START
#include "mi/neuraylib/factory.h"
#include "mi/neuraylib/idatabase.h"
#include "mi/neuraylib/imaterial_definition.h"
#include "mi/neuraylib/imdl_compiler.h"
#include "mi/neuraylib/imdl_configuration.h"
#include "mi/neuraylib/imdl_factory.h"
#include "mi/neuraylib/imdl_impexp_api.h"
#include "mi/neuraylib/imodule.h"
#include "mi/neuraylib/ineuray.h"
#include <mi/base/config.h>
#include <mi/neuraylib/iplugin_configuration.h>
#include "mi/neuraylib/iscope.h"
#include "mi/neuraylib/iversion.h"
#include <mi/base/enums.h>
#include <mi/base/ilogger.h>
#include <mi/base/iinterface.h>
#include <mi/base/interface_implement.h>
#include <mi/base/interface_declare.h>
#include <mi/base/types.h>
#include <mi/base/uuid.h>
#include <mi/neuraylib/iexpression.h>
#include <mi/neuraylib/itransaction.h>
#include <mi/neuraylib/version.h>
MDLSDK_INCLUDES_END

namespace Mdl
{
	namespace
	{
		bool MaterialIsHidden(const char* MaterialName, mi::neuraylib::ITransaction& Transaction)
		{
			mi::base::Handle<const mi::neuraylib::IMaterial_definition> MaterialDefinition =
			    mi::base::make_handle(Transaction.access<mi::neuraylib::IMaterial_definition>(MaterialName));

			const mi::base::Handle<const mi::neuraylib::IAnnotation_block> AnnotationBlock(MaterialDefinition->get_annotations());
			if (AnnotationBlock)
			{
				for (mi::Size annoIdx = 0; annoIdx < AnnotationBlock->get_size(); annoIdx++)
				{
					mi::base::Handle<const mi::neuraylib::IAnnotation> Annotation(AnnotationBlock->get_annotation(annoIdx));
					const char*                                        anno = Annotation->get_name();
					if (strcmp(Annotation->get_name(), "::anno::hidden()") == 0)
					{
						return true;
					}
				}
			}
			return false;
		}

		FString NormalizeDirectoryPathForMdl(const TCHAR* DirectoryPath)
		{
			FString NormalizedPath = DirectoryPath;
			FPaths::NormalizeDirectoryName(NormalizedPath);
			NormalizedPath += TEXT("/");

			return NormalizedPath;
		}
	}

	class FLogger : public mi::base::Interface_implement_singleton<mi::base::ILogger>
	{
	public:
		virtual void message(mi::base::Message_severity Level, const char* ModuleCategory, const mi::base::Message_details&, const char* Message) override
		{
			switch (Level)
			{
				case mi::base::MESSAGE_SEVERITY_FATAL:
					Messages.Emplace(MDLImporterLogging::EMessageSeverity::Error, FString(UTF8_TO_TCHAR(Message)));
					UE_LOGF(LogMDLImporter, Fatal, "topic: %ls, %ls", UTF8_TO_TCHAR(ModuleCategory), UTF8_TO_TCHAR(Message));
					break;
				case mi::base::MESSAGE_SEVERITY_ERROR:
					Messages.Emplace(MDLImporterLogging::EMessageSeverity::Error, FString(UTF8_TO_TCHAR(Message)));
					UE_LOGF(LogMDLImporter, Error, "topic: %ls, %ls", UTF8_TO_TCHAR(ModuleCategory), UTF8_TO_TCHAR(Message));
					break;
				case mi::base::MESSAGE_SEVERITY_WARNING:
					Messages.Emplace(MDLImporterLogging::EMessageSeverity::Warning, FString(UTF8_TO_TCHAR(Message)));
					UE_LOGF(LogMDLImporter, Warning, "topic: %ls, %ls", UTF8_TO_TCHAR(ModuleCategory), UTF8_TO_TCHAR(Message));
					break;
				case mi::base::MESSAGE_SEVERITY_INFO:
					UE_LOGF(LogMDLImporter, Log, "topic: %ls, %ls", UTF8_TO_TCHAR(ModuleCategory), UTF8_TO_TCHAR(Message));
					break;
				case mi::base::MESSAGE_SEVERITY_VERBOSE:
				case mi::base::MESSAGE_SEVERITY_DEBUG:
					UE_LOGF(LogMDLImporter, Verbose, "topic: %ls, %ls", UTF8_TO_TCHAR(ModuleCategory), UTF8_TO_TCHAR(Message));
					break;
				default:
					break;
			}
		}

		TArray<MDLImporterLogging::FLogMessage> Messages;
	};

	FApiContext::FApiContext()
	    : DsoHandle(nullptr)
	{
	}

	FApiContext::~FApiContext()
	{
		Unload(false);
	}

	bool FApiContext::Load(const FString& LibrariesPath, const FString& ModulesPath)
	{
		// Loads the MDL SDK and calls the main factory function.

		FString FilePath = FPaths::Combine(LibrariesPath, TEXT("libmdl_sdk" MI_BASE_DLL_FILE_EXT));
		DsoHandle        = FPlatformProcess::GetDllHandle(*FilePath);

		if (!DsoHandle)
		{
			UE_LOGF(LogMDLImporter, Error, "Error: Library %ls does not exist in the path %ls", TEXT("libmdl_sdk" MI_BASE_DLL_FILE_EXT), *LibrariesPath);
			return false;
		}

		void* SymbolPtr  = FPlatformProcess::GetDllExport(DsoHandle, TEXT("mi_factory"));
		if (!SymbolPtr)
		{
			UE_LOGF(LogMDLImporter, Error, "Error: Symbol %ls does not exist in the library %ls", TEXT("mi_factory"), TEXT("libmdl_sdk" MI_BASE_DLL_FILE_EXT));
			return false;
		}

		mi::neuraylib::INeuray* Neuray = mi::neuraylib::mi_factory<mi::neuraylib::INeuray>(SymbolPtr);
		if (!Neuray)
		{
			mi::base::Handle<mi::neuraylib::IVersion> Version(mi::neuraylib::mi_factory<mi::neuraylib::IVersion>(SymbolPtr));
			if (!Version)
			{
				UE_LOGF(LogMDLImporter, Error, "Error: Incompatible library.");
			}
			else
			{
				UE_LOGF(LogMDLImporter, Error, "Error: Library version %ls does not match header version %ls.",
				       ANSI_TO_TCHAR(Version->get_product_version()), ANSI_TO_TCHAR(MI_NEURAYLIB_PRODUCT_VERSION_STRING));
			}

			return false;
		}

		NeurayHandle = Neuray;

		mi::neuraylib::IMdl_compiler* Compiler = NeurayHandle->get_api_component<mi::neuraylib::IMdl_compiler>();

		// Load the FreeImage plugin.
		mi::base::Handle<mi::neuraylib::IPlugin_configuration> PluginConfig(NeurayHandle->get_api_component<mi::neuraylib::IPlugin_configuration>());
		FString FreeimageModulePath = FPaths::Combine(LibrariesPath, TEXT("nv_freeimage" MI_BASE_DLL_FILE_EXT));
		mi::Sint32 Result = PluginConfig->load_plugin_library(TCHAR_TO_ANSI(*FreeimageModulePath));
		if ( Result != 0)
		{
			UE_LOGF(LogMDLImporter, Error, "mi::neuraylib::IPlugin_configuration::load_plugin_library() failed with return code %d.", Result);
			return false;
		}

		// Load the Distiller plugin.
		FString DistillerModulePath = FPaths::Combine(LibrariesPath, TEXT("mdl_distiller" MI_BASE_DLL_FILE_EXT));
		Result = PluginConfig->load_plugin_library(TCHAR_TO_ANSI(*DistillerModulePath));
		if ( Result != 0)
		{
			UE_LOGF(LogMDLImporter, Error, "mi::neuraylib::IPlugin_configuration::load_plugin_library() failed with return code %d.", Result);
			return false;
		}

		Result = Neuray->start(true);
		if (Result)
		{
			UE_LOGF(LogMDLImporter, Error, "mi::neuraylib::INeuray::start() failed with return code %d.", Result);
			return false;
		}

		CompilerHandle = NeurayHandle->get_api_component<mi::neuraylib::IMdl_compiler>();
		DatabaseHandle = NeurayHandle->get_api_component<mi::neuraylib::IDatabase>();
		FactoryHandle  = NeurayHandle->get_api_component<mi::neuraylib::IMdl_factory>();
		DistillerPtr.Reset(new FMaterialDistiller(NeurayHandle));

		LoggerPtr = new FLogger();

		ConfigHandle = NeurayHandle->get_api_component<mi::neuraylib::IMdl_configuration>();
		ConfigHandle->set_logger(LoggerPtr);

		AddSearchPath(ModulesPath);

		LogInfo();

		return true;
	}

	void FApiContext::Unload(bool bClearDatabaseOnly)
	{
		if (bClearDatabaseOnly)
		{
			DatabaseHandle->garbage_collection();
			return;
		}
		DistillerPtr.Reset();
		ConfigHandle.reset();
		CompilerHandle.reset();
		DatabaseHandle.reset();
		FactoryHandle.reset();

		if (NeurayHandle)
		{
			MDL_CHECK_RESULT_MSG("Failed to shutdown neuray!") = NeurayHandle->shutdown(true);
		}
		NeurayHandle = 0;

		if (DsoHandle)
		{
			FPlatformProcess::FreeDllHandle(DsoHandle);
			DsoHandle = nullptr;
		}
	}

	void FApiContext::AddSearchPath(const FString& ModulesPath)
	{
		if (ModulesPath.IsEmpty() || !FPaths::DirectoryExists(ModulesPath))
		{
			return;
		}

		const FString AbsolutePath = NormalizeDirectoryPathForMdl(*ModulesPath);
		MDL_CHECK_RESULT() = ConfigHandle->add_mdl_path(TCHAR_TO_ANSI(*AbsolutePath));
	}

	void FApiContext::RemoveSearchPath(const FString& ModulesPath)
	{
		if (ModulesPath.IsEmpty() || !FPaths::DirectoryExists(ModulesPath))
		{
			return;
		}

		const FString AbsolutePath = NormalizeDirectoryPathForMdl(*ModulesPath);
		MDL_CHECK_RESULT() = ConfigHandle->remove_mdl_path(TCHAR_TO_ANSI(*AbsolutePath));
	}

	void FApiContext::AddResourceSearchPath(const FString& ResourcesPath)
	{
		if (ResourcesPath.IsEmpty() || !FPaths::DirectoryExists(ResourcesPath))
		{
			return;
		}

		const FString AbsolutePath = NormalizeDirectoryPathForMdl(*ResourcesPath);
		MDL_CHECK_RESULT() = ConfigHandle->add_resource_path(TCHAR_TO_ANSI(*AbsolutePath));
	}

	void FApiContext::RemoveResourceSearchPath(const FString& ResourcesPath)
	{
		if (ResourcesPath.IsEmpty() || !FPaths::DirectoryExists(ResourcesPath))
		{
			return;
		}

		const FString AbsolutePath = NormalizeDirectoryPathForMdl(*ResourcesPath);
		MDL_CHECK_RESULT() = ConfigHandle->remove_resource_path(TCHAR_TO_ANSI(*AbsolutePath));
	}

	bool FApiContext::LoadModule(const FString& InModuleName, FMaterialCollection& OutMaterials)
	{
		// get transaction
		const mi::base::Handle<mi::neuraylib::IScope>       Scope(DatabaseHandle->get_global_scope());
		const mi::base::Handle<mi::neuraylib::ITransaction> Transaction(Scope->create_transaction());

		mi::base::Handle<mi::neuraylib::IMdl_impexp_api> MdlImpExpApi(NeurayHandle->get_api_component<mi::neuraylib::IMdl_impexp_api>());

		mi::Sint32 Result = MdlImpExpApi->load_module(Transaction.get(), TCHAR_TO_ANSI(*InModuleName));
		if (Result < 0)
		{
			UE_LOGF(LogMDLImporter, Error, "Failed to load MDL module (%u): %ls", Result, *InModuleName);
			MDL_CHECK_RESULT() = Transaction->commit();
			return false;
		}

		const FString ElementName = TEXT("mdl") + InModuleName;
		{
			mi::base::Handle<const mi::neuraylib::IModule> Module(Transaction->access<mi::neuraylib::IModule>(TCHAR_TO_ANSI(*ElementName)));
			if (!Module.is_valid_interface())
			{
				UE_LOGF(LogMDLImporter, Error, "Invalid Module interface: %ls", *ElementName);
				MDL_CHECK_RESULT() = Transaction->commit();
				return false;
			}

			const mi::Size Count = Module->get_material_count();
			OutMaterials.Reserve(static_cast<int32>(Count));
			OutMaterials.Name = InModuleName;
			OutMaterials.Name.RemoveAt(0, 2);  // remove prefix
			for (mi::Size Index = 0; Index < Count; ++Index)
			{
				const char* Name = Module->get_material(Index);
				if (MaterialIsHidden(Name, *Transaction))
				{
					continue;
				}

				mi::base::Handle<const mi::neuraylib::IMaterial_definition> MaterialDefinition =
				    mi::base::make_handle((*Transaction).access<mi::neuraylib::IMaterial_definition>(Name));

				FString MdlName = ANSI_TO_TCHAR(MaterialDefinition->get_mdl_name());
				FString MdlModuleName = ANSI_TO_TCHAR(MaterialDefinition->get_mdl_module_name());
				FString MdlSimpleName = ANSI_TO_TCHAR(MaterialDefinition->get_mdl_simple_name());
				
				FMaterial& Material = OutMaterials.Create();
				Material.Id = Index;
				
				// Strip Module name, format is: mdl::<module_name>::<material_name>(<material-as-function parameter list>)
				FString MaterialMdlDbName = ANSI_TO_TCHAR(Name);

				int32 OpeningBracketIndex = INDEX_NONE;
				MaterialMdlDbName.FindChar(TEXT('('), OpeningBracketIndex);

				// Search module separator skipping function arguments - parameter types can also contain "::"
				const int32 Found = MaterialMdlDbName.Find(TEXT("::"), ESearchCase::IgnoreCase, ESearchDir::FromEnd, OpeningBracketIndex);
				if ((Found - 2) > 0)
				{
					MaterialMdlDbName = MaterialMdlDbName.Right(MaterialMdlDbName.Len() - Found - 2);
				}

				Material.BaseName = MaterialMdlDbName;

				// Use material name without full signature (previously material db name returned from MDL SDK didn't contain parameter list)
				Material.Name = MdlSimpleName;
			}
		}
		MDL_CHECK_RESULT() = Transaction->commit();

		return true;
	}

	bool FApiContext::UnloadModule(const FString& InModuleName)
	{
		const mi::base::Handle<mi::neuraylib::IScope>       Scope(DatabaseHandle->get_global_scope());
		const mi::base::Handle<mi::neuraylib::ITransaction> Transaction(Scope->create_transaction());

		const FString ElementName = TEXT("mdl") + InModuleName;

		const auto* Module = Transaction->access<mi::neuraylib::IModule>(TCHAR_TO_ANSI(*ElementName));
		if (Module)
		{
			const mi::Size Count = Module->get_material_count();
			for (mi::Size i = 0; i < Count; ++i)
			{
				MDL_CHECK_RESULT() = Transaction->remove(Module->get_material(i));
			}

			Module->release();
			MDL_CHECK_RESULT() = Transaction->remove(TCHAR_TO_ANSI(*ElementName));
		}
		MDL_CHECK_RESULT() = Transaction->commit();
		return Module != nullptr;
	}

	void FApiContext::LogInfo()
	{
		mi::base::Handle<const mi::neuraylib::IVersion> Version(NeurayHandle->get_api_component<const mi::neuraylib::IVersion>());

		UE_LOGF(LogMDLImporter, Log, "MDL SDK header version          = %ls", TEXT(MI_NEURAYLIB_PRODUCT_VERSION_STRING));
		UE_LOGF(LogMDLImporter, Log, "MDL SDK library product name    = %ls", UTF8_TO_TCHAR(Version->get_product_name()));
		UE_LOGF(LogMDLImporter, Log, "MDL SDK library product version = %ls", UTF8_TO_TCHAR(Version->get_product_version()));
		UE_LOGF(LogMDLImporter, Log, "MDL SDK library build number    = %ls", UTF8_TO_TCHAR(Version->get_build_number()));
		UE_LOGF(LogMDLImporter, Log, "MDL SDK library build date      = %ls", UTF8_TO_TCHAR(Version->get_build_date()));
		UE_LOGF(LogMDLImporter, Log, "MDL SDK library build platform  = %ls", UTF8_TO_TCHAR(Version->get_build_platform()));
		UE_LOGF(LogMDLImporter, Log, "MDL SDK library version string  = \"%ls\"", UTF8_TO_TCHAR(Version->get_string()));

		mi::base::Uuid NeurayIdLibArray  = Version->get_neuray_iid();
		mi::base::Uuid NeurayIdInterface = mi::neuraylib::INeuray::IID();

		UE_LOGF(LogMDLImporter, Log, "MDL SDK header interface ID           = <%2x, %2x, %2x, %2x>", NeurayIdInterface.m_id1,
		       NeurayIdInterface.m_id2, NeurayIdInterface.m_id3, NeurayIdInterface.m_id4);
		UE_LOGF(LogMDLImporter, Log, "MDL SDK library interface ID          = <%2x, %2x, %2x, %2x>\n", NeurayIdLibArray.m_id1,
		       NeurayIdLibArray.m_id2, NeurayIdLibArray.m_id3, NeurayIdLibArray.m_id4);
	}

	TArray<MDLImporterLogging::FLogMessage> FApiContext::GetLogMessages() const /*override*/
	{
		TArray<MDLImporterLogging::FLogMessage> Messages;
		Swap(Messages, LoggerPtr->Messages);
		return Messages;
	}
}

#endif  // USE_MDLSDK
