// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "common/Logging.h"


namespace Mdl
{
	class FMaterialCollection;
	class FMaterialDistiller;
	class FLogger;

	class IApiContext : FNoncopyable
	{
	public:
		virtual ~IApiContext() = default;

		/**
		 * Loads and initializes the MDL context.
		 *
		 * @param LibrariesPath - path to the libraries required by the MDL SDK(eg. nv_freeimage).
		 * @param ModulesPath - main search path for MDL modules.
		 * @return true if the context was loaded.
		 */
		virtual bool Load(const FString& LibrariesPath, const FString& ModulesPath)
		{
			return false;
		}
		/**
		 * Unloads and de-initializes the MDL context.
		 *
		 * @param bClearDatabaseOnly - clear the MDL database(MDL modules, materials, textures, etc.)
		 */
		virtual void Unload(bool bClearDatabaseOnly) {};

		/**
		 * @param ModulesPath - search path for MDL modules.
		 */
		virtual void AddSearchPath(const FString& ModulesPath) {}
		virtual void RemoveSearchPath(const FString& ModulesPath) {}

		/**
		 * @param ResourcesPath - search path for resources(i.e. textures, light profiles, bsdf measurements) used by an MDL module.
		 */
		virtual void AddResourceSearchPath(const FString& ResourcesPath) {}
		virtual void RemoveResourceSearchPath(const FString& ResourcesPath) {}

		/**
		 * Loads materials from an MDL module to the given material collection.
		 *
		 * @param InModuleName - MDL module name.
		 * @param OutMaterials - collection of materials that are present in the MDL module.
		 * @return true if the MDL was loaded to the database.
		 *
		 * @note Only the material names and ids are populated, the material(s) must be distilled after.
		 */
		virtual bool LoadModule(const FString& InModuleName, FMaterialCollection& OutMaterials)
		{
			return false;
		}
		virtual bool UnloadModule(const FString& InModuleName)
		{
			return false;
		}

		/**
		 * Returns the material distiller used for material distillation to the Unreal target model.
		 */
		virtual FMaterialDistiller* GetDistiller()
		{
			return nullptr;
		}

		/**
		 * Returns any last error or warning messages.
		 */
		virtual TArray<MDLImporterLogging::FLogMessage> GetLogMessages() const
		{
			return TArray<MDLImporterLogging::FLogMessage>();
		}
	};
}  // namespace Mdl

#ifdef USE_MDLSDK

#include "mi/base/handle.h"

namespace mi
{
	namespace neuraylib
	{
		class INeuray;
		class IMdl_compiler;
		class IMdl_configuration;
		class IDatabase;
		class IMdl_factory;
	}
}

namespace Mdl
{
	namespace Detail
	{
		mi::neuraylib::IMdl_factory* GetFactory(const IApiContext& Context);
	}

	class FApiContext : public IApiContext
	{
	public:
		FApiContext();
		virtual ~FApiContext();

		virtual bool Load(const FString& LibrariesPath, const FString& ModulesPath) override;
		virtual void Unload(bool bClearDatabaseOnly) override;

		virtual void AddSearchPath(const FString& ModulesPath) override;
		virtual void RemoveSearchPath(const FString& ModulesPath) override;

		virtual void AddResourceSearchPath(const FString& ResourcesPath) override;
		virtual void RemoveResourceSearchPath(const FString& ResourcesPath) override;

		virtual bool LoadModule(const FString& InFilePath, FMaterialCollection& OutMaterials) override;
		virtual bool UnloadModule(const FString& FilePath) override;

		virtual FMaterialDistiller* GetDistiller() override;

		virtual TArray<MDLImporterLogging::FLogMessage> GetLogMessages() const override;

	private:
		void LogInfo();

	private:
		void*                                               DsoHandle;
		mi::base::Handle<mi::neuraylib::INeuray>            NeurayHandle;
		mi::base::Handle<mi::neuraylib::IMdl_configuration> ConfigHandle;
		mi::base::Handle<mi::neuraylib::IMdl_compiler>      CompilerHandle;
		mi::base::Handle<mi::neuraylib::IDatabase>          DatabaseHandle;
		mi::base::Handle<mi::neuraylib::IMdl_factory>       FactoryHandle;
		TUniquePtr<FMaterialDistiller>                      DistillerPtr;
		FLogger*                                            LoggerPtr;

		friend mi::neuraylib::IMdl_factory* Detail::GetFactory(const IApiContext& Context);
	};

	namespace Detail
	{
		inline mi::neuraylib::IMdl_factory* GetFactory(const IApiContext& Context)
		{
			return static_cast<const FApiContext&>(Context).FactoryHandle.get();
		}
	}

	inline FMaterialDistiller* FApiContext::GetDistiller()
	{
		return DistillerPtr.Get();
	}
}

#else  // #ifdef USE_MDLSDK

namespace Mdl
{
	class FApiContext : public IApiContext
	{
	};
}

#endif  // ifndef USE_MDLSDK
