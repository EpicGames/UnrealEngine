// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "mdl/Utility.h"


#include "mi/base/handle.h"
#include "mi/neuraylib/iexpression.h"
#include <mi/base/iinterface.h>
#include <mi/base/types.h>

namespace Mdl
{
	struct FArgumentGetter
	{
		const mi::neuraylib::ICompiled_material*                        Material;
		mi::base::Handle<const mi::neuraylib::IExpression_direct_call>& ParentCall;
		FString                                                         SubPath;

		FArgumentGetter(const mi::neuraylib::ICompiled_material* Material, mi::base::Handle<const mi::neuraylib::IExpression_direct_call>& ParentCall)
		    : Material(Material)
		    , ParentCall(ParentCall)
		{
		}

		const mi::neuraylib::IExpression* Get(const char* ArgumentName)
		{
			return Lookup::GetArgument(Material, ParentCall.get(), ArgumentName);
		}

		const mi::neuraylib::IExpression* Get(const char* InSubPath, const char* InArgumentName)
		{
			mi::base::Handle<const mi::neuraylib::IExpression_direct_call> TmpCall(Lookup::GetCall(InSubPath, Material, ParentCall.get()));
			return Lookup::GetArgument(Material, TmpCall.get(), InArgumentName);
		}

		const mi::neuraylib::IExpression* GetFromSubPath(const char* ArgumentName)
		{
			return Get(TCHAR_TO_ANSI(*SubPath), ArgumentName);
		}
	};
}
