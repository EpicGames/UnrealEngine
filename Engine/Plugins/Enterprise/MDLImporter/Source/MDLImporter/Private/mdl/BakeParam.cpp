// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef USE_MDLSDK

#include "BakeParam.h"

#include "MdlSdkDefines.h"
#include "Utility.h"
#include "common/TextureSource.h"

MDLSDK_INCLUDES_START
#include "mi/neuraylib/icanvas.h"
#include "mi/neuraylib/idata.h"
#include "mi/neuraylib/itile.h"
#include "mi/neuraylib/set_get.h"
#include "mi/neuraylib/typedefs.h"
#include <mi/base/handle.h>
#include <mi/base/iinterface.h>
#include <mi/base/types.h>
#include <mi/base/uuid.h>
#include <mi/math/color.h>
#include <mi/math/vector.h>
#include <mi/neuraylib/icolor.h>
#include <mi/neuraylib/ienum.h>
#include <mi/neuraylib/inumber.h>
MDLSDK_INCLUDES_END

namespace
{
	template <typename InputType, typename OutputType, typename SetFunc>
	bool SetValueFrom(const mi::IData& ValueData, OutputType& OutValue, SetFunc Func)
	{
		InputType InValue;
		if (mi::get_value(&ValueData, InValue) == 0)
		{
			const auto DefaultValue = OutValue;
			Func(InValue, OutValue);
			return OutValue != DefaultValue;
		}

		checkSlow(false);
		UE_LOGF(LogMDLImporter, Error, "Dynamic type doesn't have correct Value type : %s", ValueData.get_type_name());

		return false;
	}
}
namespace Mdl
{
	FBakeParam::FBakeParam()
		: Id(0)
		, RemapFunc(nullptr)
	    , InputExpression(nullptr)
	    , ExpressionData(nullptr)
	{
	}

	FBakeParam::FBakeParam(int Id, FMaterial::FExpressionEntry& ExpressionEntry, Common::FTextureProperty& TextureEntry, EValueType ValueType,
		FRemapFunc RemapFunc /*= nullptr*/)
		: Id(Id)
		, ValueType(ValueType)
		, RemapFunc(RemapFunc)
	    , InputExpression(nullptr)
	    , BakedTextureData(CreateTextureBakeData(TextureEntry.Path, TextureEntry.Source))
	    , ExpressionData(&ExpressionEntry)
	{
	}

	bool FBakeParam::SetExpression(UMaterialExpression* Expression, bool bIsTexture, int32 Index)
	{
		check(ExpressionData);
		if (!ExpressionData)
			return false;

		ExpressionData->Expression = Expression;
		ExpressionData->Index      = Index;
		ExpressionData->bIsTexture = bIsTexture;
		return true;
	}

	bool FBakeParam::SetBakedValue(mi::IData& ValueData)
	{
		if (!BakedData.IsValid())
		{
			return false;
		}

		if (RemapFunc)
		{
			RemapFunc(ValueData);
		}

		BakedData->SetValue(ValueData);
		return true;
	}

	bool FBakeParam::SetBakedTexture(const FString& Name, mi::neuraylib::ICanvas& CanvasData)
	{
		if (!BakedTextureData.IsValid())
		{
			return false;
		}

		if (RemapFunc)
		{
			RemapFunc(CanvasData);
		}

		mi::base::Handle<const mi::neuraylib::ITile> Tile(CanvasData.get_tile());
		const mi::Float32*                           Buffer = static_cast<const mi::Float32*>(Tile->get_data());

		BakedTextureData->SetName(Name);
		const int Channels = Mdl::Util::GetChannelCount(CanvasData);
		BakedTextureData->SetData(Buffer, Tile->get_resolution_x(), Tile->get_resolution_y(), Channels);
		return true;
	}

	FBakeTextureData::FBakeTextureData(FString& Name, FTextureSourcePtr& Source)
		: Name(Name)
		, Source(Source)
	{
		check(Source == nullptr);
	}

	void FBakeTextureData::SetName(const FString& InName)
	{
		Name = InName;
	}

	inline void FBakeTextureData::SetData(const float* InData, int InWidth, int InHeight, int InChannels)
	{
		check(Source == nullptr);
		Source = Common::CreateTextureSource(InData, InWidth, InHeight, InChannels, true);
	}

	namespace Detail
	{
		bool FBakedFloatValue::SetValue(const mi::IData& ValueData)
		{
			return SetValueFrom<mi::Float32, float>(ValueData, BakeDst, [](const mi::Float32& InValue, float& OutValue) {
				OutValue = InValue;
				checkSlow(InValue >= 0.f);
			});
		}

		bool FBakedVectorValue::SetValue(const mi::IData& ValueData)
		{
			return SetValueFrom<mi::Color, FVector3f>(ValueData, BakeDst, [](const mi::Color& InValue, FVector3f& OutValue) {
				OutValue.X = InValue.r;
				OutValue.Y = InValue.g;
				OutValue.Z = InValue.b;
			});
		}

		bool FBakedLinearColorValue::SetValue(const mi::IData& ValueData)
		{
			return SetValueFrom<mi::Color, FLinearColor>(ValueData, BakeDst, [](const mi::Color& InColor, FLinearColor& OutColor) {
				OutColor.R = InColor.r;
				OutColor.G = InColor.g;
				OutColor.B = InColor.b;
				OutColor.A = InColor.a;
			});
		}
	}
}

#endif  // #ifdef USE_MDLSDK
