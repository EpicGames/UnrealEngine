// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "mdl/Common.h"
#include "mdl/Material.h"


namespace Common { struct FTextureProperty; }

struct FImage;
namespace mi
{
	class IData;
	namespace base
	{
		class IInterface;
	}
	namespace neuraylib
	{
		class ICanvas;
		class IExpression;
	}
}

namespace Mdl
{
	class IBakeDestination
	{
	public:
		virtual ~IBakeDestination() = default;

		virtual bool SetValue(const mi::IData& ValueData) = 0;
	};  // class IBakeDestination

	class FBakeTextureData
	{
	public:
		typedef FImage* FTextureSourcePtr;

		FBakeTextureData(FString& Name, FTextureSourcePtr& Source);

		void SetName(const FString& Name);
		void SetData(const float* Data, int Width, int Height, int Channels);

	private:
		FString&           Name;
		FTextureSourcePtr& Source;
	};  // class FBakeTextureData

	struct FBakeParam
	{
		using FRemapFunc = TFunction<void(mi::base::IInterface&)>;

		int        Id;
		EValueType ValueType;
		FRemapFunc RemapFunc;
		// MDL material bake path for a texture or simple value
		FString InputBakePath;
		// MDL expression from the bake path
		const mi::neuraylib::IExpression* InputExpression;

		FBakeParam();

		FBakeParam(int                          Id,
		           FMaterial::FExpressionEntry& ExpressionEntry,
		           Common::FTextureProperty&    TextureEntry,
		           EValueType                   ValueType,
		           FRemapFunc                   RemapFunc = nullptr);

		template <typename T, int D>
		FBakeParam(int Id, FMaterial::TPropertyEntry<T, D>& Map, FRemapFunc RemapFunc = nullptr);

		template <typename T, int D>
		FBakeParam(int Id, FMaterial::TMapEntry<T, D>& Map, FRemapFunc RemapFunc = nullptr);

		//@note The remap function will be ignored.
		bool SetExpression(UMaterialExpression* Expression, bool bIsTexture, int32 Index);
		//@note Will remap the baked value via the remap function.
		bool SetBakedValue(mi::IData& ValueData);
		//@note Will remap the canvas via the remap function.
		bool SetBakedTexture(const FString& Name, mi::neuraylib::ICanvas& CanvasData);

		bool HasBakedData() const;
		bool HasBakedTextureData() const;

	private:
		FBakeTextureData* CreateTextureBakeData(FString& Path, FBakeTextureData::FTextureSourcePtr& Source)
		{
			if (Source)
			{
				UE_LOGF(LogMDLImporter, Log, "The texture was already baked: %ls", *Path);
				return nullptr;
			}
			return new FBakeTextureData(Path, Source);
		}

	private:
		TSharedPtr<IBakeDestination> BakedData;
		TSharedPtr<FBakeTextureData> BakedTextureData;
		FMaterial::FExpressionEntry* ExpressionData;
	};  // struct FBakeParam

	namespace Detail
	{
		class FBakedFloatValue : public IBakeDestination
		{
			float& BakeDst;

		public:
			FBakedFloatValue(float& Dst)  // destination to bake
				: BakeDst(Dst)
			{
			}
			virtual bool SetValue(const mi::IData& ValueData) override;
		};  // class FBakedFloatValue

		class FBakedVectorValue : public IBakeDestination
		{
			FVector3f& BakeDst;

		public:
			FBakedVectorValue(FVector3f& Dst)  // destination to bake
				: BakeDst(Dst)
			{
			}
			virtual bool SetValue(const mi::IData& ValueData) override;
		};  // class FBakedVectorValue

		class FBakedLinearColorValue : public IBakeDestination
		{
			FLinearColor& BakeDst;

		public:
			FBakedLinearColorValue(FLinearColor& Dst)  // destination to bake
				: BakeDst(Dst)
			{
			}
			virtual bool SetValue(const mi::IData& ValueData) override;
		};  // class FBakedLinearColorValue

		template <typename T>
		struct TGetValueTypeHelper;

		template <>
		struct TGetValueTypeHelper<float>
		{
			// Don't name these members 'ValueType' or the static analysis tool gets confused
			static const Mdl::EValueType TypeValue = Mdl::EValueType::Float;
			using BakeType                         = FBakedFloatValue;
		};

		template <>
		struct TGetValueTypeHelper<FLinearColor>
		{
			static const Mdl::EValueType TypeValue = Mdl::EValueType::ColorRGBA;
			using BakeType                         = FBakedLinearColorValue;
		};

		template <>
		struct TGetValueTypeHelper<FVector3f>
		{
			static const Mdl::EValueType TypeValue = Mdl::EValueType::ColorRGB;
			using BakeType                         = FBakedVectorValue;
		};
	}  // namespace Detail

	template <typename T, int D>
	FBakeParam::FBakeParam(int Id, FMaterial::TPropertyEntry<T, D>& PropertyEntry, FRemapFunc RemapFunc /* = nullptr*/)
		: Id(Id)
		, RemapFunc(RemapFunc)
	    , InputExpression(nullptr)
	    , ExpressionData(&PropertyEntry.ExpressionData)
	{
		typedef Detail::TGetValueTypeHelper<typename FMaterial::TMapEntry<T, D>::Type> GetValueHelper;
		typedef typename GetValueHelper::BakeType                                      BakeType;

		ValueType = GetValueHelper::TypeValue;
		BakedData = TSharedPtr<IBakeDestination>(new BakeType(PropertyEntry.Value));
	}

	template <typename T, int D>
	FBakeParam::FBakeParam(int Id, FMaterial::TMapEntry<T, D>& MapEntry, FRemapFunc RemapFunc /* = nullptr*/)
		: Id(Id)
		, RemapFunc(RemapFunc)
	    , InputExpression(nullptr)
	    , BakedTextureData(CreateTextureBakeData(MapEntry.Texture.Path, MapEntry.Texture.Source))
	    , ExpressionData(&MapEntry.ExpressionData)
	{
		typedef Detail::TGetValueTypeHelper<typename FMaterial::TMapEntry<T, D>::Type> GetValueHelper;
		typedef typename GetValueHelper::BakeType                                      BakeType;

		ValueType = GetValueHelper::TypeValue;
		BakedData = TSharedPtr<IBakeDestination>(new BakeType(MapEntry.Value));
	}

	inline bool FBakeParam::HasBakedData() const
	{
		return BakedData.IsValid();
	}

	inline bool FBakeParam::HasBakedTextureData() const
	{
		return BakedTextureData.IsValid();
	}
}
