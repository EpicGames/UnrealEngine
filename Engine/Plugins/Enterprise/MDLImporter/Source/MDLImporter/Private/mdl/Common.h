// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "common/Logging.h"


#if !defined(MDL_MSVC_EDITCONTINUE_WORKAROUND)
	#define MDL_MSVC_EDITCONTINUE_WORKAROUND 0
#endif

namespace Mdl
{
	enum class EValueType
	{
		Float = 0,
		// 2 x Float32
		Float2,
		// 3 x Float32
		Float3,
		// 3 x Float32 representing RGB color
		ColorRGB,
		// 4 x Float32 representing RGBA color
		ColorRGBA,
		Count
	};

	enum class EParameterType
	{
		BaseColor = 0,
		Metallic,
		Specular,
		Roughness,
		Opacity,
		Emission,
		Normal,
		Displacement,
		ClearcoatWeight,
		ClearcoatRoughness,
		ClearcoatNormal,
		IOR,
		VolumeAbsorption,
		VolumeScattering,
		Count
	};

	const TCHAR* ToString(EValueType Type);
	const TCHAR* ToString(EParameterType Type);
	uint32       ComponentCount(EValueType Type);

#define MDL_TOKENPASTE(x, y) x##y
#define MDL_TOKENPASTE2(x, y) MDL_TOKENPASTE(x, y)
#if MDL_MSVC_EDITCONTINUE_WORKAROUND
	// MSVC does not allow __LINE__ in template parameters when edit & continue is enabled (see C2975 @ MSDN)
	#define MDL_CHECK_RESULT() const Mdl::Detail::FCheckRes<0> MDL_TOKENPASTE2(res, __LINE__)
	#define MDL_CHECK_RESULT_MSG(msg) const Mdl::Detail::FCheckRes<0> MDL_TOKENPASTE2(res, __LINE__)
#else
	#define MDL_CHECK_RESULT() const Mdl::Detail::FCheckRes<__LINE__> MDL_TOKENPASTE2(res, __LINE__)
	#define MDL_CHECK_RESULT_MSG(msg) const Mdl::Detail::FCheckRes<__LINE__> MDL_TOKENPASTE2(res, __LINE__)
#endif

	namespace Detail
	{
		template <int LineNumber>
		struct FCheckRes
		{
			FCheckRes(int Result)
			{
				if (Result)
				{
					checkSlow(false);
					UE_LOGF(LogMDLImporter, Error, "Function returned: %d at line: %d", Result, LineNumber);
				}
			}
		};
	}
}
