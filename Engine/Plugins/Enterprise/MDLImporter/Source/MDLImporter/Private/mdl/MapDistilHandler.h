// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


class FString;

namespace mi
{
	namespace neuraylib
	{
		class ICompiled_material;
		class IMaterial_definition;
		class ITransaction;
	}
}
namespace Mdl
{
	struct FBakeParam;

	class IMapDistilHandler
	{
	public:
		virtual ~IMapDistilHandler() = default;

		virtual void PreImport(const mi::neuraylib::IMaterial_definition& MDLMaterialDefinition,
		                       const mi::neuraylib::ICompiled_material&   MDLMaterial,
		                       mi::neuraylib::ITransaction&               MDLTransaction) = 0;
		/**
		 * Import the map from the given bake path.
		 * @return True if the map was handled, otherwise will bake the map.
		 */
		virtual bool Import(const FString& MapName, bool bIsTexture, Mdl::FBakeParam& MapBakeParam) = 0;

		virtual void PostImport() = 0;
	};
}
