// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "common/TextureProperty.h"
#include "common/Utility.h"


class UMaterialExpression;
struct FImage;
namespace mi
{
	namespace neuraylib
	{
		class ITransaction;
		class INeuray;
	}
}

namespace Mdl
{
	struct FBakeParam;

	struct FMaterial
	{
		using FInstantiateFunc = TFunction<FString(mi::neuraylib::INeuray* Neuray, mi::neuraylib::ITransaction*)>;
		using FPreProcessFunc  = TFunction<void(mi::neuraylib::ITransaction* Transaction, TArray<FBakeParam>&)>;
		using FPostProcessFunc = TFunction<void(FMaterial&)>;

		struct FExpressionEntry
		{
			UMaterialExpression* Expression;
			int32                Index;
			bool                 bIsTexture;

			FExpressionEntry()
			    : Expression(nullptr)
			    , Index(0)
			    , bIsTexture(false)
			{
			}
		};

		template <typename ValueType, int DefaultValue = 255>
		struct TPropertyEntry
		{
			using Type = ValueType;

			// Property will be defined from a material expression.
			FExpressionEntry ExpressionData;
			// Property is a simple value.
			ValueType Value;

			TPropertyEntry()
				: Value(DefaultValue / 255.f)
			{
			}

			// Returns true if a value was baked or an expression created.
			bool WasProcessed() const;
			// Returns true if a map parameter was baked and the baked value is different from the default one.
			bool WasValueBaked() const;
			bool HasExpression() const;
		};
		template <typename ValueType, int DefaultValue = 255>
		struct TMapEntry : TPropertyEntry<ValueType, DefaultValue>
		{
			using Type = ValueType;

			Common::FTextureProperty Texture;

			// Returns true if a texture was baked or an expression created.
			bool WasProcessed() const;
			bool HasTexture() const;
		};
		struct FNormalMapEntry
		{
			// Texture will be defined from a material expression.
			FExpressionEntry ExpressionData;
			// Texture will be defined from a texture source or path.
			Common::FTextureProperty Texture;
			float                    Strength;

			FNormalMapEntry()
				: Strength(1.f)
			{
			}

			// Returns true if a texture was baked or an expression created.
			bool WasProcessed() const;
			bool HasExpression() const;
		};
		using FDisplacementMapEntry = FNormalMapEntry;

		struct FTextureArrayEntry
		{
			Common::FTextureProperty Texture;
			/// Depth or number of slices of the array of textures.
			int Depth;
			/// Texture size of a single entry.
			float Size;

			FTextureArrayEntry()
				: Depth(0)
				, Size(0)
			{
			}
		};
		struct FClearcoatEntry
		{
			TMapEntry<float> Weight;
			TMapEntry<float> Roughness;
			FNormalMapEntry  Normal;
		};
		struct FCarpaintEntry
		{
			struct FThetaSliceLUT
			{
				Common::FTextureProperty Texture;
				float                    TexelSize;

				FThetaSliceLUT()
					: TexelSize(0.f)
				{
				}
			};
			FTextureArrayEntry Flakes;
			FVector3f          FlakesColorValue;
			FThetaSliceLUT     ThetaFiLUT;
			int                NumThetaI;
			int                NumThetaF;
			int                MaxThetaI;
			bool               bEnabled;

			FCarpaintEntry()
				: NumThetaI(0)
				, NumThetaF(0)
				, MaxThetaI(0)
				, bEnabled(false)
			{
			}
		};

		uint32  Id;
		FString Name; // Name used to identify Material

		// Base name in the Mdl db(full name is something like ::<ModuleName>::<BaseName>)
		// note: separated from Name, because with resent Mdl SDK changes db name includes full signature(i.e. material-as-function argument list)
		FString BaseName; 

		FString GetBaseName()
		{
			return BaseName;
		}
		// The preferred baking texture size.
		int PreferredWidth;
		int PreferredHeight;

		TMapEntry<FVector3f>    BaseColor;
		TMapEntry<float, 0>   Metallic;
		TMapEntry<float>      Specular;
		TMapEntry<float>      Roughness;
		TMapEntry<float>      Opacity;
		TMapEntry<FVector3f, 0> Emission;
		TPropertyEntry<float> EmissionStrength;
		FNormalMapEntry       Normal;
		FDisplacementMapEntry Displacement;

		FClearcoatEntry Clearcoat;
		FCarpaintEntry  Carpaint;

		TPropertyEntry<FVector3f>    IOR;
		TPropertyEntry<FVector3f, 0> Absorption;
		TMapEntry<FVector3f, 0>      Scattering;

		float TilingFactor;
		FVector2D Tiling;

		FInstantiateFunc InstantiateFunction;
		FPreProcessFunc  PreProcessFunction;
		FPostProcessFunc PostProcessFunction;

		FMaterial();

		void ExecutePostProcess();
		void Disable();
		bool IsDisabled() const;

		void SetPreferredSize(int Width, int Height);

		bool operator==(const FMaterial& Other) const;
	};

	inline FMaterial::FMaterial()
	    : PreferredWidth(0)
	    , PreferredHeight(0)
		, TilingFactor(1.f)
	    , Tiling(1.f, 1.f)
	{
		using namespace Common;

		BaseColor.Texture.SetTextureMode(ETextureMode::Color);
		Metallic.Texture.SetTextureMode(ETextureMode::Grayscale);
		Specular.Texture.SetTextureMode(ETextureMode::Grayscale);
		Roughness.Texture.SetTextureMode(ETextureMode::Grayscale);
		Opacity.Texture.SetTextureMode(ETextureMode::Grayscale);
		Emission.Texture.SetTextureMode(ETextureMode::Color);
		Scattering.Texture.SetTextureMode(ETextureMode::Color);
		Normal.Texture.SetTextureMode(ETextureMode::Normal);
		Displacement.Texture.SetTextureMode(ETextureMode::Displace);

		Clearcoat.Roughness.Texture.SetTextureMode(ETextureMode::Grayscale);
		Clearcoat.Weight.Texture.SetTextureMode(ETextureMode::Grayscale);
		Clearcoat.Normal.Texture.SetTextureMode(ETextureMode::Normal);
	}

	inline bool FMaterial::operator==(const FMaterial& Other) const
	{
		return Id == Other.Id && Name == Other.Name;
	}

	inline void FMaterial::ExecutePostProcess()
	{
		if (PostProcessFunction)
		{
			PostProcessFunction(*this);
		}
	}

	inline void FMaterial::Disable()
	{
		Name.Empty();
	}

	inline bool FMaterial::IsDisabled() const
	{
		return Name.IsEmpty();
	}

	inline void FMaterial::SetPreferredSize(int Width, int Height)
	{
		// force highest power of two
		PreferredWidth  = Common::HighestPowerofTwo(Width);
		PreferredHeight = Common::HighestPowerofTwo(Height);
	}

	//

	template <typename ValueType, int DefaultValue>
	bool FMaterial::TPropertyEntry<ValueType, DefaultValue>::WasValueBaked() const
	{
		TPropertyEntry Default;
		return Value != Default.Value;
	}

	template <typename ValueType, int DefaultValue>
	bool FMaterial::TPropertyEntry<ValueType, DefaultValue>::WasProcessed() const
	{
		return WasValueBaked() || ExpressionData.Expression != nullptr;
	}

	template <typename ValueType, int DefaultValue>
	bool FMaterial::TPropertyEntry<ValueType, DefaultValue>::HasExpression() const
	{
		return ExpressionData.Expression != nullptr;
	}

	template <typename ValueType, int DefaultValue>
	bool FMaterial::TMapEntry<ValueType, DefaultValue>::WasProcessed() const
	{
		return !Texture.Path.IsEmpty() || this->ExpressionData.Expression != nullptr;
	}

	template <typename ValueType, int DefaultValue>
	bool FMaterial::TMapEntry<ValueType, DefaultValue>::HasTexture() const
	{
		return !Texture.Path.IsEmpty() || (this->ExpressionData.Expression && this->ExpressionData.bIsTexture);
	}

	inline bool FMaterial::FNormalMapEntry::WasProcessed() const
	{
		return !Texture.Path.IsEmpty() || ExpressionData.Expression != nullptr;
	}

	inline bool FMaterial::FNormalMapEntry::HasExpression() const
	{
		return ExpressionData.Expression != nullptr;
	}
}
