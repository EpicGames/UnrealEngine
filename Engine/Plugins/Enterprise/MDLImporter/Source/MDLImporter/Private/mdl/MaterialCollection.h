// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "mdl/Material.h"

namespace Mdl
{
	class FMaterialCollection
	{
	public:
		int32 Count() const;

		FMaterial& Create();
		void       Reserve(int32 Count);
		void       Remove(const FMaterial& Material);
		void       RemoveAt(int32 Index);

		FMaterial&       operator[](int32 Index);
		const FMaterial& operator[](int32 Index) const;

		template <typename Predicate>
		const FMaterial* FindByPredicate(Predicate Pred) const;

	public:
		FString Name;

	private:
		TArray<FMaterial> Data;

	public:
		// forward TArray ranges
		FORCEINLINE TArray<FMaterial>::RangedForIteratorType begin()
		{
			return Data.begin();
		}
		FORCEINLINE TArray<FMaterial>::RangedForConstIteratorType begin() const
		{
			return Data.begin();
		}
		FORCEINLINE TArray<FMaterial>::RangedForIteratorType end()
		{
			return Data.end();
		}
		FORCEINLINE TArray<FMaterial>::RangedForConstIteratorType end() const
		{
			return Data.end();
		}
	};

	inline int32 FMaterialCollection::Count() const
	{
		return Data.Num();
	}

	inline void FMaterialCollection::Reserve(int32 Count)
	{
		Data.Reserve(Count);
	}

	inline FMaterial& FMaterialCollection::Create()
	{
		return Data.Emplace_GetRef();
	}

	inline void FMaterialCollection::Remove(const FMaterial& Material)
	{
		Data.Remove(Material);
	}

	inline void FMaterialCollection::RemoveAt(int32 Index)
	{
		Data.RemoveAt(Index);
	}

	inline FMaterial& FMaterialCollection::operator[](int32 Index)
	{
		return Data[Index];
	}

	inline const FMaterial& FMaterialCollection::operator[](int32 Index) const
	{
		return Data[Index];
	}

	template <typename Predicate>
	const FMaterial* FMaterialCollection::FindByPredicate(Predicate Pred) const
	{
		return Data.FindByPredicate(Pred);
	}
}
