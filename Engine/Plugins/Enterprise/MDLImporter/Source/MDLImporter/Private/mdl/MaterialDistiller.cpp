// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef USE_MDLSDK

#include "MaterialDistiller.h"

#define MDL_DEBUG_PRINT_MATERIAL 3

#include "ArgumentGetter.h"
#include "BakeParam.h"
#include "Common.h"
#include "MaterialCollection.h"
#include "SemanticParser.h"
#include "Utility.h"
#include <mi/base/uuid.h>
#include <mi/math/color.h>
#include <mi/math/vector.h>
#include <mi/neuraylib/ienum.h>
#include <mi/neuraylib/type_traits.h>
#include <mi/neuraylib/vector_typedefs.h>
#if MDL_DEBUG_PRINT_MATERIAL != 0
#include "MaterialPrinter.h"
#endif
#include "MdlSdkDefines.h"

#include "Misc/Paths.h"

MDLSDK_INCLUDES_START
#include "mdl/MapDistilHandler.h"
#include "mi/neuraylib/icanvas.h"
#include "mi/neuraylib/icolor.h"
#include "mi/neuraylib/icompiled_material.h"
#include "mi/neuraylib/idata.h"
#include "mi/neuraylib/idatabase.h"
#include "mi/neuraylib/iimage_api.h"
#include "mi/neuraylib/imaterial_definition.h"
#include "mi/neuraylib/imdl_distiller_api.h"
#include "mi/neuraylib/ineuray.h"
#include "mi/neuraylib/inumber.h"
#include "mi/neuraylib/iscope.h"
#include "mi/neuraylib/itile.h"
#include "mi/neuraylib/itransaction.h"
#include "mi/neuraylib/ivector.h"
#include "mi/neuraylib/set_get.h"
#include "mi/neuraylib/typedefs.h"
MDLSDK_INCLUDES_END

namespace Mdl
{
	namespace
	{
		// Creates a distilled material from a compiled material instance
		const mi::neuraylib::ICompiled_material* CreateDistilledMaterial(mi::neuraylib::ITransaction*       Transaction,
		                                                                        mi::neuraylib::IMdl_distiller_api* Distiller,
		                                                                        const FString&                     MaterialDbName,
		                                                                        const char*                        TargetModel)
		{
			mi::base::Handle<const mi::neuraylib::ICompiled_material> CompiledMaterial(
			    Transaction->access<mi::neuraylib::ICompiled_material>(TCHAR_TO_ANSI(*MaterialDbName)));

			if (!CompiledMaterial)
			{
				return nullptr;
			}

			mi::Sint32                                                Result = 0;
			mi::base::Handle<const mi::neuraylib::ICompiled_material> DistilledMaterial(
			    Distiller->distill_material(CompiledMaterial.get(), TargetModel, nullptr, &Result));
			MDL_CHECK_RESULT() = Result;

			if (DistilledMaterial)
			{ 
				DistilledMaterial->retain();
			}
			
			return DistilledMaterial.get();
		}

		void RemapNormalTexture(mi::base::IInterface& CanvasInterface)
		{
			mi::base::Handle<mi::neuraylib::ICanvas> Canvas(CanvasInterface.get_interface<mi::neuraylib::ICanvas>());
			if (!Canvas)
			{
				return;
			}

			mi::base::Handle<mi::neuraylib::ITile> Tile(Canvas->get_tile());
			mi::Float32*                           Buffer = static_cast<mi::Float32*>(Tile->get_data());

			const mi::Uint32 ChannelCount = 3;
			const mi::Uint32 Count        = Canvas->get_resolution_x() * Canvas->get_resolution_y() * ChannelCount;
			Common::RemapNormalFlipGreen(Count, Buffer);
		}

		void RemapRoughness(mi::base::IInterface& DataInterface)
		{
			mi::Float32 Value;
			if (mi::get_value(static_cast<mi::IData*>(&DataInterface), Value) == 0)
			{
				Value = FMath::Sqrt(Value);
				static_cast<mi::IFloat32&>(DataInterface).set_value(Value);
				return;
			}

			mi::base::Handle<mi::neuraylib::ICanvas> Canvas(DataInterface.get_interface<mi::neuraylib::ICanvas>());
			if (!Canvas)
			{
				return;
			}

			mi::base::Handle<mi::neuraylib::ITile> Tile(Canvas->get_tile());
			mi::Float32*                           Buffer = static_cast<mi::Float32*>(Tile->get_data());

			const mi::Uint32 ChannelCount = 1;
			const mi::Uint32 Count        = Canvas->get_resolution_x() * Canvas->get_resolution_y() * ChannelCount;
			for (mi::Uint32 Index = 0; Index < Count; ++Index)
			{
				Buffer[Index] = FMath::Sqrt(Buffer[Index]);
			}
		}

		inline FBakeParam& GetParam(int Id, TArray<FBakeParam>& MaterialBakeParam)
		{
			FBakeParam* Found = MaterialBakeParam.FindByPredicate([Id](const FBakeParam& Param) { return Param.Id == Id; });
			return *Found;
		}

		inline FBakeParam& GetParam(EParameterType Type, TArray<FBakeParam>& MaterialBakeParam)
		{
			return GetParam((int)Type, MaterialBakeParam);
		}

		// Handles the material semantic for Unreal target models and sets the bake paths for the maps
		void HandleMaterialSemanticUnreal(mi::neuraylib::ITransaction*             Transaction,
		                               const mi::neuraylib::ICompiled_material* CompiledMaterial,
		                               TArray<FBakeParam>&                      BakeParams)
		{
			// Access surface.scattering function
			Mdl::FSemanticParser Parser("surface.scattering", Transaction, CompiledMaterial);
			Mdl::FArgumentGetter ArgumentGetter(CompiledMaterial, Parser.ParentCall);

			// Check for a clearcoat layer, first. If present, it is the outermost layer
			if (Parser.Semantic == mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_CUSTOM_CURVE_LAYER)
			{
				// Setup clearcoat bake paths
				GetParam(EParameterType::ClearcoatWeight, BakeParams).InputBakePath      = Parser.PathPrefix + TEXT("weight");
				GetParam(EParameterType::ClearcoatWeight, BakeParams).InputExpression    = ArgumentGetter.Get("weight");
				GetParam(EParameterType::ClearcoatRoughness, BakeParams).InputBakePath   = Parser.PathPrefix + TEXT("layer.roughness_u");
				GetParam(EParameterType::ClearcoatRoughness, BakeParams).InputExpression = ArgumentGetter.Get("layer", "roughness_u");
				GetParam(EParameterType::ClearcoatNormal, BakeParams).InputBakePath      = Parser.PathPrefix + TEXT("normal");
				GetParam(EParameterType::ClearcoatNormal, BakeParams).InputExpression    = ArgumentGetter.Get("normal");
				// Get clear-coat base layer
				Parser.SetNextCall("base");
			}
			// Check for a weighted layer. Sole purpose of this layer is the transportation of
			// the under-clearcoat-normal. It contains an empty base and a layer with the actual material body
			if (Parser.Semantic == mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_WEIGHTED_LAYER)
			{
				// Collect under-clearcoat normal
				GetParam(EParameterType::Normal, BakeParams).InputBakePath   = Parser.PathPrefix + TEXT("normal");
				GetParam(EParameterType::Normal, BakeParams).InputExpression = ArgumentGetter.Get("normal");
				// Chain further
				Parser.SetNextCall("layer");
			}
			// Check for a normalized mix. This mix combines the metallic and dielectric parts of the material
			if (Parser.Semantic == mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_NORMALIZED_MIX)
			{
				// The top-mix component is supposed to be a glossy bsdf
				ArgumentGetter.SubPath = TEXT("components.1");
				// Collect metallic weight
				GetParam(EParameterType::Metallic, BakeParams).InputBakePath   = Parser.PathPrefix + TEXT("components.1.weight");
				GetParam(EParameterType::Metallic, BakeParams).InputExpression = ArgumentGetter.GetFromSubPath("weight");

				ArgumentGetter.SubPath = TEXT("components.1.component");
				// And other metallic parameters
				GetParam(EParameterType::Roughness, BakeParams).InputBakePath   = Parser.PathPrefix + TEXT("components.1.component.roughness_u");
				GetParam(EParameterType::Roughness, BakeParams).InputExpression = ArgumentGetter.GetFromSubPath("roughness_u");
				// Base_color can be taken from any of the leaf-bsdfs. It is supposed to be the same.
				GetParam(EParameterType::BaseColor, BakeParams).InputBakePath   = Parser.PathPrefix + TEXT("components.1.component.tint");
				GetParam(EParameterType::BaseColor, BakeParams).InputExpression = ArgumentGetter.GetFromSubPath("tint");
				// Chain further
				Parser.SetNextCall("components.0.component");
			}
			if (Parser.Semantic == mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_CUSTOM_CURVE_LAYER)
			{
				// Collect specular parameters
				GetParam(EParameterType::Specular, BakeParams).InputBakePath   = Parser.PathPrefix + TEXT("weight");
				GetParam(EParameterType::Specular, BakeParams).InputExpression = ArgumentGetter.Get("weight");
				if (GetParam(EParameterType::Roughness, BakeParams).InputBakePath.IsEmpty())
				{
					GetParam(EParameterType::Roughness, BakeParams).InputBakePath   = Parser.PathPrefix + TEXT("layer.roughness_u");
					GetParam(EParameterType::Roughness, BakeParams).InputExpression = ArgumentGetter.Get("layer", "roughness_u");
				}
				// Chain further
				Parser.SetNextCall("base");
			}

			if (Parser.Semantic == mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_MICROFACET_GGX_VCAVITIES_BSDF)
			{
				if (GetParam(EParameterType::Metallic, BakeParams).InputBakePath.IsEmpty())
				{
					mi::base::Handle<mi::IFloat32> Value(Transaction->create<mi::IFloat32>());
					Value->set_value(1.f);
					GetParam(EParameterType::Metallic, BakeParams).SetBakedValue(*Value);
				}
				if (GetParam(EParameterType::Roughness, BakeParams).InputBakePath.IsEmpty())
				{
					GetParam(EParameterType::Roughness, BakeParams).InputBakePath   = Parser.PathPrefix + TEXT("roughness_u");
					GetParam(EParameterType::Roughness, BakeParams).InputExpression = ArgumentGetter.Get("roughness_u");
				}
				if (GetParam(EParameterType::BaseColor, BakeParams).InputBakePath.IsEmpty())
				{
					GetParam(EParameterType::BaseColor, BakeParams).InputBakePath   = Parser.PathPrefix + TEXT("tint");
					GetParam(EParameterType::BaseColor, BakeParams).InputExpression = ArgumentGetter.Get("tint");
				}
			}
			else if (Parser.Semantic == mi::neuraylib::IFunction_definition::DS_INTRINSIC_DF_DIFFUSE_REFLECTION_BSDF)
			{
				if (GetParam(EParameterType::BaseColor, BakeParams).InputBakePath.IsEmpty())
				{
					GetParam(EParameterType::BaseColor, BakeParams).InputBakePath   = Parser.PathPrefix + "tint";
					GetParam(EParameterType::BaseColor, BakeParams).InputExpression = ArgumentGetter.Get("tint");
				}
			}
		}

		void SetupBaseMaterial(mi::neuraylib::ITransaction*             Transaction,
		                       const mi::neuraylib::ICompiled_material* DistilledMaterial,
		                       Mdl::FMaterial&                          Material,
		                       TArray<FBakeParam>&                      MaterialBakeParams,
		                       float MetersPerSceneUnit)
		{
			TArray<FBakeParam>& BakeParams = MaterialBakeParams;
			// Setup some Unreal material parameters
			BakeParams.Emplace((int)EParameterType::BaseColor, Material.BaseColor);
			BakeParams.Emplace((int)EParameterType::Metallic, Material.Metallic);
			BakeParams.Emplace((int)EParameterType::Roughness, Material.Roughness, RemapRoughness);
			BakeParams.Emplace((int)EParameterType::Specular, Material.Specular);
			BakeParams.Emplace((int)EParameterType::Opacity, Material.Opacity);
			BakeParams.Emplace((int)EParameterType::Normal, Material.Normal.ExpressionData, Material.Normal.Texture,  //
			                   Mdl::EValueType::Float3, RemapNormalTexture);

			BakeParams.Emplace((int)EParameterType::Emission, Material.Emission, [&Material](mi::base::IInterface& ColorInterface) {
				mi::Color Color;
				if (mi::get_value(static_cast<mi::IData*>(&ColorInterface), Color) == 0)
				{
					Material.EmissionStrength.Value = FMath::Max(FMath::Max(Color.r, Color.g), Color.b);
					if (Material.EmissionStrength.Value)
					{
						Color.r /= Material.EmissionStrength.Value;
						Color.g /= Material.EmissionStrength.Value;
						Color.b /= Material.EmissionStrength.Value;
						static_cast<mi::IColor3&>(ColorInterface).set_value(Color);
					}
				}
			});

			BakeParams.Emplace((int)EParameterType::Displacement, Material.Displacement.ExpressionData, Material.Displacement.Texture,
			                   Mdl::EValueType::Float3,
			                   [&Material, MetersPerSceneUnit](mi::base::IInterface& ColorInterface)  //
			                   {
				                   mi::base::Handle<mi::neuraylib::ICanvas> Canvas(ColorInterface.get_interface<mi::neuraylib::ICanvas>());
				                   mi::base::Handle<mi::neuraylib::ITile>   Tile(Canvas->get_tile());
								   const char* PixelType = Tile->get_type();
				                   mi::Float32* Data = static_cast<mi::Float32*>(Tile->get_data());

				                   const mi::Uint32 ChannelCount = 3;
				                   const mi::Uint32 Count        = Canvas->get_resolution_x() * Canvas->get_resolution_y() * ChannelCount;

				                   float MaxValue = 0.f;
				                   for (mi::Uint32 Index = 0; Index < Count; Index += ChannelCount)
				                   {
					                   MaxValue = FMath::Max(Data[Index], MaxValue);
					                   MaxValue = FMath::Max(Data[Index + 1], MaxValue);
					                   MaxValue = FMath::Max(Data[Index + 2], MaxValue);
				                   }
				                   Material.Displacement.Strength = MaxValue / MetersPerSceneUnit; // meters to unreal units
				                   for (mi::Uint32 Index = 0; Index < Count; Index += ChannelCount)
				                   {
					                   Data[Index]     = Data[Index] / MaxValue;
					                   Data[Index + 1] = Data[Index + 1] / MaxValue;
					                   Data[Index + 2] = Data[Index + 2] / MaxValue;
				                   }
			                   });

			BakeParams.Emplace((int)EParameterType::ClearcoatWeight, Material.Clearcoat.Weight);
			BakeParams.Emplace((int)EParameterType::ClearcoatRoughness, Material.Clearcoat.Roughness, RemapRoughness);
			BakeParams.Emplace((int)EParameterType::ClearcoatNormal, Material.Clearcoat.Normal.ExpressionData, Material.Clearcoat.Normal.Texture,
			                   Mdl::EValueType::Float3, RemapNormalTexture);

			BakeParams.Emplace((int)EParameterType::IOR, Material.IOR);

			HandleMaterialSemanticUnreal(Transaction, DistilledMaterial, MaterialBakeParams);

			// Check for cutout-opacity

			auto& OpacityParam = GetParam(EParameterType::Opacity, BakeParams);

			OpacityParam.InputBakePath   = Lookup::GetValidSubExpression("geometry.cutout_opacity", DistilledMaterial);
			OpacityParam.InputExpression = DistilledMaterial->lookup_sub_expression("geometry.cutout_opacity");

			// Check for displacement
			GetParam(EParameterType::Displacement, BakeParams).InputBakePath =
			    Lookup::GetValidSubExpression("geometry.displacement", DistilledMaterial);

			// Check for IOR
			GetParam(EParameterType::IOR, BakeParams).InputBakePath = Lookup::GetValidSubExpression("ior", DistilledMaterial);

			// Check for emission color
			GetParam(EParameterType::Emission, BakeParams).InputBakePath =
			    Lookup::GetValidSubExpression("surface.emission.intensity", DistilledMaterial);
			GetParam(EParameterType::Emission, BakeParams).InputExpression = DistilledMaterial->lookup_sub_expression("surface.emission.intensity");
		}

		void SetupExtraMaterialProperties(mi::neuraylib::ITransaction*             Transaction,
		                                  const mi::neuraylib::ICompiled_material* CompiledMaterial,
		                                  Mdl::FMaterial&                          Material,
		                                  TArray<FBakeParam>&                      MaterialBakeParams)
		{
			TArray<FBakeParam>& BakeParams = MaterialBakeParams;

			BakeParams.Emplace((int)EParameterType::VolumeAbsorption, Material.Absorption);
			BakeParams.Emplace((int)EParameterType::VolumeScattering, Material.Scattering);

			GetParam(EParameterType::VolumeAbsorption, BakeParams).InputBakePath = TEXT("volume.absorption_coefficient");
			GetParam(EParameterType::VolumeScattering, BakeParams).InputBakePath = TEXT("volume.scattering_coefficient");
		}

		void PrintDebug(const mi::neuraylib::ICompiled_material* CompiledMaterial, const mi::neuraylib::ICompiled_material* DistilledMaterial,
		                mi::neuraylib::ITransaction* Transaction)
		{
#if MDL_DEBUG_PRINT_MATERIAL != 0
			Mdl::FMaterialPrinter Printer;
#if MDL_DEBUG_PRINT_MATERIAL & 1
			UE_LOGF(LogMDLImporter, Log, "Compiled:\n%ls", *Printer.Print(*CompiledMaterial, Transaction));
#endif
#if MDL_DEBUG_PRINT_MATERIAL & 2
			UE_LOGF(LogMDLImporter, Log, "Distilled:\n%ls", *Printer.Print(*DistilledMaterial, Transaction));
#endif
#endif
		}

		int GetMapPriority(EParameterType MapType)
		{
			int Priority = 10;
			switch (MapType)
			{
					// normal maps have highest priority as other maps may depend on them
				case EParameterType::Normal:
					Priority = 0;
				case EParameterType::ClearcoatNormal:
					Priority = 1;
				default:
					break;
			}
			return Priority;
		}
	}

	FMaterialDistiller::FMaterialDistiller(mi::base::Handle<mi::neuraylib::INeuray> Handle)
	    : Neuray(Handle)
	    , Handle(Handle->get_api_component<mi::neuraylib::IMdl_distiller_api>())
	    , ImageApi(Neuray->get_api_component<mi::neuraylib::IImage_api>())
	    , MapHandler(nullptr)
	{
		BakeResolution     = 1024;
		BakeSamples        = 2;
		MetersPerSceneUnit = 1.f;

		for (FIntPoint& Size : BakeCanvasesSize)
			Size.X = Size.Y = 0;
	}

	FMaterialDistiller::~FMaterialDistiller() {}

	bool FMaterialDistiller::Distil(FMaterialCollection& Materials, FProgressFunc ProgressFunc) const
	{
		mi::base::Handle<mi::neuraylib::IDatabase> Database(Neuray->get_api_component<mi::neuraylib::IDatabase>());

		int MaterialIndex = 0;
		for (FMaterial& Material : Materials)
		{
			if (Material.IsDisabled())
			{
				continue;
			}

			mi::base::Handle<mi::neuraylib::IScope>       Scope(Database->get_global_scope());
			mi::base::Handle<mi::neuraylib::ITransaction> Transaction(Scope->create_transaction());

			// Compile the material before distillation
			FString       MaterialDbName         = Mdl::Util::GetMaterialDatabaseName(Materials.Name, Material.BaseName);
			const FString MaterialInstanceDbName = Mdl::Util::GetMaterialInstanceName(MaterialDbName);
			const FString MaterialCompiledDbName = MaterialDbName + TEXT("_compiled");
			if (!Material.InstantiateFunction)
			{
				Mdl::Util::CreateMaterialInstance(Transaction.get(), MaterialDbName, MaterialInstanceDbName);
			}
			else
			{
				MaterialDbName = Material.InstantiateFunction(Neuray.get(), Transaction.get());
			}

			// use class compilation, otherwise annotations and parameters will be stripped out
			const bool bClassCompilation = true;
			Mdl::Util::CompileMaterialInstance(Transaction.get(), Neuray.get(), MaterialInstanceDbName, MaterialCompiledDbName, true, MetersPerSceneUnit);

			const FString MaterialName = Materials.Count() == 1 ? Material.Name : Materials.Name + TEXT("_") + Material.Name;
			Distil(Transaction.get(), MaterialName, MaterialDbName, MaterialCompiledDbName, Material);

			MDL_CHECK_RESULT() = Transaction->remove(TCHAR_TO_ANSI(*MaterialInstanceDbName));
			MDL_CHECK_RESULT() = Transaction->remove(TCHAR_TO_ANSI(*MaterialCompiledDbName));

			Transaction->commit();

			Material.ExecutePostProcess();

			if (ProgressFunc)
			{
				ProgressFunc(Material.Name, MaterialIndex++);
			}
		}

		return true;
	}

	// Constructs a material for the target model, extracts the bake paths relevant for this
	// model from the compiled material
	void FMaterialDistiller::Distil(mi::neuraylib::ITransaction* Transaction,
	                                const FString&               MaterialName,
	                                const FString&               MaterialDbName,
	                                const FString&               MaterialCompiledDbName,
	                                FMaterial&                   Material) const
	{
		// Order MDL SDK to distill material to Unreal target(GGX with ClearCoat)
		// Note: the string constant initialized this weird way to evade being caught by the reference check tool that searches for <this word> that must not be used in Unreal source code anymore
		static const char* TargetModel = "u" "e" "4"; 

		TArray<FBakeParam> MaterialBakeParams;
		MaterialBakeParams.Reserve((int)EParameterType::Count);

		mi::base::Handle<const mi::neuraylib::ICompiled_material> CompiledMaterial(
			Transaction->access<mi::neuraylib::ICompiled_material>(TCHAR_TO_ANSI(*MaterialCompiledDbName)));

		mi::base::Handle<const mi::neuraylib::IMaterial_definition> MaterialDefinition(
		    Transaction->access<mi::neuraylib::IMaterial_definition>(TCHAR_TO_ANSI(*MaterialDbName)));
		// Distill the compiled material to the target material
		const mi::neuraylib::ICompiled_material* DistilledMaterial =
		    CreateDistilledMaterial(Transaction, Handle.get(), MaterialCompiledDbName, TargetModel);
		if (!DistilledMaterial)
			return;

		PrintDebug(CompiledMaterial.get(), DistilledMaterial, Transaction);

		// Setup material maps and bake them after
		MaterialBakeParams.Empty();
		SetupBaseMaterial(Transaction, DistilledMaterial, Material, MaterialBakeParams, MetersPerSceneUnit);
		SetupExtraMaterialProperties(Transaction, DistilledMaterial, Material, MaterialBakeParams);
		if (Material.PreProcessFunction)
		{
			Material.PreProcessFunction(Transaction, MaterialBakeParams);
		}

		// Bake the base/distilled properties
		FIntPoint BakeTextureSize;
		BakeTextureSize.X = FMath::Max<uint32>(BakeResolution, Material.PreferredWidth);
		BakeTextureSize.Y = FMath::Max<uint32>(BakeResolution, Material.PreferredHeight);

		if (MapHandler)
		{
			MapHandler->PreImport(*MaterialDefinition, *DistilledMaterial, *Transaction);
		}
		DistilMaps(Transaction, MaterialName, DistilledMaterial, BakeTextureSize, MaterialBakeParams);
		if (MapHandler)
		{
			MapHandler->PostImport();
		}

		DistilledMaterial->release();
	}

	void FMaterialDistiller::BakeConstantValue(mi::neuraylib::ITransaction* Transaction,
	                                           const mi::neuraylib::IBaker* Baker,
	                                           int                          MapTypeInt,
	                                           FBakeParam&                  MapBakeParam) const
	{
		const EParameterType MapType = static_cast<EParameterType>(MapTypeInt);
		if (!MapBakeParam.HasBakedData())
		{
			UE_LOGF(LogMDLImporter, Log, "Bake value was null for: %ls", Mdl::ToString(MapType));
			return;
		}

		// Create baking destination
		mi::base::Handle<mi::IData> ValueData;
		switch (MapBakeParam.ValueType)
		{
			case Mdl::EValueType::Float:
			{
				mi::base::Handle<mi::IFloat32> Value(Transaction->create<mi::IFloat32>());
				ValueData = Value->get_interface<mi::IData>();
				break;
			}
			case Mdl::EValueType::Float3:
			{
				mi::base::Handle<mi::IFloat32_3> Value(Transaction->create<mi::IFloat32_3>());
				ValueData = Value->get_interface<mi::IData>();
				break;
			}
			case Mdl::EValueType::ColorRGB:
			{
				mi::base::Handle<mi::IColor> Value(Transaction->create<mi::IColor>());
				ValueData = Value->get_interface<mi::IData>();
				break;
			}
			default:
			{
				UE_LOGF(LogMDLImporter, Log, "Ignoring unsupported value type : %d", (int)MapBakeParam.ValueType);
				return;
			}
		}

		// Bake the constant parameter and set the value
		MDL_CHECK_RESULT() = Baker->bake_constant(ValueData.get(), BakeSamples);
		if (MapBakeParam.SetBakedValue(*ValueData.get()))
		{
			// Log baking info
			switch (MapBakeParam.ValueType)
			{
				case Mdl::EValueType::Float:
				{
					float Value;
					make_handle(ValueData->get_interface<mi::IFloat32>())->get_value(Value);
					UE_LOGF(LogMDLImporter, Log, "Baked value: %f for: %ls from: %ls", Value, Mdl::ToString(MapType), *MapBakeParam.InputBakePath);
					break;
				}
				case Mdl::EValueType::Float3:
				{
					mi::Float32_3 Value;
					make_handle(ValueData->get_interface<mi::IFloat32_3>())->get_value(Value);
					UE_LOGF(LogMDLImporter, Log, "Baked float3: %f %f %f for: %ls from: %ls", Value.x, Value.y, Value.z, Mdl::ToString(MapType),
					       *MapBakeParam.InputBakePath);
					break;
				}
				case Mdl::EValueType::ColorRGB:
				{
					mi::Color Value;
					make_handle(ValueData->get_interface<mi::IColor>())->get_value(Value);
					UE_LOGF(LogMDLImporter, Log, "Baked color: %f %f %f %f for: %ls from: %ls", Value.r, Value.g, Value.b, Value.a,
					       Mdl::ToString(MapType), *MapBakeParam.InputBakePath);
					break;
				}
				default:
					break;
			}
		}
	}

	bool FMaterialDistiller::IsEmptyTexture(const mi::neuraylib::IBaker* Baker, EValueType ValueType) const
	{
		const uint32 Size         = 64;
		const uint32 ChannelCount = ComponentCount(ValueType);
		check(ChannelCount <= 4);
		mi::base::Handle<mi::neuraylib::ICanvas>& Canvas = PreBakeCanvases[(int)ValueType];
		if (Canvas == nullptr)
		{
			Canvas = ImageApi->create_canvas(TCHAR_TO_ANSI(Mdl::ToString(ValueType)), Size, Size);
		}
		MDL_CHECK_RESULT() = Baker->bake_texture(Canvas.get(), 1);

		mi::base::Handle<const mi::neuraylib::ITile> Tile(Canvas->get_tile());
		const mi::Float32*                           Buffer = static_cast<const mi::Float32*>(Tile->get_data());

		check(4 >= ChannelCount);
		FVector4 CurrentValue(0);
		FVector4 NextValue(0);
		// check if all values are the same
		for (uint32 Index = 0; Index < Size * Size - ChannelCount; Index += ChannelCount)
		{
			const mi::Float32* BufferCurrent = Buffer + Index;
			for (uint32 ChannelIndex = 0; ChannelIndex < ChannelCount; ++ChannelIndex)
			{
				CurrentValue[ChannelIndex] = BufferCurrent[ChannelIndex];
			}
			BufferCurrent = Buffer + Index + ChannelCount;
			for (uint32 ChannelIndex = 0; ChannelIndex < ChannelCount; ++ChannelIndex)
			{
				NextValue[ChannelIndex] = BufferCurrent[ChannelIndex];
			}
			if (!CurrentValue.Equals(NextValue))
			{
				return false;
			}
		}

		return true;
	}

	void FMaterialDistiller::BakeTexture(const FString& MaterialName, const mi::neuraylib::IBaker* Baker, int MapTypeInt, FIntPoint BakeTextureSize,
	                                     FBakeParam& MapBakeParam) const
	{
		if (!MapBakeParam.HasBakedTextureData())
		{
			UE_LOGF(LogMDLImporter, Log, "Bake texture path is null : %d", MapTypeInt);
			return;
		}

		// Pre-bake check, there's no easy way to find out if a texture is defined or not, e.g. there may be expressions with empty texture parameters
		if (IsEmptyTexture(Baker, MapBakeParam.ValueType))
		{
			UE_LOGF(LogMDLImporter, Log, "Bake texture was empty: %d", MapTypeInt);
			return;
		}

		// Get the canvas destination
		mi::base::Handle<mi::neuraylib::ICanvas>& Canvas = BakeCanvases[(int)MapBakeParam.ValueType];
		if (Canvas == nullptr || BakeCanvasesSize[(int)MapBakeParam.ValueType] != BakeTextureSize)
		{
			Canvas = ImageApi->create_canvas(TCHAR_TO_ANSI(Mdl::ToString(MapBakeParam.ValueType)), BakeTextureSize.X, BakeTextureSize.Y);
		}

		// Bake texture
		MDL_CHECK_RESULT() = Baker->bake_texture(Canvas.get(), BakeSamples);

		const EParameterType MapType     = static_cast<EParameterType>(MapTypeInt);
		const FString        TextureName = MaterialName + TEXT("_") + Mdl::ToString(MapType);
		if (MapBakeParam.SetBakedTexture(TextureName, *Canvas))
		{
			UE_LOGF(LogMDLImporter, Log, "Baked texture %ls for: %ls from: %ls with gamma: %f", *FPaths::GetBaseFilename(TextureName),
			       Mdl::ToString(MapType), *MapBakeParam.InputBakePath, Canvas->get_gamma());
		}
		else
		{
			UE_LOGF(LogMDLImporter, Warning, "Failed to bake texture %ls for: %ls from: %ls", *FPaths::GetBaseFilename(TextureName),
			       Mdl::ToString(MapType), *MapBakeParam.InputBakePath);
		}
	}

	void FMaterialDistiller::DistilMaps(mi::neuraylib::ITransaction*             Transaction,
	                                    const FString&                           MaterialName,
	                                    const mi::neuraylib::ICompiled_material* Material,
	                                    FIntPoint                                BakeTextureSize,
	                                    TArray<FBakeParam>&                      MaterialBakeParams) const
	{
		// sort maps processing order based on priority
		if (MapHandler)
		{
			MaterialBakeParams.Sort([](const auto& Lhs, const auto& Rhs)  //
			                        {
				                        int LhsPriority = GetMapPriority(static_cast<EParameterType>(Lhs.Id));
				                        int RhsPriority = GetMapPriority(static_cast<EParameterType>(Rhs.Id));
				                        return LhsPriority < RhsPriority;
			                        });
		}

		for (FBakeParam& MapBakeParam : MaterialBakeParams)
		{
			const EParameterType MapType = static_cast<EParameterType>(MapBakeParam.Id);

			const auto ExpressionHandle = mi::base::make_handle(MapBakeParam.InputExpression);

			// Ignore empty bake paths
			if (MapBakeParam.InputBakePath.IsEmpty())
			{
				MapBakeParam.InputExpression = nullptr;
				continue;
			}

			// Create baker for current path
			mi::base::Handle<const mi::neuraylib::IBaker> Baker(
			    Handle->create_baker(Material, TCHAR_TO_ANSI(*MapBakeParam.InputBakePath), mi::neuraylib::BAKE_ON_CPU));
			checkSlow(Baker.is_valid_interface());

			const bool bIsTexture = !Baker->is_uniform();
			if (MapHandler && MapHandler->Import(Mdl::ToString(MapType), bIsTexture, MapBakeParam))
			{
				// No need to bake from path as path was handled.
			}
			else
			{
				if (bIsTexture)
				{
					BakeTexture(MaterialName, Baker.get(), (int)MapType, BakeTextureSize, MapBakeParam);
				}
				else
				{
					BakeConstantValue(Transaction, Baker.get(), (int)MapType, MapBakeParam);
				}
			}

			MapBakeParam.InputExpression = nullptr;
		}
	}
}

#endif  // #ifdef USE_MDLSDK
