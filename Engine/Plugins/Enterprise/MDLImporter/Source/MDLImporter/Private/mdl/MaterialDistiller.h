// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/IntPoint.h"
#include "Templates/Function.h"

#ifdef USE_MDLSDK

#include "mdl/Common.h"
#include "mi/base/handle.h"

#endif // #ifdef USE_MDLSDK

namespace Mdl
{
	class FApiContext;
	class FMaterialCollection;
	class IMapDistilHandler;
	struct FBakeParam;
	struct FMaterial;

	class IMaterialDistiller : FNoncopyable
	{
	public:
		using FProgressFunc = TFunction<void(const FString& MaterialName, int MaterialIndex)>;

		virtual ~IMaterialDistiller() = default;

		/**
		 * Sets a custom map handler used for the distillation.
		 */
		virtual void SetMapHanlder(IMapDistilHandler* MapHandler) {}

		/**
		 * Sets the baking settings of the material to be baked.
		 *
		 * @param Resolution - resolution of the baked textures.
		 * @param Samples - MSAA samples for the baked textures.
		 */
		virtual void SetBakingSettings(uint32 Resolution, uint32 Samples) {}

		/**
		 * Sets the conversion factor for the baked textures.
		 */
		virtual void SetMetersPerSceneUnit(float MetersPerSceneUnit) {}

		/**
		 * Bakes and distills the materials by populating the material's properties.
		 *
		 * @param Materials - materials to bake, materials with an empty name will be ignored.
		 * @param ProgressFunc - callback called when a material has been distilled.
		 * @return true if the baking process was a success.
		 */
		virtual bool Distil(FMaterialCollection& Materials, FProgressFunc ProgressFunc) const
		{
			return false;
		}
	};
}

#ifdef USE_MDLSDK

namespace mi
{
	namespace neuraylib
	{
		class INeuray;
		class IMdl_distiller_api;
		class IImage_api;
		class ITransaction;
		class IBaker;
		class ICompiled_material;
		class ICanvas;
	}
}
namespace Mdl
{
	class FMaterialDistiller : public IMaterialDistiller
	{
	public:
		FMaterialDistiller() = default;

		virtual ~FMaterialDistiller();

		virtual void SetMapHanlder(IMapDistilHandler* MapHandler) override;

		virtual void SetBakingSettings(uint32 Resolution, uint32 Samples) override;

		virtual void SetMetersPerSceneUnit(float MetersPerSceneUnit) override;

		virtual bool Distil(FMaterialCollection& Materials, FProgressFunc ProgressFunc) const override;

	private:
		FMaterialDistiller(mi::base::Handle<mi::neuraylib::INeuray> Handle);

		void Distil(mi::neuraylib::ITransaction* Transaction,
		            const FString&               MaterialName,
		            const FString&               MaterialDbName,
		            const FString&               MaterialCompiledDbName,
		            FMaterial&                   Material) const;

		void BakeConstantValue(mi::neuraylib::ITransaction* Transaction,
		                       const mi::neuraylib::IBaker* Baker,
		                       int                          MapType,
		                       FBakeParam&                  MapBakeParam) const;

		bool IsEmptyTexture(const mi::neuraylib::IBaker* Baker, EValueType ValueType) const;

		void BakeTexture(const FString&               MaterialName,     //
		                 const mi::neuraylib::IBaker* Baker,            //
		                 int                          MapType,          //
		                 FIntPoint                    BakeTextureSize,  //
		                 FBakeParam&                  MapBakeParam) const;

		void DistilMaps(mi::neuraylib::ITransaction*             Transaction,
		                const FString&                           MaterialName,
		                const mi::neuraylib::ICompiled_material* Material,
		                FIntPoint                                BakeTextureSize,
		                TArray<FBakeParam>&                      MaterialBakeParams) const;

	private:
		inline int ClosestPowerOfTwo(int Value)
		{
			--Value;
			Value |= Value >> 1;
			Value |= Value >> 2;
			Value |= Value >> 4;
			Value |= Value >> 8;
			Value |= Value >> 16;
			return ++Value;
		}

	private:
		mi::base::Handle<mi::neuraylib::INeuray>            Neuray;
		mi::base::Handle<mi::neuraylib::IMdl_distiller_api> Handle;
		mi::base::Handle<mi::neuraylib::IImage_api>         ImageApi;
		mutable mi::base::Handle<mi::neuraylib::ICanvas>    BakeCanvases[(int)EValueType::Count];
		mutable mi::base::Handle<mi::neuraylib::ICanvas>    PreBakeCanvases[(int)EValueType::Count];
		mutable FIntPoint                                   BakeCanvasesSize[(int)EValueType::Count];

		IMapDistilHandler* MapHandler;
		uint32             BakeResolution;
		uint32             BakeSamples;
		float              MetersPerSceneUnit;

		friend class FApiContext;
	};

	inline void FMaterialDistiller::SetMapHanlder(IMapDistilHandler* Handler)
	{
		MapHandler = Handler;
	}

	inline void FMaterialDistiller::SetBakingSettings(uint32 Resolution, uint32 Samples)
	{
		BakeResolution = FMath::Min<uint32>(ClosestPowerOfTwo(Resolution), 16384);
		BakeSamples    = FMath::Min<uint32>(ClosestPowerOfTwo(Samples), 16);
	}

	inline void FMaterialDistiller::SetMetersPerSceneUnit(float InMetersPerSceneUnit)
	{
		MetersPerSceneUnit = InMetersPerSceneUnit;
	}
}
#else  // #ifdef USE_MDLSDK

namespace Mdl
{
	class FMaterialDistiller : public IMaterialDistiller
	{
	};
}

#endif  // #ifndef USE_MDLSDK
