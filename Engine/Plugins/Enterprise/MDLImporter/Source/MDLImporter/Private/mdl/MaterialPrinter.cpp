// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef USE_MDLSDK

#include "MaterialPrinter.h"

#include "common/Logging.h"
#include "mdl/MdlSdkDefines.h"

MDLSDK_INCLUDES_START
#include "mi/neuraylib/icompiled_material.h"
#include "mi/neuraylib/ifunction_definition.h"
#include "mi/neuraylib/imaterial_instance.h"
#include "mi/neuraylib/imdl_factory.h"
#include "mi/neuraylib/istring.h"
#include "mi/neuraylib/itexture.h"
#include "mi/neuraylib/itransaction.h"
#include <mi/base/handle.h>
#include <mi/base/iinterface.h>
#include <mi/base/types.h>
#include <mi/base/uuid.h>
#include <mi/neuraylib/iexpression.h>
#include <mi/neuraylib/itype.h>
#include <mi/neuraylib/ivalue.h>
MDLSDK_INCLUDES_END

namespace
{
	FString GetIndentation(int32 Indent, int32 Offset = 0)
	{
		int32 Count = FMath::Max(0, Indent + Offset);
		return FString::ChrN(2 * Count, ' ');
	}

	bool IsConstructor(mi::neuraylib::IFunction_definition::Semantics Semantic)
	{
		return Semantic >= mi::neuraylib::IFunction_definition::DS_CONV_CONSTRUCTOR &&
		       Semantic <= mi::neuraylib::IFunction_definition::DS_TEXTURE_CONSTRUCTOR;
	}

	bool IsTypeConverter(mi::neuraylib::IFunction_definition::Semantics Semantic)
	{
		return Semantic == mi::neuraylib::IFunction_definition::DS_CONV_CONSTRUCTOR ||
		       Semantic == mi::neuraylib::IFunction_definition::DS_CONV_OPERATOR;
	}

	bool IsArrayConstructor(mi::neuraylib::IFunction_definition::Semantics Semantic)
	{
		return Semantic == mi::neuraylib::IFunction_definition::DS_INTRINSIC_DAG_ARRAY_CONSTRUCTOR;
	}

	bool IsArrayIndexOperator(mi::neuraylib::IFunction_definition::Semantics Semantic)
	{
		return Semantic == mi::neuraylib::IFunction_definition::DS_ARRAY_INDEX;
	}

	bool IsUnaryOperator(mi::neuraylib::IFunction_definition::Semantics Semantic)
	{
		return Semantic >= mi::neuraylib::IFunction_definition::DS_UNARY_FIRST && Semantic <= mi::neuraylib::IFunction_definition::DS_UNARY_LAST;
	}

	bool IsBinaryOperator(mi::neuraylib::IFunction_definition::Semantics Semantic)
	{
		return Semantic >= mi::neuraylib::IFunction_definition::DS_BINARY_FIRST && Semantic <= mi::neuraylib::IFunction_definition::DS_BINARY_LAST;
	}

	bool IsTernaryOperator(mi::neuraylib::IFunction_definition::Semantics Semantic)
	{
		return Semantic == mi::neuraylib::IFunction_definition::DS_TERNARY;
	}

	bool IsSelectorOperator(mi::neuraylib::IFunction_definition::Semantics Semantic)
	{
		return Semantic == mi::neuraylib::IFunction_definition::DS_SELECT ||
		       Semantic == mi::neuraylib::IFunction_definition::DS_INTRINSIC_DAG_FIELD_ACCESS;
	}

	bool IsCallLikeOperator(mi::neuraylib::IFunction_definition::Semantics Semantic)
	{
		// intrinsic functions
		return (Semantic >= mi::neuraylib::IFunction_definition::DS_INTRINSIC_MATH_FIRST &&
		        Semantic <= mi::neuraylib::IFunction_definition::DS_INTRINSIC_DEBUG_LAST) ||
		       // this includes standard function calls
		       Semantic == mi::neuraylib::IFunction_definition::DS_UNKNOWN;
	}

	FString& operator<<(FString& Str, float Value)
	{
		return Str += FString::SanitizeFloat(Value);
	}

	FString& operator<<(FString& Str, double Value)
	{
		return Str += FString::SanitizeFloat(Value);
	}

	FString& operator<<(FString& Str, int Value)
	{
		return Str += FString::FromInt(Value);
	}

	FString& operator<<(FString& Str, const FString& Other)
	{
		return Str += Other;
	}
}

namespace Mdl
{
	FString FMaterialPrinter::Print(const mi::neuraylib::ICompiled_material& Material, mi::neuraylib::ITransaction* Transaction)
	{
		// reset in case of reuse
		{
			TraverseResult.Empty();
			Indent = 0;
			Imports.Empty();

			UsedModules.Empty();
			UsedResources.Empty();

			ParametersToInline.Empty();
			TraverseInlineSwap.Empty();
			IndentInlineSwap = 0;

			TraversalStage = ETraveralStage::Start;
		}

		IMaterialTraverser::Traverse(Material, Transaction);

		FString Output;
		Output << "mdl 1.4;\n\n";

		// add required includes
		int32   LastPos = INDEX_NONE;
		FString LastModule("");
		for (const FString& Import : Imports)
		{
			const int32 CurrentPos = Import.Find(TEXT("::"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
			if (CurrentPos != LastPos || Import.Find(LastModule) != 0)
			{
				LastModule = Import.Mid(0, CurrentPos);
				LastPos    = CurrentPos;
				Output << TEXT("// import ") << LastModule << TEXT("::*;\n");
				UsedModules.Add(LastModule);
			}
			if (CurrentPos == 0)  // show imports of the base namespace (just to list them up)
			{
				Output << TEXT("//* ");
			}

			Output << TEXT("import ") << Import << TEXT(";\n");
		}

		// append the result of the traversal
		Output << TraverseResult;

		return Output;
	}

	FString FMaterialPrinter::Print(const mi::neuraylib::IMaterial_instance& Material,
	                                mi::neuraylib::IMdl_factory*             MdlFactory,
	                                mi::neuraylib::ITransaction*             Transaction)
	{
		FString Output;

		mi::base::Handle<mi::neuraylib::IExpression_factory> ExpressionFactory(MdlFactory->create_expression_factory(Transaction));

		const mi::Size                                          Count = Material.get_parameter_count();
		mi::base::Handle<const mi::neuraylib::IExpression_list> Arguments(Material.get_arguments());

		FString DumpedText;
		for (mi::Size Index = 0; Index < Count; ++Index)
		{
			mi::base::Handle<const mi::neuraylib::IExpression> Argument(Arguments->get_expression(Index));

			const char*                         Name = Material.get_parameter_name(Index);
			mi::base::Handle<const mi::IString> ArgumentText(ExpressionFactory->dump(Argument.get(), Name, 1));

			DumpedText = UTF8_TO_TCHAR(ArgumentText->get_c_str());
			if (DumpedText.Len() > 512)
			{
				DumpedText.RemoveAt(512, DumpedText.Len() - 512);
				DumpedText += TEXT("\n\t\t...");
			}
			Output << TEXT("    argument ") << DumpedText << TEXT("\n");
		}
		Output << TEXT("\n");
		return Output;
	}

	void FMaterialPrinter::StageBegin(const mi::neuraylib::ICompiled_material& Material, ETraveralStage Stage, mi::neuraylib::ITransaction*)
	{
		TraversalStage = Stage;

		switch (Stage)
		{
			case ETraveralStage::Parameters:
			{
				TraverseResult << TEXT("(\n");
				break;
			}
			case ETraveralStage::Temporaries:
			{
				TraverseResult << (Material.get_temporary_count() == 0 ? TEXT(" = ") : TEXT(" = let{\n"));
				break;
			}
			case ETraveralStage::Body:
			default:
				break;
		}
	}

	void FMaterialPrinter::StageEnd(const mi::neuraylib::ICompiled_material& Material, ETraveralStage Stage, mi::neuraylib::ITransaction*)
	{
		switch (Stage)
		{
			case ETraveralStage::Parameters:
			{
				TraverseResult << TEXT(")\n");
				// at this point one could add annotations here,
				// if they are made available to the context
				break;
			}
			case ETraveralStage::Temporaries:
			{
				TraverseResult << (Material.get_temporary_count() == 0 ? TEXT("") : TEXT("} in "));
				break;
			}
			case ETraveralStage::Body:
			{
				TraverseResult << TEXT(";\n");
				break;
			}
			default:
				break;
		}
	}

	void FMaterialPrinter::HandleModulesAndImports(const mi::base::Handle<const mi::neuraylib::IFunction_definition>& FuncDef,
	                                               int                                                                SemanticInt,
	                                               FString&                                                           FunctionName)
	{
		const auto    Semantic   = static_cast<mi::neuraylib::IFunction_definition::Semantics>(SemanticInt);
		const FString ModuleName = FuncDef->get_module();
		if (ModuleName == TEXT("mdl::<builtins>"))
		{
			return;
		}

		// type conversion using constructors can lead to invalid mdl code
		// as these conversion constructors are created in the local module space
		if (IsTypeConverter(Semantic))
		{
			// Keep imports for structure definitions and arrays of structures as
			// well as enums and arrays of enums as they can be user defined.
			// For other types, we can drop the qualification part
			bool bDropQualification = true;

			const mi::base::Handle<const mi::neuraylib::IType> ReturnType(FuncDef->get_return_type());

			const mi::base::Handle<const mi::neuraylib::IType_struct> ReturnTypeStruct(
			    ReturnType->get_interface<const mi::neuraylib::IType_struct>());

			const mi::base::Handle<const mi::neuraylib::IType_enum> ReturnTypeEnum(ReturnType->get_interface<const mi::neuraylib::IType_enum>());

			const mi::base::Handle<const mi::neuraylib::IType_array> ReturnTypeArray(ReturnType->get_interface<const mi::neuraylib::IType_array>());

			if (ReturnTypeStruct || ReturnTypeEnum)
			{
				bDropQualification = false;
			}
			else if (ReturnTypeArray)
			{
				const mi::base::Handle<const mi::neuraylib::IType> ReturnTypeArrayEType(ReturnTypeArray->get_element_type());

				const mi::base::Handle<const mi::neuraylib::IType_struct> ReturnTypeEStruct(
				    ReturnTypeArrayEType->get_interface<const mi::neuraylib::IType_struct>());

				const mi::base::Handle<const mi::neuraylib::IType_enum> ReturnTypeEEnum(
				    ReturnTypeArrayEType->get_interface<const mi::neuraylib::IType_enum>());

				if (ReturnTypeEStruct || ReturnTypeEEnum)
				{
					bDropQualification = false;
				}
			}

			if (bDropQualification)
			{
				// strip qualification part of the name
				const int32 Found = FunctionName.Find(TEXT("::"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
				if (Found != INDEX_NONE)
				{
					FunctionName = FunctionName.Mid(Found + 2);
				}
			}
			else
			{
				// add the the struct or enum definition to the imports
				Imports.Add(FunctionName);
			}
		}
		// everything before the dot needs to be imported
		else if (IsSelectorOperator(Semantic))
		{
			// if this is a qualified name (and not the name of variable)
			if (FunctionName.Find(TEXT("::"), ESearchCase::IgnoreCase, ESearchDir::FromEnd) != INDEX_NONE)
			{
				int32 PosDot = 0;
				FunctionName.FindChar('.', PosDot);
				Imports.Add(FunctionName.Mid(0, PosDot));
			}
		}
		// general case, import the function name
		else
		{
			Imports.Add(FunctionName);
		}
	}

	void FMaterialPrinter::HandleVisitBeginExpression(const mi::neuraylib::ICompiled_material& Material,
	                                                  const FTraversalElement&                 Element,
	                                                  mi::neuraylib::ITransaction*             Transaction)
	{
		switch (Element.Expression->get_kind())
		{
			case mi::neuraylib::IExpression::EK_CONSTANT:
				return;  // nothing to print here

			case mi::neuraylib::IExpression::EK_CALL:
				return;  // for compiled materials, this will not happen

			case mi::neuraylib::IExpression::EK_PARAMETER:
			{
				mi::base::Handle<const mi::neuraylib::IExpression_parameter> ExprParam(
				    Element.Expression->get_interface<const mi::neuraylib::IExpression_parameter>());

				// get the parameter name
				bool    bWasGenerated;
				FString Name = GetParameterName(Material, ExprParam->get_index(), &bWasGenerated);

				// if we choose to inline generated parameters, we do it here
				if (!bKeepCompiledStructure && bWasGenerated)
				{
					// to get the right indentation, we need to add the indent for each line
					FString       ToInline = ParametersToInline[Name];
					const FString Replace  = TEXT("\n") + GetIndentation(Indent);
					ToInline.ReplaceInline(TEXT("\n"), *Replace);
					TraverseResult << ToInline;
					TraverseResult << TEXT("/*inlined generated param*/");
				}
				// otherwise handle generated parameters like any other
				else
				{
					TraverseResult << Name << TEXT("/*param*/");
				}

				return;
			}
			case mi::neuraylib::IExpression::EK_DIRECT_CALL:
			{
				const mi::base::Handle<const mi::neuraylib::IExpression_direct_call> ExprDcall(
				    Element.Expression->get_interface<const mi::neuraylib::IExpression_direct_call>());
				const mi::base::Handle<const mi::neuraylib::IExpression_list>     Args(ExprDcall->get_arguments());
				const mi::base::Handle<const mi::neuraylib::IFunction_definition> FuncDef(
				    Transaction->access<mi::neuraylib::IFunction_definition>(ExprDcall->get_definition()));
				const mi::neuraylib::IFunction_definition::Semantics Semantic = FuncDef->get_semantic();

				FString FunctionName = FuncDef->get_mdl_name();
				int32   Index        = 0;
				FunctionName.FindChar('(', Index);
				FunctionName = FunctionName.Mid(0, Index);

				// keep track of used modules and/or imports
				HandleModulesAndImports(FuncDef, Semantic, FunctionName);

				// array constructors are one special case of constructor
				if (IsArrayConstructor(Semantic))
				{
					// array type is the type of the first parameter
					const mi::base::Handle<const mi::neuraylib::IExpression> ArgZero(Args->get_expression(mi::Size(0)));
					const mi::base::Handle<const mi::neuraylib::IType>       ArgZeroType(ArgZero->get_type());
					const FString                                            TypeString = TypeToString(ArgZeroType.get());
					TraverseResult << TypeString << TEXT("[](/*array constructor*/");
					return;
				}

				// check for special cases based on the Semantic
				if (IsSelectorOperator(Semantic) || IsUnaryOperator(Semantic) || IsArrayIndexOperator(Semantic))
				{
					return;
				}

				if (IsBinaryOperator(Semantic) || IsTernaryOperator(Semantic))
				{
					TraverseResult << TEXT("(");
					return;
				}

				if (IsTypeConverter(Semantic))
				{
					TraverseResult << FunctionName << TEXT("(");
					return;
				}

				mi::Size ArgCount = FuncDef->get_parameter_count();
				if (IsConstructor(Semantic))
				{
					if (ArgCount > 0)
					{
						TraverseResult << FunctionName << TEXT("(/*constructor*/");
					}
					else
					{
						TraverseResult << FunctionName << TEXT("()/*constructor*/");
					}
					return;
				}

				if (IsCallLikeOperator(Semantic))
				{
					if (ArgCount > 0)
					{
						TraverseResult << FunctionName << TEXT("(/*call*/");
					}
					else
					{
						TraverseResult << FunctionName << TEXT("()/*call*/");
					}
					return;
				}

				UE_LOGF(LogMDLImporter, Error, "[MaterialPrinter] ran into unhandled Semantic: '%ls' Semantic: '%d'", *FunctionName, Semantic);
				return;
			}
			case mi::neuraylib::IExpression::EK_TEMPORARY:
			{
				const mi::base::Handle<const mi::neuraylib::IExpression_temporary> ExprTemp(
				    Element.Expression->get_interface<const mi::neuraylib::IExpression_temporary>());
				TraverseResult << GetTemporaryName(ExprTemp->get_index());
				return;
			}
			case mi::neuraylib::IExpression::EK_FORCE_32_BIT:
				return;  // not a valid case
			default:
				return;
		}
	}

	void FMaterialPrinter::HandleVisitBeginValue(const FTraversalElement& Element, mi::neuraylib::ITransaction* Transaction)
	{
		switch (Element.Value->get_kind())
		{
			case mi::neuraylib::IValue::VK_BOOL:
			{
				const mi::base::Handle<const mi::neuraylib::IValue_bool> ValueBool(Element.Value->get_interface<const mi::neuraylib::IValue_bool>());
				TraverseResult << (ValueBool->get_value() ? TEXT("true") : TEXT("false"));
				return;
			}
			case mi::neuraylib::IValue::VK_INT:
			{
				const mi::base::Handle<const mi::neuraylib::IValue_int> ValueInt(Element.Value->get_interface<const mi::neuraylib::IValue_int>());
				TraverseResult << ValueInt->get_value();
				return;
			}
			case mi::neuraylib::IValue::VK_ENUM:
			{
				const mi::base::Handle<const mi::neuraylib::IValue_enum> ValueEnum(Element.Value->get_interface<const mi::neuraylib::IValue_enum>());

				const mi::base::Handle<const mi::neuraylib::IType_enum> TypeEnum(ValueEnum->get_type());

				FString EnumName      = EnumTypeToString(TypeEnum.get());
				int32   Index         = EnumName.Find(TEXT("::"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
				FString EnumNamespace = EnumName.Mid(0, Index);

				TraverseResult << EnumNamespace << TEXT("::" << ValueEnum->get_name()) << TEXT("/*enum of type: '") << EnumName << TEXT("'*/");
				return;
			}
			case mi::neuraylib::IValue::VK_FLOAT:
			{
				const mi::base::Handle<const mi::neuraylib::IValue_float> ValueFloat(
				    Element.Value->get_interface<const mi::neuraylib::IValue_float>());
				TraverseResult << ValueFloat->get_value() << TEXT("f");
				return;
			}
			case mi::neuraylib::IValue::VK_DOUBLE:
			{
				const mi::base::Handle<const mi::neuraylib::IValue_double> ValueDouble(
				    Element.Value->get_interface<const mi::neuraylib::IValue_double>());
				TraverseResult << ValueDouble->get_value();
				return;
			}
			case mi::neuraylib::IValue::VK_STRING:
			{
				const mi::base::Handle<const mi::neuraylib::IValue_string> ValueString(
				    Element.Value->get_interface<const mi::neuraylib::IValue_string>());
				TraverseResult << TEXT("\"") << ValueString->get_value() << TEXT("\"");
				return;
			}
			case mi::neuraylib::IValue::VK_VECTOR:
			{
				const mi::base::Handle<const mi::neuraylib::IValue_vector> ValueVector(
				    Element.Value->get_interface<const mi::neuraylib::IValue_vector>());
				const mi::base::Handle<const mi::neuraylib::IType_vector> VectorType(ValueVector->get_type());
				TraverseResult << VectorTypeToString(VectorType.get()) << TEXT("(");
				return;
			}
			case mi::neuraylib::IValue::VK_MATRIX:
			{
				const mi::base::Handle<const mi::neuraylib::IValue_matrix> ValueMatrix(
				    Element.Value->get_interface<const mi::neuraylib::IValue_matrix>());
				const mi::base::Handle<const mi::neuraylib::IType_matrix> MatrixType(ValueMatrix->get_type());
				TraverseResult << MatrixTypeToString(MatrixType.get()) << TEXT("(");
				return;
			}
			case mi::neuraylib::IValue::VK_COLOR:
			{
				TraverseResult << TEXT("color(");
				return;
			}
			case mi::neuraylib::IValue::VK_ARRAY:
			{
				const mi::base::Handle<const mi::neuraylib::IType>       Type(Element.Value->get_type());
				const mi::base::Handle<const mi::neuraylib::IType_array> ArrayType(Type.get_interface<const mi::neuraylib::IType_array>());
				TraverseResult << ArrayTypeToString(ArrayType.get()) << TEXT("(");
				return;
			}
			case mi::neuraylib::IValue::VK_STRUCT:
			{
				const mi::base::Handle<const mi::neuraylib::IValue_struct> ValueStruct(
				    Element.Value->get_interface<const mi::neuraylib::IValue_struct>());
				const mi::base::Handle<const mi::neuraylib::IType_struct> StrucType(ValueStruct->get_type());

				// there are special mdl keywords that are handled as struct internally
				bool    bIsKeyword;
				FString TypeName = StructTypeToString(StrucType.get(), &bIsKeyword);
				TraverseResult << TypeName << TEXT("(");

				if (!bIsKeyword)
				{
					TraverseResult << TEXT("/*struct*/");
				}
				return;
			}
			case mi::neuraylib::IValue::VK_INVALID_DF:
			{
				const mi::base::Handle<const mi::neuraylib::IValue_invalid_df> InvalidDf(
				    Element.Value->get_interface<const mi::neuraylib::IValue_invalid_df>());
				const mi::base::Handle<const mi::neuraylib::IType_reference> RefType(InvalidDf->get_type());

				// use the default constructor for distribution functions
				TraverseResult << TypeToString(RefType.get()) << TEXT("()");
				return;
			}
			case mi::neuraylib::IValue::VK_TEXTURE:
			{
				const mi::base::Handle<const mi::neuraylib::IValue_texture> TextureValue(
				    Element.Value->get_interface<const mi::neuraylib::IValue_texture>());
				const mi::base::Handle<const mi::neuraylib::IType_texture> TextureType(TextureValue->get_type());
				TraverseResult << TypeToString(TextureType.get()) << TEXT("(");

				const char* MdlPath = TextureValue->get_file_path();
				if (MdlPath)
				{
					TraverseResult << TEXT('\"') << UTF8_TO_TCHAR(MdlPath) << TEXT('\"');
					UsedResources.Add(UTF8_TO_TCHAR(MdlPath));  // keep track of this information

					// get the texture for gamma value
					const mi::base::Handle<const mi::neuraylib::ITexture> Texture(
					    Transaction->access<mi::neuraylib::ITexture>(TextureValue->get_value()));

					if (Texture)
					{
						// gamma
						const float Gamma = Texture->get_gamma();
						if (FMath::IsNearlyEqual(Gamma, 0.0f))
						{
							TraverseResult << TEXT(", gamma: ::tex::gamma_default");
						}
						else if (FMath::IsNearlyEqual(Gamma, 1.0f))
						{
							TraverseResult << TEXT(", gamma: ::tex::gamma_linear");
						}
						else if (FMath::IsNearlyEqual(Gamma, 2.2f))
						{
							TraverseResult << TEXT(", gamma: ::tex::gamma_srgb");
						}
						Imports.Add("::tex::gamma_mode");
					}
					else
					{
						// when the compiler removes unresolved resources, this is never reached
						UE_LOGF(LogMDLImporter, Error, "[MaterialPrinter] unresolved texture: '%s'", MdlPath);
					}
				}
				TraverseResult << TEXT(")");
				return;
			}
			case mi::neuraylib::IValue::VK_LIGHT_PROFILE:
			{
				const mi::base::Handle<const mi::neuraylib::IValue_light_profile> ValueLightProfile(
				    Element.Value->get_interface<const mi::neuraylib::IValue_light_profile>());
				const mi::base::Handle<const mi::neuraylib::IType_light_profile> TypeLightProfile(ValueLightProfile->get_type());

				TraverseResult << TypeToString(TypeLightProfile.get()) << TEXT("(");

				const char* MdlPath = ValueLightProfile->get_file_path();
				if (MdlPath)
				{
					UsedResources.Add(UTF8_TO_TCHAR(MdlPath));  // keep track of this information
					TraverseResult << TEXT('\"') << UTF8_TO_TCHAR(MdlPath) << TEXT('\"');
				}

				TraverseResult << TEXT(")");
				return;
			}
			case mi::neuraylib::IValue::VK_BSDF_MEASUREMENT:
			{
				const mi::base::Handle<const mi::neuraylib::IValue_bsdf_measurement> ValueBsdfMeasurement(
				    Element.Value->get_interface<const mi::neuraylib::IValue_bsdf_measurement>());
				const mi::base::Handle<const mi::neuraylib::IType_bsdf_measurement> TypeBsdfMeasurement(ValueBsdfMeasurement->get_type());

				TraverseResult << TypeToString(TypeBsdfMeasurement.get()) << TEXT("(");

				const char* MDLPath = ValueBsdfMeasurement->get_file_path();
				if (MDLPath)
				{
					UsedResources.Add(UTF8_TO_TCHAR(MDLPath));  // keep track of this information
					TraverseResult << TEXT("\"") << UTF8_TO_TCHAR(MDLPath) << TEXT("\"");
				}

				TraverseResult << TEXT(")");
				return;
			}

			default:
				return;
		}
	}

	void FMaterialPrinter::VisitBegin(const mi::neuraylib::ICompiled_material& Material,
	                                  const FTraversalElement&                 Element,
	                                  mi::neuraylib::ITransaction*             Transaction)
	{
		Indent++;

		// major cases: parameter, temporary, expression or value
		if (Element.Expression)
		{
			HandleVisitBeginExpression(Material, Element, Transaction);
		}
		// major cases: parameter, temporary, expression or value
		else if (Element.Value)
		{
			HandleVisitBeginValue(Element, Transaction);
		}
		// major cases: parameter, temporary, expression or value
		else if (Element.Parameter)
		{
			bool    bWasGenerated;
			FString Name = GetParameterName(Material, Element.SiblingIndex, &bWasGenerated);

			// in case we want to inline the compiler generated parameters
			// we temporarily swap the the stream for the time we process the parameter
			// we will swap back in the 'visit_end' method
			if (!bKeepCompiledStructure && bWasGenerated)
			{
				TraverseInlineSwap = TraverseResult;
				TraverseResult.Reset();

				Swap(Indent, IndentInlineSwap);
				Indent = 0;
			}
			// if we don't line, we need to define a parameter
			else
			{
				mi::base::Handle<const mi::neuraylib::IType> Type(Element.Parameter->Value->get_type());
				// assuming all parameters are uniforms at this point
				TraverseResult << GetIndentation(Indent) << TEXT("uniform " << TypeToString(Type.get())) << TEXT(" ") << Name << TEXT(" = ");
			}
		}
		// major cases: parameter, temporary, expression or value
		else if (Element.Temporary)
		{
			mi::base::Handle<const mi::neuraylib::IType> TemporaryReturnType(Element.Temporary->Expression->get_type());

			// include the return type if required
			const FString ReturnType = TypeToString(TemporaryReturnType.get());
			// no '::' has to be a build-in type
			if (ReturnType.Find(TEXT("::"), ESearchCase::IgnoreCase, ESearchDir::FromEnd) != INDEX_NONE)
			{
				Imports.Add(ReturnType);
			}

			TraverseResult << GetIndentation(Indent) << ReturnType << TEXT(" ") << GetTemporaryName(Element.SiblingIndex) << TEXT(" = ");
		}
	}

	void FMaterialPrinter::HandleVisitChildExpression(const FTraversalElement& Element, uint32 ChildIndex, mi::neuraylib::ITransaction* Transaction)
	{
		switch (Element.Expression->get_kind())
		{
			case mi::neuraylib::IExpression::EK_DIRECT_CALL:
			{
				const mi::base::Handle<const mi::neuraylib::IExpression_direct_call> ExprDcall(
				    Element.Expression->get_interface<const mi::neuraylib::IExpression_direct_call>());
				const mi::base::Handle<const mi::neuraylib::IExpression_list>     Arguments(ExprDcall->get_arguments());
				const mi::base::Handle<const mi::neuraylib::IFunction_definition> FuncDef(
				    Transaction->access<mi::neuraylib::IFunction_definition>(ExprDcall->get_definition()));
				const mi::neuraylib::IFunction_definition::Semantics Semantic = FuncDef->get_semantic();

				FString FunctionName = FuncDef->get_mdl_name();
				int32   Index        = 0;
				FunctionName.FindChar('(', Index);
				FunctionName = FunctionName.Mid(0, Index);

				// check for special cases based on the Semantic

				if (IsSelectorOperator(Semantic))
				{
					return;
				}

				if (IsUnaryOperator(Semantic))
				{
					FString Op = FunctionName.Mid(8);
					TraverseResult << Op;
					return;
				}

				if (IsBinaryOperator(Semantic))
				{
					if (ChildIndex == 1)
					{
						FString Op = FunctionName.Mid(8);
						TraverseResult << TEXT(" ") << Op << TEXT(" ");
					}
					return;
				}

				if (IsTernaryOperator(Semantic))
				{
					if (ChildIndex == 1)
					{
						TraverseResult << TEXT("\n") << GetIndentation(Indent) << TEXT("? ");
					}
					else if (ChildIndex == 2)
					{
						TraverseResult << TEXT("\n") << GetIndentation(Indent) << TEXT(": ");
					}
					return;
				}

				if (IsArrayIndexOperator(Semantic))
				{
					if (ChildIndex == 1)
					{
						TraverseResult << TEXT("[");
					}
					return;
				}

				// argument lists without line breaks
				if (IsTypeConverter(Semantic))
				{
					TraverseResult << (ChildIndex == 0 ? "" : ",");
					return;
				}

				// default case: argument lists with line breaks
				// syntax of function call with parameters
				if (IsCallLikeOperator(Semantic) || IsConstructor(Semantic))
				{
					TraverseResult << (ChildIndex == 0 ? "\n" : ",\n") << GetIndentation(Indent) << Arguments->get_name(ChildIndex)
					               << TEXT(": ");  // named parameters
					return;
				}

				if (IsArrayConstructor(Semantic))
				{
					TraverseResult << (ChildIndex == 0 ? "\n" : ",\n") << GetIndentation(Indent);  // no names
					return;
				}
				UE_LOGF(LogMDLImporter, Error, "[MaterialPrinter] ran into unhandled Semantic: '%ls' Semantic: '%d'", *FunctionName, Semantic);
				return;
			}
			default:
				break;
		}
	}

	void FMaterialPrinter::VisitChild(const mi::neuraylib::ICompiled_material& /*Material*/, const FTraversalElement& Element,
	                                  uint32 /*ChildrenCount*/, uint32 ChildIndex, mi::neuraylib::ITransaction* Transaction)
	{
		// major cases: parameter, temporary, expression or value
		if (Element.Expression)
		{
			HandleVisitChildExpression(Element, ChildIndex, Transaction);
		}
		// major cases: parameter, temporary, expression or value
		else if (Element.Value)
		{
			switch (Element.Value->get_kind())
			{
				case mi::neuraylib::IValue::VK_VECTOR:  // Intended fallthrough
				case mi::neuraylib::IValue::VK_COLOR:   // Intended fallthrough
				case mi::neuraylib::IValue::VK_ARRAY:
				{
					if (ChildIndex != 0)
					{
						TraverseResult << TEXT(", ");
					}
					return;
				}

				case mi::neuraylib::IValue::VK_MATRIX:  // Intended fallthrough
				case mi::neuraylib::IValue::VK_STRUCT:
				{
					TraverseResult << (ChildIndex == 0 ? TEXT("\n") : TEXT(",\n")) << GetIndentation(Indent);
					return;
				}

				default:
					return;
			}
		}
	}

	void FMaterialPrinter::HandleVisitEndExpression(const FTraversalElement& Element, mi::neuraylib::ITransaction* Transaction)
	{
		switch (Element.Expression->get_kind())
		{
			case mi::neuraylib::IExpression::EK_DIRECT_CALL:
			{
				const mi::base::Handle<const mi::neuraylib::IExpression_direct_call> ExprDcall(
				    Element.Expression->get_interface<const mi::neuraylib::IExpression_direct_call>());
				const mi::base::Handle<const mi::neuraylib::IExpression_list>     Args(ExprDcall->get_arguments());
				const mi::base::Handle<const mi::neuraylib::IFunction_definition> FuncDef(
				    Transaction->access<mi::neuraylib::IFunction_definition>(ExprDcall->get_definition()));

				const mi::neuraylib::IFunction_definition::Semantics Semantic = FuncDef->get_semantic();

				if (IsUnaryOperator(Semantic))
				{
					break;
				}

				if (IsArrayIndexOperator(Semantic))
				{
					TraverseResult << TEXT("]");
					break;
				}
				if (IsSelectorOperator(Semantic))
				{
					FString Selector = UTF8_TO_TCHAR(FuncDef->get_mdl_name());
					int32   Index    = 0;
					Selector.FindChar('(', Index);
					Selector = Selector.Mid(0, Index);
					Index    = 0;
					Selector.FindLastChar('.', Index);
					Selector = Selector.Mid(Index);
					TraverseResult << Selector;
					break;
				}
				// function call syntax without line break
				if (IsBinaryOperator(Semantic) || IsTernaryOperator(Semantic))
				{
					TraverseResult << TEXT(")");
					break;
				}
				if (IsTypeConverter(Semantic))
				{
					TraverseResult << TEXT(")/*type conversion*/");
					break;
				}

				// function call syntax with line break contains standard functions
				if (IsCallLikeOperator(Semantic) || IsConstructor(Semantic))
				{
					mi::Size ArgCount = FuncDef->get_parameter_count();
					if (ArgCount == 0)
					{
						break;
					}

					TraverseResult << TEXT("\n") << GetIndentation(Indent, -1) << TEXT(")");
					break;
				}

				if (IsArrayConstructor(Semantic))
				{
					// a bit special because the parameter count is zero
					// for the array constructor Semantic
					TraverseResult << TEXT("\n") << GetIndentation(Indent, -1) << TEXT(")");
					break;
				}

				// error case (should not happen):
				FString Name = UTF8_TO_TCHAR(FuncDef->get_mdl_name());
				UE_LOGF(LogMDLImporter, Error, "[MaterialPrinter] ran into unhandled Semantic: '%ls' Semantic: '%d'", *Name, Semantic);
				break;
			}
			default:
				break;
		}
	}

	void FMaterialPrinter::HandleVisitEndParameter(const mi::neuraylib::ICompiled_material& Material, const FTraversalElement& Element)
	{
		// we need to know if this is a generated parameter
		bool    bWasGenerated;
		FString Name = GetParameterName(Material, Element.SiblingIndex, &bWasGenerated);

		// optionally add annotations here (in the generated == false case)

		// in case we want to inline the compiler generated parameters
		// we temporarily swapped the the stream for the time we process the parameter
		// this happened in the 'visit_begin' method
		if (!bKeepCompiledStructure && bWasGenerated)
		{
			// keep the printed code and swap back
			ParametersToInline.Add(Name, TraverseResult);

			TraverseResult.Reset();
			TraverseResult = TraverseInlineSwap;

			Swap(Indent, IndentInlineSwap);

			// there is a special case to keep the mdl output right we need to remove the
			// comma if the last parameter was a generated one.
			// In case there is no 'normal' parameter, we must not do that.
			if (Element.SiblingIndex == Element.SiblingCount - 1)
			{
				// therefore we simply check if the last two characters are ",\n"
				// and if so, we replace them by "\n "
				if (TraverseResult.EndsWith(TEXT(",\n")))
				{
					TraverseResult.RemoveAt(TraverseResult.Len() - 2, 2);
					TraverseResult << TEXT("\n");
				}
			}
		}
		// if we don't inline, we finish the parameter definition
		else
		{
			TraverseResult << ((Element.SiblingIndex < Element.SiblingCount - 1) ? ",\n" : "\n");
		}
	}

	void FMaterialPrinter::VisitEnd(const mi::neuraylib::ICompiled_material& Material,
	                                const FTraversalElement&                 Element,
	                                mi::neuraylib::ITransaction*             Transaction)
	{
		// major cases: parameter, temporary, expression or value
		if (Element.Expression)
		{
			HandleVisitEndExpression(Element, Transaction);
		}

		// major cases: parameter, temporary, expression or value
		else if (Element.Value)
		{
			switch (Element.Value->get_kind())
			{
				case mi::neuraylib::IValue::VK_VECTOR:  // Intended fallthrough
				case mi::neuraylib::IValue::VK_COLOR:   // Intended fallthrough
				case mi::neuraylib::IValue::VK_ARRAY:
				{
					TraverseResult << TEXT(")");
					break;
				}

				case mi::neuraylib::IValue::VK_STRUCT:  // Intended fallthrough
				case mi::neuraylib::IValue::VK_MATRIX:
				{
					TraverseResult << TEXT("\n") << GetIndentation(Indent, -1) << TEXT(")");
					break;
				}

				default:
					break;
			}
		}
		// major cases: parameter, temporary, expression or value
		else if (Element.Parameter)
		{
			HandleVisitEndParameter(Material, Element);
		}

		// major cases: parameter, temporary, expression or value
		else if (Element.Temporary)
		{
			TraverseResult << TEXT(";\n");
		}

		Indent--;
	}

	FString FMaterialPrinter::TypeToString(const mi::neuraylib::IType* Type)
	{
		mi::base::Handle<const mi::neuraylib::IType_atomic> AtomicType(Type->get_interface<const mi::neuraylib::IType_atomic>());
		if (AtomicType)
		{
			return AtomicTypeToString(AtomicType.get());
		}

		switch (Type->get_kind())
		{
			case mi::neuraylib::IType::TK_COLOR:
			{
				return TEXT("color");
			}
			case mi::neuraylib::IType::TK_STRUCT:
			{
				mi::base::Handle<const mi::neuraylib::IType_struct> StructType(Type->get_interface<const mi::neuraylib::IType_struct>());
				return StructTypeToString(StructType.get());
			}
			case mi::neuraylib::IType::TK_ARRAY:
			{
				mi::base::Handle<const mi::neuraylib::IType_array> ArrayType(Type->get_interface<const mi::neuraylib::IType_array>());
				return ArrayTypeToString(ArrayType.get());
			}
			case mi::neuraylib::IType::TK_VECTOR:
			{
				mi::base::Handle<const mi::neuraylib::IType_vector> VectorType(Type->get_interface<const mi::neuraylib::IType_vector>());
				return VectorTypeToString(VectorType.get());
			}

			case mi::neuraylib::IType::TK_TEXTURE:
			{
				mi::base::Handle<const mi::neuraylib::IType_texture> TextureType(Type->get_interface<const mi::neuraylib::IType_texture>());
				switch (TextureType->get_shape())
				{
					case mi::neuraylib::IType_texture::TS_2D:
						return TEXT("texture_2d");

					case mi::neuraylib::IType_texture::TS_3D:
						return TEXT("texture_3d");

					case mi::neuraylib::IType_texture::TS_CUBE:
						return TEXT("texture_cube");

					case mi::neuraylib::IType_texture::TS_PTEX:
						return TEXT("texture_ptex");

					default:
						break;
				}
			}
			break;

			case mi::neuraylib::IType::TK_LIGHT_PROFILE:
				return TEXT("light_profile");

			case mi::neuraylib::IType::TK_BSDF_MEASUREMENT:
				return TEXT("bsdf_measurement");

			case mi::neuraylib::IType::TK_BSDF:
				return TEXT("bsdf");

			case mi::neuraylib::IType::TK_EDF:
				return TEXT("edf");

			case mi::neuraylib::IType::TK_VDF:
				return TEXT("vdf");

			default:
				break;
		}
		return TEXT("UNKNOWN_TYPE");
	}

	FString FMaterialPrinter::EnumTypeToString(const mi::neuraylib::IType_enum* EnumType)
	{
		FString Symbol = EnumType->get_symbol();

		// make sure the type is included properly
		Imports.Add(Symbol);
		return Symbol;
	}

	FString FMaterialPrinter::StructTypeToString(const mi::neuraylib::IType_struct* StructType, bool* bIsMaterialKeyword)
	{
		FString Symbol = UTF8_TO_TCHAR(StructType->get_symbol());

		// deal with keywords defined in MDL spec section 13 on 'Materials'
		// the compiler handles these constructs as structures
		if (Symbol == TEXT("::material") || Symbol == TEXT("::bsdf") || Symbol == TEXT("::edf") || Symbol == TEXT("::vdf") ||
		    Symbol == TEXT("::material_surface") || Symbol == TEXT("::material_emission") || Symbol == TEXT("::material_volume") ||
		    Symbol == TEXT("::material_geometry"))
		{
			if (bKeepCompiledStructure && TraversalStage == ETraveralStage::Parameters)
			{
				UE_LOG(LogMDLImporter, Error, TEXT("[MaterialPrinter]The compiled material defines a \
					parameter of type '%s' ', which results in the printing of invalid mdl code."),
				       *Symbol.Mid(2));
			}

			if (bIsMaterialKeyword)
			{
				*bIsMaterialKeyword = true;
			}
			return Symbol.Mid(2);
		}
		if (bIsMaterialKeyword)
		{
			*bIsMaterialKeyword = false;
		}

		// if this is not the case, we need to make sure the type is included properly
		Imports.Add(Symbol);
		return Symbol;
	}

	FString FMaterialPrinter::AtomicTypeToString(const mi::neuraylib::IType_atomic* AtomicType)
	{
		switch (AtomicType->get_kind())
		{
			case mi::neuraylib::IType::TK_BOOL:
				return TEXT("bool");
			case mi::neuraylib::IType::TK_INT:
				return TEXT("int");
			case mi::neuraylib::IType::TK_FLOAT:
				return TEXT("float");
			case mi::neuraylib::IType::TK_DOUBLE:
				return TEXT("double");
			case mi::neuraylib::IType::TK_STRING:
				return TEXT("string");
			case mi::neuraylib::IType::TK_ENUM:
			{
				mi::base::Handle<const mi::neuraylib::IType_enum> EnumType(AtomicType->get_interface<const mi::neuraylib::IType_enum>());
				return EnumTypeToString(EnumType.get());
			}
			default:
				break;
		}
		return TEXT("UNKNOWN_TYPE");
	}

	FString FMaterialPrinter::VectorTypeToString(const mi::neuraylib::IType_vector* VectorType)
	{
		const mi::base::Handle<const mi::neuraylib::IType_atomic> ElemType(VectorType->get_element_type());
		return AtomicTypeToString(ElemType.get()) + FString::FromInt(VectorType->get_size());
	}

	FString FMaterialPrinter::MatrixTypeToString(const mi::neuraylib::IType_matrix* MatrixType)
	{
		const mi::base::Handle<const mi::neuraylib::IType_vector> ColumnType(MatrixType->get_element_type());

		const mi::base::Handle<const mi::neuraylib::IType_atomic> ElemType(ColumnType->get_element_type());

		FString Result = AtomicTypeToString(ElemType.get());
		Result << FString::FromInt(ColumnType->get_size()) << FString(TEXT("x")) << (int)MatrixType->get_size();
		return Result;
	}

	FString FMaterialPrinter::ArrayTypeToString(const mi::neuraylib::IType_array* ArrayType)
	{
		const mi::base::Handle<const mi::neuraylib::IType> ElemType(ArrayType->get_element_type());

		const mi::base::Handle<const mi::neuraylib::IType_atomic> AtomicType(ElemType->get_interface<const mi::neuraylib::IType_atomic>());

		FString Result = TypeToString(ElemType.get());
		if (ArrayType->is_immediate_sized())
		{
			Result << FString(TEXT("[")) << (int)ArrayType->get_size() << TEXT("]");
		}
		else
		{
			Result << TEXT("[]");
		}

		return Result;
	}
}

#endif  // #ifdef USE_MDLSDK
