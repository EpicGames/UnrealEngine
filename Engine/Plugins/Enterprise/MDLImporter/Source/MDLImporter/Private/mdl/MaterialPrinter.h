// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "mdl/MaterialTraverser.h"

#include "Containers/Map.h"

namespace mi
{
    namespace neuraylib
    {
        class ICompiled_material;
        class IType_enum;
        class IType_struct;
        class IType_atomic;
        class IType_array;
        class IType_matrix;
        class IType_vector;
        class IMaterial_instance;
        class IMdl_factory;
        class IType;
        class IFunction_definition;
    }
    namespace base
    {
        template <class Interface>
        class Handle;
    }
}
namespace Mdl
{
    class FMaterialPrinter : protected IMaterialTraverser
    {
    public:
        FString Print(const mi::neuraylib::ICompiled_material& Material, mi::neuraylib::ITransaction* Transaction);

        FString Print(const mi::neuraylib::IMaterial_instance& Material,
                      mi::neuraylib::IMdl_factory*             MdlFactory,
                      mi::neuraylib::ITransaction*             Transaction);

    protected:
        using IMaterialTraverser::ETraveralStage;
        using IMaterialTraverser::FTraversalElement;

        void StageBegin(const mi::neuraylib::ICompiled_material& Material, ETraveralStage Stage, mi::neuraylib::ITransaction* Transaction) override;
        void StageEnd(const mi::neuraylib::ICompiled_material& Material, ETraveralStage Stage, mi::neuraylib::ITransaction* Transaction) override;
        void VisitBegin(const mi::neuraylib::ICompiled_material& Material,
                        const FTraversalElement&                 Element,
                        mi::neuraylib::ITransaction*             Transaction) override;
        void VisitChild(const mi::neuraylib::ICompiled_material& Material, const FTraversalElement& Element, uint32 childrenCount, uint32 ChildIndex,
                        mi::neuraylib::ITransaction* Transaction) override;
        void VisitEnd(const mi::neuraylib::ICompiled_material& Material,
                      const FTraversalElement&                 Element,
                      mi::neuraylib::ITransaction*             Transaction) override;

    private:
        // Returns the type of an enum as string.
        FString EnumTypeToString(const mi::neuraylib::IType_enum* EnumType);
        // Returns the type of a struct as string.
        FString StructTypeToString(const mi::neuraylib::IType_struct* StructType, bool* bIsMaterialKeyword = nullptr);
        // Returns the type of an elemental type as string.
        FString AtomicTypeToString(const mi::neuraylib::IType_atomic* AtomicType);
        // Returns a vector type as string.
        FString VectorTypeToString(const mi::neuraylib::IType_vector* VectorType);
        // Returns a matrix type as string.
        FString MatrixTypeToString(const mi::neuraylib::IType_matrix* MatrixType);
        // Returns an array type as string.
        FString ArrayTypeToString(const mi::neuraylib::IType_array* ArrayType);
        // Returns the name of type as string.
        FString TypeToString(const mi::neuraylib::IType* Type);

        void HandleModulesAndImports(const mi::base::Handle<const mi::neuraylib::IFunction_definition>& FuncDef, int Semantic, FString& FunctionName);
        void HandleVisitBeginExpression(const mi::neuraylib::ICompiled_material& Material,
                                        const FTraversalElement&                 Element,
                                        mi::neuraylib::ITransaction*             Transaction);
        void HandleVisitBeginValue(const FTraversalElement& Element, mi::neuraylib::ITransaction* Transaction);

        void HandleVisitChildExpression(const FTraversalElement& Element, uint32 ChildIndex, mi::neuraylib::ITransaction* Transaction);

        void HandleVisitEndExpression(const FTraversalElement& Element, mi::neuraylib::ITransaction* Transaction);
        void HandleVisitEndParameter(const mi::neuraylib::ICompiled_material& Material, const FTraversalElement& Element);

    private:
        FString        TraverseResult;
        ETraveralStage TraversalStage;
        uint32         Indent;

        TSet<FString> Imports;
        TSet<FString> UsedModules;
        TSet<FString> UsedResources;

        // favor compiler created structure (may create invalid mdl)
        TMap<FString, FString> ParametersToInline;
        FString                TraverseInlineSwap;
        uint32                 IndentInlineSwap;

        bool bKeepCompiledStructure;
    };
}
