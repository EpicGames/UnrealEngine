// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef USE_MDLSDK

#include "MaterialTraverser.h"

#include "MdlSdkDefines.h"

#include "Containers/StringConv.h"

MDLSDK_INCLUDES_START
#include "mi/neuraylib/icompiled_material.h"
#include "mi/neuraylib/iexpression.h"
#include "mi/neuraylib/ivalue.h"
#include <mi/base/handle.h>
#include <mi/base/iinterface.h>
#include <mi/base/types.h>
#include <mi/base/uuid.h>
MDLSDK_INCLUDES_END

namespace Mdl
{
    void IMaterialTraverser::Traverse(const mi::neuraylib::ICompiled_material& Material, mi::neuraylib::ITransaction* Transaction)
    {
        mi::base::Handle<const mi::neuraylib::IExpression_direct_call> Body(Material.get_body());

        // parameter stage
        StageBegin(Material, ETraveralStage::Parameters, Transaction);
        const uint32 ParamCount = Material.get_parameter_count();
        for (uint32 Index = 0; Index < ParamCount; ++Index)
        {
            const mi::base::Handle<const mi::neuraylib::IValue> Argugment(Material.get_argument(Index));
            FParameter                                          Parameter(Argugment.get());
            Traverse(Material, FTraversalElement(&Parameter, ParamCount, Index), Transaction);
        }
        StageEnd(Material, ETraveralStage::Parameters, Transaction);

        // temporary stage
        StageBegin(Material, ETraveralStage::Temporaries, Transaction);
        const uint32 TempCount = Material.get_temporary_count();
        for (uint32 Index = 0; Index < TempCount; ++Index)
        {
            const mi::base::Handle<const mi::neuraylib::IExpression> ReferencedExpression(Material.get_temporary(Index));
            FTemporary                                               Temporary(ReferencedExpression.get());
            Traverse(Material, FTraversalElement(&Temporary, TempCount, Index), Transaction);
        }
        StageEnd(Material, ETraveralStage::Temporaries, Transaction);

        // body stage
        StageBegin(Material, ETraveralStage::Body, Transaction);
        Traverse(Material, FTraversalElement(Body.get()), Transaction);
        StageEnd(Material, ETraveralStage::Body, Transaction);

        // mark finished
        StageBegin(Material, ETraveralStage::Finished, Transaction);
    }

    void IMaterialTraverser::Traverse(const mi::neuraylib::ICompiled_material& Material,
                                      const FTraversalElement&                 Element,
                                      mi::neuraylib::ITransaction*             Transaction)
    {
        VisitBegin(Material, Element, Transaction);

        // major cases: parameter, temporary, expression or value
        if (Element.Expression)
        {
            switch (Element.Expression->get_kind())
            {
                case mi::neuraylib::IExpression::EK_CONSTANT:
                {
                    const mi::base::Handle<const mi::neuraylib::IExpression_constant> ExprConstant(
                        Element.Expression->get_interface<const mi::neuraylib::IExpression_constant>());
                    const mi::base::Handle<const mi::neuraylib::IValue> Value(ExprConstant->get_value());

                    Traverse(Material, FTraversalElement(Value.get()), Transaction);
                    break;
                }

                case mi::neuraylib::IExpression::EK_DIRECT_CALL:
                {
                    const mi::base::Handle<const mi::neuraylib::IExpression_direct_call> ExprDcall(
                        Element.Expression->get_interface<const mi::neuraylib::IExpression_direct_call>());
                    const mi::base::Handle<const mi::neuraylib::IExpression_list> Arguments(ExprDcall->get_arguments());

                    const uint32 ArgCount = Arguments->get_size();
                    for (uint32 ArgIndex = 0; ArgIndex < ArgCount; ++ArgIndex)
                    {
                        mi::base::Handle<const mi::neuraylib::IExpression> Expression(Arguments->get_expression(ArgIndex));

                        VisitChild(Material, Element, ArgCount, ArgIndex, Transaction);
                        Traverse(Material, FTraversalElement(Expression.get(), ArgCount, ArgIndex), Transaction);
                    }
                    break;
                }

                case mi::neuraylib::IExpression::EK_PARAMETER:     // nothing special to do
                case mi::neuraylib::IExpression::EK_TEMPORARY:     // nothing special to do
                case mi::neuraylib::IExpression::EK_CALL:          // will not happen for compiled materials
                case mi::neuraylib::IExpression::EK_FORCE_32_BIT:  // not a valid value
                    break;
                default:
                    break;
            }
        }
        // major cases: parameter, temporary, expression or value
        else if (Element.Value)
        {
            switch (Element.Value->get_kind())
            {
                case mi::neuraylib::IValue::VK_BOOL:           // Intended fallthrough
                case mi::neuraylib::IValue::VK_INT:            // Intended fallthrough
                case mi::neuraylib::IValue::VK_FLOAT:          // Intended fallthrough
                case mi::neuraylib::IValue::VK_DOUBLE:         // Intended fallthrough
                case mi::neuraylib::IValue::VK_STRING:         // Intended fallthrough
                case mi::neuraylib::IValue::VK_ENUM:           // Intended fallthrough
                case mi::neuraylib::IValue::VK_INVALID_DF:     // Intended fallthrough
                case mi::neuraylib::IValue::VK_TEXTURE:        // Intended fallthrough
                case mi::neuraylib::IValue::VK_LIGHT_PROFILE:  // Intended fallthrough
                case mi::neuraylib::IValue::VK_BSDF_MEASUREMENT:
                    break;  // nothing to do here

                // the following values have children
                case mi::neuraylib::IValue::VK_VECTOR:  // Intended fallthrough
                case mi::neuraylib::IValue::VK_MATRIX:  // Intended fallthrough
                case mi::neuraylib::IValue::VK_COLOR:   // Intended fallthrough
                case mi::neuraylib::IValue::VK_ARRAY:   // Intended fallthrough
                case mi::neuraylib::IValue::VK_STRUCT:
                {
                    const mi::base::Handle<const mi::neuraylib::IValue_compound> ValueCompound(
                        Element.Value->get_interface<const mi::neuraylib::IValue_compound>());

                    uint32 CompoundSize = ValueCompound->get_size();
                    for (uint32 Index = 0; Index < CompoundSize; ++Index)
                    {
                        const mi::base::Handle<const mi::neuraylib::IValue> CompoundElement(ValueCompound->get_value(Index));

                        VisitChild(Material, Element, CompoundSize, Index, Transaction);
                        Traverse(Material, FTraversalElement(CompoundElement.get(), CompoundSize, Index), Transaction);
                    }
                    break;
                }

                case mi::neuraylib::IValue::VK_FORCE_32_BIT:
                    break;  // not a valid value
                default:
                    break;
            }
        }
        // major cases: parameter, temporary, expression or value
        else if (Element.Parameter)
        {
            Traverse(Material, FTraversalElement(Element.Parameter->Value), Transaction);
        }
        // major cases: parameter, temporary, expression or value
        else if (Element.Temporary)
        {
            Traverse(Material, FTraversalElement(Element.Temporary->Expression), Transaction);
        }

        VisitEnd(Material, Element, Transaction);
    }

    FString IMaterialTraverser::GetParameterName(const mi::neuraylib::ICompiled_material& Material, uint32 Index, bool* bWasGenerated) const
    {
        FString Name = UTF8_TO_TCHAR(Material.get_parameter_name(Index));

        // dots in parameter names are not allowed in mdl, so these are the compiler generated ones.
        if (bWasGenerated)
        {
            int32 Found    = 0;
            *bWasGenerated = Name.FindChar('.', Found);
        }
        return Name;
    }
}

#endif  // #ifdef USE_MDLSDK
