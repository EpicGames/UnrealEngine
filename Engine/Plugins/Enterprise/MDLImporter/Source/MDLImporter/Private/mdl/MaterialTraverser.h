// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

struct FScriptContainerElement;

namespace mi
{
    namespace neuraylib
    {
        class ICompiled_material;
        class ITransaction;
        class IValue;
        class IExpression;
    }
}
namespace Mdl
{
    /**
     * A base class that implements traversal logic for compiled materials.
     */
    class IMaterialTraverser
    {
    public:
        virtual ~IMaterialTraverser() {}

    protected:
        /**
         * Possible stages of the traversal of an compiled material
         */
        enum class ETraveralStage
        {
            Start = 0,
            // Indicates that the parameters of the material are currently traversed
            Parameters,
            // Indicates that the temporaries of the material are currently traversed
            Temporaries,
            // Indicates that the main body of the material are currently traversed
            Body,
            // Traversal is done
            Finished,
        };

        /**
         * An internal structure that is passed to the user code while traversing.
         * This struct is used while visiting the material parameters.
         */
        struct FParameter
        {
            explicit FParameter(const mi::neuraylib::IValue* Value)
                : Value(Value)
            {
            }

            const mi::neuraylib::IValue* Value;
        };

        /**
         * An internal structure that is passed to the user code while traversing.
         * This struct is used while visiting the materials temporaries.
         */
        struct FTemporary
        {
            explicit FTemporary(const mi::neuraylib::IExpression* Expression)
                : Expression(Expression)
            {
            }

            const mi::neuraylib::IExpression* Expression;
        };

        /**
         * Encapsulated to current element that is visited during the traversal
         * It contains either an IExpression, an IValue, a Parameter or a Temporary while the
         * others are nullptr.
         */
        struct FTraversalElement
        {
            explicit FTraversalElement(const mi::neuraylib::IExpression* Expression, uint32 SiblingCount = 1, uint32 SiblingIndex = 0);
            explicit FTraversalElement(const mi::neuraylib::IValue* Value, uint32 SiblingCount = 1, uint32 SiblingIndex = 0);
            explicit FTraversalElement(const FParameter* Parameter, uint32 SiblingCount = 1, uint32 SiblingIndex = 0);
            explicit FTraversalElement(const FTemporary* Temporary, uint32 SiblingCount = 1, uint32 SiblingIndex = 0);

            // Not nullptr if the current traversal element is an IExpression.
            const mi::neuraylib::IExpression* Expression;
            // Not nullptr if the current traversal element is an IValue.
            const mi::neuraylib::IValue* Value;
            // Not nullptr if the current traversal element is a Parameter.
            // This can happen only in the ES_PARAMETERS stage.
            const FParameter* Parameter;
            // Not nullptr if the current traversal element is a Parameter.
            // This can happen only in the ES_TEMPORARAY stage.
            const FTemporary* Temporary;
            // Total number of children at the parent of the currently traversed element.
            uint32 SiblingCount;
            // Index of the currently traversed element in the list of children at the parent.
            uint32 SiblingIndex;
        };

    protected:
        IMaterialTraverser() {}

        /**
         * Traverses a compiled material and calls the corresponding virtual visit methods.
         * This method is meant to be called by deriving class to start the actual traversal.
         *
         * @param Material - The material that is traversed.
         * @param Transaction - The transaction from which to access material data.
         */
        void Traverse(const mi::neuraylib::ICompiled_material& Material, mi::neuraylib::ITransaction* Transaction);

        /**
         * Called at the beginning of each traversal stage: Parameters, Temporaries and Body.
         *
         * @param Material - The material that is traversed.
         * @param Stage - The stage that was entered.
         * @param Transaction - The transaction from which to access material data.
         */
        virtual void StageBegin(const mi::neuraylib::ICompiled_material& Material,
                                ETraveralStage                           Stage,
                                mi::neuraylib::ITransaction*             Transaction) = 0;

        /**
         * Called at the end of each traversal stage: Parameters, Temporaries and Body.
         *
         * @param Material - The material that is traversed.
         * @param Stage - The stage that was finisheS.
         * @param Transaction - The transaction from which to access material data.
         */
        virtual void StageEnd(const mi::neuraylib::ICompiled_material& Material, ETraveralStage Stage, mi::neuraylib::ITransaction* Transaction) = 0;

        /**
         * Called when the traversal reaches a new element.
         *
         * @param Material - The material that is traversed.
         * @param Element - The element that was reached.
         * @param Transaction - The transaction from which to access material data.
         */
        virtual void VisitBegin(const mi::neuraylib::ICompiled_material& Material,
                                const FTraversalElement&                 Element,
                                mi::neuraylib::ITransaction*             Transaction) = 0;

        /**
         * Occurs only if the current element has multiple child elements, e.g., a function call.
         * In that case, the method is called before each of the children are traversed, e.g.,
         * before each argument of a function call.
         *
         * @param Material - The material that is traversed.
         * @param Element - The currently traversed element with multiple children.
         * @param ChildrenCount - Number of children of the current element.
         * @param ChildIndex - The index of the child that will be traversed next.
         * @param Transaction - The transaction from which to access material data.
         */
        virtual void VisitChild(const mi::neuraylib::ICompiled_material& Material, const FTraversalElement& Element, uint32 ChildrenCount,
                                uint32 ChildIndex, mi::neuraylib::ITransaction* Transaction) = 0;

        /**
         * Called when the traversal reaches finishes an element.
         *
         * @param Material - The material that is traversed.
         * @param Element - The element that was finished.
         * @param Transaction - The transaction from which to access material data.
         */
        virtual void VisitEnd(const mi::neuraylib::ICompiled_material& Material,
                              const FTraversalElement&                 Element,
                              mi::neuraylib::ITransaction*             Transaction) = 0;

        /**
         * Gets the name of a parameter of the traversed material.
         *
         * @param  Material - The material that is traversed.
         * @param  Index - Index of the parameter in the materials parameter list.
         * @param  bWasGenerated - Optional output parameter that indicates whether the parameter
         * was generated by the compiler rather than defined in the material definition.
         * @return The parameter name.
         */
        FString GetParameterName(const mi::neuraylib::ICompiled_material& Material, uint32 Index, bool* bWasGenerated = nullptr) const;

        /**
         * Gets the name of a temporary of the traversed material.
         * Since the name is usually unknown, due to optimization, a proper name is generated.
         *
         * @param  Index - Index of the parameter in the materials temporary list.
         * @return The temporary name.
         */
        FString GetTemporaryName(uint32 Index) const;

    private:
        /**
         * Recursive function that is used for the actual traversal.
         * The names of templates are lost during compilation. Therefore, we generate numbered ones.
         *
         * @param Material - The material that is traversed..
         * @param Element - The element that is currently visited.
         * @param Transaction - The transaction from which to access material data.
         */
        void Traverse(const mi::neuraylib::ICompiled_material& Material, const FTraversalElement& Element, mi::neuraylib::ITransaction* Transaction);
    };

    inline IMaterialTraverser::FTraversalElement::FTraversalElement(const mi::neuraylib::IExpression* Expression, uint32 SiblingCount /*= 1*/,
                                                                    uint32 SiblingIndex /*= 0*/)
        : Expression(Expression)
        , Value(nullptr)
        , Parameter(nullptr)
        , Temporary(nullptr)
        , SiblingCount(SiblingCount)
        , SiblingIndex(SiblingIndex)
    {
    }

    inline IMaterialTraverser::FTraversalElement::FTraversalElement(const mi::neuraylib::IValue* Value, uint32 SiblingCount /*= 1*/,
                                                                    uint32 SiblingIndex /*= 0*/)
        : Expression(nullptr)
        , Value(Value)
        , Parameter(nullptr)
        , Temporary(nullptr)
        , SiblingCount(SiblingCount)
        , SiblingIndex(SiblingIndex)
    {
    }

    inline IMaterialTraverser::FTraversalElement::FTraversalElement(const FParameter* Parameter, uint32 SiblingCount /*= 1*/,
                                                                    uint32 SiblingIndex /*= 0*/)
        : Expression(nullptr)
        , Value(nullptr)
        , Parameter(Parameter)
        , Temporary(nullptr)
        , SiblingCount(SiblingCount)
        , SiblingIndex(SiblingIndex)
    {
    }

    inline IMaterialTraverser::FTraversalElement::FTraversalElement(const FTemporary* Temporary, uint32 SiblingCount /*= 1*/,
                                                                    uint32 SiblingIndex /*= 0*/)
        : Expression(nullptr)
        , Value(nullptr)
        , Parameter(nullptr)
        , Temporary(Temporary)
        , SiblingCount(SiblingCount)
        , SiblingIndex(SiblingIndex)
    {
    }

    inline FString IMaterialTraverser::GetTemporaryName(uint32 Index) const
    {
        return FString("temporary_") + FString::FromInt(Index);
    }
}
