// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef USE_MDLSDK

#include "Common.h"


namespace Mdl
{
	const TCHAR* ToString(EValueType Type)
	{
		// must match MDL SDK string values
		switch (Type)
		{
			case Mdl::EValueType::Float:
				return TEXT("Float32");
			case Mdl::EValueType::Float2:
				return TEXT("Float32<2>");
			case Mdl::EValueType::Float3:
				return TEXT("Float32<3>");
			case Mdl::EValueType::ColorRGB:
				return TEXT("Rgb_fp");
			case Mdl::EValueType::ColorRGBA:
				return TEXT("Color");
			default:
				checkSlow(false);
				return TEXT("Float32");
		}
	}

	const TCHAR* ToString(EParameterType Type)
	{
		switch (Type)
		{
			case Mdl::EParameterType::BaseColor:
				return TEXT("BaseColor");
			case Mdl::EParameterType::Metallic:
				return TEXT("Metallic");
			case Mdl::EParameterType::Specular:
				return TEXT("Specular");
			case Mdl::EParameterType::Roughness:
				return TEXT("Roughness");
			case Mdl::EParameterType::Opacity:
				return TEXT("Opacity");
			case Mdl::EParameterType::Normal:
				return TEXT("Normal");
			case Mdl::EParameterType::Displacement:
				return TEXT("Displacement");
			case Mdl::EParameterType::ClearcoatWeight:
				return TEXT("ClearcoatWeight");
			case Mdl::EParameterType::ClearcoatRoughness:
				return TEXT("ClearcoatRoughness");
			case Mdl::EParameterType::ClearcoatNormal:
				return TEXT("ClearcoatNormal");
			case Mdl::EParameterType::IOR:
				return TEXT("IOR");
			case Mdl::EParameterType::VolumeAbsorption:
				return TEXT("VolumeAbsorption");
			case Mdl::EParameterType::VolumeScattering:
				return TEXT("VolumeScattering");
			case Mdl::EParameterType::Emission:
				return TEXT("Emission");
			case Mdl::EParameterType::Count:
				return TEXT("Count");
			default:
				return TEXT("Unknown");
		}
	}

	uint32 ComponentCount(EValueType Type)
	{
		switch (Type)
		{
			case Mdl::EValueType::Float:
				return 1;
			case Mdl::EValueType::Float2:
				return 2;
			case Mdl::EValueType::Float3:
			case Mdl::EValueType::ColorRGB:
				return 3;
			case Mdl::EValueType::ColorRGBA:
				return 4;
			default:
				checkSlow(false);
				return 0;
		}
	}
}

#endif  // #ifdef USE_MDLSDK
