// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Special header

#if PLATFORM_WINDOWS

#ifndef MDLSDK_INCLUDES_START
#define MDLSDK_INCLUDES_START \
	THIRD_PARTY_INCLUDES_START \
	__pragma(warning(push)) \
	__pragma(warning(disable: 4456))  /* declaration of 'v' hides previous local declaration */ 
#endif // MDLSDK_INCLUDES_START

#ifndef MDLSDK_INCLUDES_END
#define MDLSDK_INCLUDES_END \
	__pragma(warning(pop)) \
	THIRD_PARTY_INCLUDES_END
#endif // MDLSDK_INCLUDE_END

#else

#ifndef MDLSDK_INCLUDES_START
#define MDLSDK_INCLUDES_START \
	THIRD_PARTY_INCLUDES_START 
#endif // MDLSDK_INCLUDES_START

#ifndef MDLSDK_INCLUDES_END
#define MDLSDK_INCLUDES_END 
	THIRD_PARTY_INCLUDES_END
#endif // MDLSDK_INCLUDE_END

#endif

