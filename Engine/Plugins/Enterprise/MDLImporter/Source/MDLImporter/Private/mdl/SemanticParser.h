// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "mdl/Utility.h"


#include "mi/base/handle.h"
#include "mi/neuraylib/ifunction_definition.h"
#include <mi/base/iinterface.h>
#include <mi/base/types.h>
#include <mi/neuraylib/iexpression.h>

namespace Mdl
{
	struct FSemanticParser
	{
		FString                                                        PathPrefix;
		mi::base::Handle<const mi::neuraylib::IExpression_direct_call> ParentCall;
		mi::neuraylib::IFunction_definition::Semantics                 Semantic;
		mi::neuraylib::ITransaction*                                   Transaction;
		const mi::neuraylib::ICompiled_material*                       Material;

		FSemanticParser(const char* BasePath, mi::neuraylib::ITransaction* Transaction, const mi::neuraylib::ICompiled_material* Material)
		    : ParentCall(Lookup::GetCall(BasePath, Material))
		    , Semantic((mi::neuraylib::IFunction_definition::Semantics)Lookup::GetSemantic(Transaction, ParentCall.get()))
		    , Transaction(Transaction)
		    , Material(Material)
		{
			PathPrefix = ANSI_TO_TCHAR(BasePath);
			PathPrefix.Append(TEXT("."));
		}

		void SetNextCall(const char* SubPath)
		{
			ParentCall = Lookup::GetCall(SubPath, Material, ParentCall.get());
			Semantic   = (mi::neuraylib::IFunction_definition::Semantics)Lookup::GetSemantic(Transaction, ParentCall.get());
			PathPrefix.Append(ANSI_TO_TCHAR(SubPath));
			PathPrefix.Append(TEXT("."));
		}
	};
}
