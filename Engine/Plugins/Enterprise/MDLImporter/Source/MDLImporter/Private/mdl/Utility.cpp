// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef USE_MDLSDK

#include "Utility.h"

#include "Common.h"
#include "MdlSdkDefines.h"

#include "Misc/Paths.h"

MDLSDK_INCLUDES_START
#include "mi/neuraylib/icanvas.h"
#include "mi/neuraylib/icompiled_material.h"
#include "mi/neuraylib/ifunction_definition.h"
#include "mi/neuraylib/iimage.h"
#include "mi/neuraylib/iimage_api.h"
#include "mi/neuraylib/imaterial_definition.h"
#include "mi/neuraylib/imaterial_instance.h"
#include "mi/neuraylib/imdl_factory.h"
#include "mi/neuraylib/ineuray.h"
#include "mi/neuraylib/itexture.h"
#include "mi/neuraylib/itile.h"
#include "mi/neuraylib/itransaction.h"
#include "mi/neuraylib/imdl_execution_context.h"
#include <mi/base/handle.h>
#include <mi/base/iinterface.h>
#include <mi/base/types.h>
#include <mi/base/uuid.h>
#include <mi/neuraylib/iexpression.h>
MDLSDK_INCLUDES_END

namespace Mdl
{
	namespace
	{
		const char* GetPixelType(int Channels)
		{
			// always float
			switch (Channels)
			{
				case 1:
					return "Float32";
				case 2:
					return "Float32<2>";
				case 3:
					return "Rgb_fp";
				case 4:
					return "Color";
				default:
					break;
			}
			UE_LOGF(LogMDLImporter, Error, "Unsupported channel count %d", Channels);
			return nullptr;
		}
	}
	namespace Util
	{
		FString GetTextureFileName(const mi::neuraylib::ITexture* Texture)
		{
			check(Texture);
			FString FileName = ANSI_TO_TCHAR(Texture->get_image());
			FileName.RemoveFromStart("MI_default_image_");
			FileName.RemoveFromEnd("_");
			FPaths::NormalizeDirectoryName(FileName);
			return FileName;
		}

		bool CreateMaterialInstance(mi::neuraylib::ITransaction* Transaction, const FString& MaterialDbName, const FString& MaterialInstanceDbName)
		{
			mi::base::Handle<const mi::neuraylib::IMaterial_definition> MaterialCheck(
			    Transaction->access<mi::neuraylib::IMaterial_definition>(TCHAR_TO_ANSI(*MaterialInstanceDbName)));
			if (MaterialCheck)
			{
				UE_LOGF(LogMDLImporter, Log, "Material instance was already created %ls", *MaterialInstanceDbName);
				// already created
				return false;
			}

			// Create and store the new material instance

			mi::base::Handle<const mi::neuraylib::IMaterial_definition> MaterialDefinition(
			    Transaction->access<mi::neuraylib::IMaterial_definition>(TCHAR_TO_ANSI(*MaterialDbName)));
			if (!MaterialDefinition)
			{
				return false;
			}

			mi::Sint32                                          Result;
			mi::base::Handle<mi::neuraylib::IMaterial_instance> MaterialInstance(MaterialDefinition->create_material_instance(nullptr, &Result));
			MDL_CHECK_RESULT() = Result;

			MDL_CHECK_RESULT() = Transaction->store(MaterialInstance.get(), TCHAR_TO_ANSI(*MaterialInstanceDbName));

			return true;
		}

		bool CompileMaterialInstance(mi::neuraylib::ITransaction* Transaction,
									 mi::neuraylib::INeuray*	  Neuray,
		                             const FString&               MaterialInstanceDbName,
		                             const FString&               MaterialCompiledDbName,
		                             bool                         bIsClassCompilation,
		                             float                        MetersPerSceneUnit)
		{
			mi::base::Handle<const mi::neuraylib::IMaterial_instance> MaterialInstance(
			    Transaction->access<mi::neuraylib::IMaterial_instance>(TCHAR_TO_ANSI(*MaterialInstanceDbName)));
			if (!MaterialInstance)
			{
				return false;
			}

			mi::Uint32 Flags =
			    bIsClassCompilation ? mi::neuraylib::IMaterial_instance::CLASS_COMPILATION : mi::neuraylib::IMaterial_instance::DEFAULT_OPTIONS;

			mi::neuraylib::IMdl_factory* MdlFactory = Neuray->get_api_component<mi::neuraylib::IMdl_factory>();
			mi::base::Handle<mi::neuraylib::IMdl_execution_context> Context(MdlFactory->create_execution_context());
			Context->set_option(TCHAR_TO_ANSI(TEXT("mdl_meters_per_scene_unit")), MetersPerSceneUnit);
			//Context->set_option(TCHAR_TO_ANSI(TEXT("mdl_wavelength_min")), WavelengthMin); The default is already 380
			//Context->set_option(TCHAR_TO_ANSI(TEXT("mdl_wavelength_max")), WavelengthMax); The default is already 780

			mi::base::Handle<mi::neuraylib::ICompiled_material> CompiledMaterial(
			    MaterialInstance->create_compiled_material(Flags, Context.get()));

			uint64 ErrorCount = Context->get_error_messages_count();
			if (ErrorCount != 0)
			{
				UE_LOGF(LogMDLImporter, Error, "There were errors creating compiled material '%ls':", *MaterialInstanceDbName);
				for (uint64 Index = 0; Index < ErrorCount; Index++)
				{
					const mi::neuraylib::IMessage* Message = Context->get_error_message(Index);
					if (Message)
					{
						UE_LOGF(LogMDLImporter, Error, "Error %llu: '%ls'", Index, ANSI_TO_TCHAR(Message->get_string()));
					}
				}
				return false;
			}

			// Store the compiled material instance
			MDL_CHECK_RESULT() = Transaction->store(CompiledMaterial.get(), TCHAR_TO_ANSI(*MaterialCompiledDbName));

			return true;
		}

		mi::neuraylib::ICanvas* CreateCanvas(int Channels, int Width, int Height, mi::neuraylib::INeuray* Neuray, float Gamma /*= 1.f*/)
		{
			mi::neuraylib::IImage_api* ImageApi  = Neuray->get_api_component<mi::neuraylib::IImage_api>();
			const char*                PixelType = GetPixelType(Channels);
			mi::neuraylib::ICanvas*    Canvas    = ImageApi->create_canvas(PixelType, Width, Height, 1, false, Gamma);

			return Canvas;
		}

		mi::neuraylib::ICanvas* CreateCanvas(
		    int Channels, int Width, int Height, const void* Src, bool bFlipY, mi::neuraylib::INeuray* Neuray, float Gamma /*= 1.f*/)
		{
			mi::neuraylib::IImage_api* ImageApi  = Neuray->get_api_component<mi::neuraylib::IImage_api>();
			const char*                PixelType = GetPixelType(Channels);
			mi::neuraylib::ICanvas*    Canvas    = ImageApi->create_canvas(PixelType, Width, Height, 1, false, Gamma);
			MDL_CHECK_RESULT()                   = ImageApi->write_raw_pixels(Width, Height, Canvas, 0, 0, 0, Src, bFlipY, PixelType);

			return Canvas;
		}

		FString CreateTexture(mi::neuraylib::ITransaction* Transaction,
		                      const FString&               TextureDbName,
		                      mi::neuraylib::ICanvas*      Canvas,
		                      bool                         bIsShared /*= false*/)
		{
			// create and store image for texture with the given canvas
			mi::base::Handle<mi::neuraylib::IImage> Image(Transaction->create<mi::neuraylib::IImage>("Image"));
			MDL_CHECK_RESULT()        = Image->set_from_canvas(Canvas, bIsShared) == false;
			const FString ImageDbName = TextureDbName + TEXT("_image");
			MDL_CHECK_RESULT()        = Transaction->store(Image.get(), TCHAR_TO_ANSI(*ImageDbName));

			// create and store texture
			mi::base::Handle<mi::neuraylib::ITexture> Texture(Transaction->create<mi::neuraylib::ITexture>("Texture"));
			MDL_CHECK_RESULT()                = Texture->set_image(TCHAR_TO_ANSI(*ImageDbName));
			const FString ResultTextureDbName = TextureDbName + TEXT("_texture");
			MDL_CHECK_RESULT()                = Transaction->store(Texture.get(), TCHAR_TO_ANSI(*ResultTextureDbName));

			return ResultTextureDbName;
		}

		int GetChannelCount(const mi::neuraylib::ICanvas& Canvas)
		{
			mi::base::Handle<const mi::neuraylib::ITile> Tile(Canvas.get_tile());

			const FString Type = ANSI_TO_TCHAR(Tile->get_type());
			if (Type == TEXT("Float32"))
				return 1;
			if (Type == TEXT("Float32<2>"))
				return 2;
			if (Type == TEXT("Rgb_fp") || Type == TEXT("Float32<3>"))
				return 3;
			if (Type == TEXT("Color") || Type == TEXT("Float32<4>"))
				return 4;
			check(false);
			return 0;
		}
	}

	namespace Lookup
	{
		// Converts the given expression to a direct_call, thereby resolving temporaries.
		// Returns nullptr if the passed expression is not a direct call.
		const mi::neuraylib::IExpression_direct_call* GetDirectCall(const mi::neuraylib::IExpression*        Expression,
		                                                            const mi::neuraylib::ICompiled_material* Material)
		{
			if (!Expression)
			{
				return nullptr;
			}

			switch (Expression->get_kind())
			{
				case mi::neuraylib::IExpression::EK_DIRECT_CALL:
					return Expression->get_interface<mi::neuraylib::IExpression_direct_call>();
				case mi::neuraylib::IExpression::EK_TEMPORARY:
				{
					mi::base::Handle<const mi::neuraylib::IExpression_temporary> ExpressionTemp(
					    Expression->get_interface<const mi::neuraylib::IExpression_temporary>());
					mi::base::Handle<const mi::neuraylib::IExpression_direct_call> ResultCall(
					    Material->get_temporary<const mi::neuraylib::IExpression_direct_call>(ExpressionTemp->get_index()));

					ResultCall->retain();
					return ResultCall.get();
				}
				default:
					break;
			}
			return nullptr;
		}

		// Returns the argument 'argument_name' of the given call.
		const mi::neuraylib::IExpression_direct_call* GetCallFromArgument(const mi::neuraylib::ICompiled_material*      Material,
		                                                                  const mi::neuraylib::IExpression_direct_call* Call,
		                                                                  const char*                                   ArgumentName)
		{
			if (!Call)
			{
				return nullptr;
			}

			mi::base::Handle<const mi::neuraylib::IExpression_list> CallArguments(Call->get_arguments());
			mi::base::Handle<const mi::neuraylib::IExpression>      Arg(CallArguments->get_expression(ArgumentName));
			return GetDirectCall(Arg.get(), Material);
		}

		// Looks up the sub expression 'path' within the compiled material starting at parent_call.
		// If parent_call is nullptr, the  material will be traversed from the root.
		const mi::neuraylib::IExpression_direct_call* GetCall(const char*                                   Path,
		                                                      const mi::neuraylib::ICompiled_material*      Material,
		                                                      const mi::neuraylib::IExpression_direct_call* ParentCall /*= nullptr*/)
		{
			mi::base::Handle<const mi::neuraylib::IExpression_direct_call> ResultCall;

			if (ParentCall == nullptr)
			{
				mi::base::Handle<const mi::neuraylib::IExpression> Expression(Material->lookup_sub_expression(Path));
				ResultCall = GetDirectCall(Expression.get(), Material);
			}
			else
			{
				ResultCall = mi::base::make_handle_dup(ParentCall);

				TArray<FString> Paths;
				const FString   RemainingPath = Path;
				RemainingPath.ParseIntoArray(Paths, TEXT("."), true);
				for (const FString& Arg : Paths)
				{
					ResultCall = GetCallFromArgument(Material, ResultCall.get(), TCHAR_TO_ANSI(*Arg));
					if (!ResultCall)
					{
						return nullptr;
					}
				}
			}
			if (!ResultCall)
			{
				return nullptr;
			}

			ResultCall->retain();
			return ResultCall.get();
		}

		// Returns the semantic of the function definition which corresponds to
		// the given call or DS::UNKNOWN in case the expression is nullptr.
		int GetSemantic(mi::neuraylib::ITransaction* Transaction, const mi::neuraylib::IExpression_direct_call* Call)
		{
			if (!Call)
			{
				return mi::neuraylib::IFunction_definition::DS_UNKNOWN;
			}

			mi::base::Handle<const mi::neuraylib::IFunction_definition> FunctionDef(
			    Transaction->access<const mi::neuraylib::IFunction_definition>(Call->get_definition()));
			check(FunctionDef.is_valid_interface());

			return FunctionDef->get_semantic();
		}

		FString GetValidSubExpression(const char* SubExpression, const mi::neuraylib::ICompiled_material* Material)
		{
			if (!SubExpression)
			{
				return FString();
			}

			mi::base::Handle<const mi::neuraylib::IExpression> Emission(Material->lookup_sub_expression(SubExpression));
			if (Emission.is_valid_interface())
			{
				return ANSI_TO_TCHAR(SubExpression);
			}
			return FString();
		}

		FString GetCallDefinition(const char* SubExpression, const mi::neuraylib::ICompiled_material* Material)
		{
			if (!SubExpression)
			{
				return FString();
			}

			mi::base::Handle<const mi::neuraylib::IExpression> Expression(Material->lookup_sub_expression(SubExpression));

			if (Expression.is_valid_interface() && Expression->get_kind() == mi::neuraylib::IExpression::EK_DIRECT_CALL)
			{
				mi::base::Handle<const mi::neuraylib::IExpression_direct_call> Call(
				    Expression->get_interface<mi::neuraylib::IExpression_direct_call>());
				return UTF8_TO_TCHAR(Call->get_definition());
			}
			return FString();
		}

		const mi::neuraylib::IExpression* GetArgument(const mi::neuraylib::ICompiled_material*      Material,
		                                              const mi::neuraylib::IExpression_direct_call* ParentCall,
		                                              const char*                                   ArgumentName)
		{
			if (!ParentCall)
			{
				return mi::base::make_handle<const mi::neuraylib::IExpression>(nullptr).get();
			}
			mi::base::Handle<const mi::neuraylib::IExpression_list> CallArgs(ParentCall->get_arguments());

			const mi::neuraylib::IExpression* Arg = CallArgs->get_expression(ArgumentName);
			return Arg;
		}
	}
}

#endif  // #ifdef USE_MDLSDK
