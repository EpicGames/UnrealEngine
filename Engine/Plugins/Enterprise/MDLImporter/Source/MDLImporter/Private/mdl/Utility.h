// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringConv.h"
#include "Containers/UnrealString.h"

namespace mi
{
	namespace neuraylib
	{
		class ITransaction;
		class IExpression;
		class IExpression_direct_call;
		class ICompiled_material;
		class IExpression;
		class IMdl_factory;
		class IImage_api;
		class ICanvas;
		class ITexture;
		class INeuray;
	}
}

namespace Mdl
{
	namespace Util
	{
		FString GetMaterialDatabaseName(const FString& ModuleName, const FString& MaterialName, bool bIgnorePrefix = false);
		FString GetMaterialInstanceName(const FString& MaterialDbName);
		FString GetModuleName(const FString& MaterialDbName);
		FString GetTextureFileName(const mi::neuraylib::ITexture* Texture);

		/**
		 * Create a material instance from the material definition with the default arguments.
		 * @return true if a new material instance was created, false if was previously created.
		 */
		bool CreateMaterialInstance(mi::neuraylib::ITransaction* Transaction, const FString& MaterialDbName, const FString& MaterialInstanceDbName);

		/**
		 * Compile a new material instance and stores it.
		 */
		bool CompileMaterialInstance(mi::neuraylib::ITransaction* Transaction,
									 mi::neuraylib::INeuray*	  Neuray,
		                             const FString&               MaterialInstanceDbName,
		                             const FString&               MaterialCompiledDbName,
		                             bool                         bIsClassCompilation,
		                             float                        MetersPerSceneUnit);

		/**
		 * Creates a image canvas with the size.
		 */
		mi::neuraylib::ICanvas* CreateCanvas(int Channels, int Width, int Height, mi::neuraylib::INeuray* Neuray, float Gamma = 1.f);

		/**
		 * Creates a image canvas with the given buffer data.
		 */
		mi::neuraylib::ICanvas* CreateCanvas(
		    int Channels, int Width, int Height, const void* Src, bool bFlipY, mi::neuraylib::INeuray* Neuray, float Gamma = 1.f);

		/**
		 * Creates a texture in the database with the given canvas.
		 * @return the texture's DB name.
		 */
		FString CreateTexture(mi::neuraylib::ITransaction* Transaction,
		                      const FString&               TextureDbName,
		                      mi::neuraylib::ICanvas*      Canvas,
		                      bool                         bIsShared = false);

		int GetChannelCount(const mi::neuraylib::ICanvas& Canvas);
	}
	namespace Lookup
	{
		/**
		 * Looks up the sub expression 'path' within the compiled material starting at parent_call.
		 * If parent_call is nullptr, the  material will be traversed from the root.
		 */
		const mi::neuraylib::IExpression_direct_call* GetCall(const char*                                   Path,
		                                                      const mi::neuraylib::ICompiled_material*      Material,
		                                                      const mi::neuraylib::IExpression_direct_call* ParentCall = nullptr);

		/**
		 * Returns the semantic of the function definition which corresponds to
		 * the given call or DS::UNKNOWN in case the expression is nullptr.
		 */
		int GetSemantic(mi::neuraylib::ITransaction* Transaction, const mi::neuraylib::IExpression_direct_call* Call);

		const mi::neuraylib::IExpression* GetArgument(const mi::neuraylib::ICompiled_material*      Material,
		                                              const mi::neuraylib::IExpression_direct_call* ParentCall,
		                                              const char*                                   ArgumentName);

		FString GetValidSubExpression(const char* SubExpression, const mi::neuraylib::ICompiled_material* Material);
		FString GetValidSubExpression(const FString& SubExpression, const mi::neuraylib::ICompiled_material* Material);

		FString GetCallDefinition(const char* SubExpression, const mi::neuraylib::ICompiled_material* Material);
		FString GetCallDefinition(const FString& SubExpression, const mi::neuraylib::ICompiled_material* Material);
	}

	//

	namespace Util
	{
		inline FString GetMaterialDatabaseName(const FString& ModuleName, const FString& MaterialName, bool bIgnorePrefix)
		{
			if (bIgnorePrefix)
				return TEXT("::") + ModuleName + TEXT("::") + MaterialName;
			return TEXT("mdl::") + ModuleName + TEXT("::") + MaterialName;
		}

		inline FString GetMaterialInstanceName(const FString& MaterialDbName)
		{
			return MaterialDbName + TEXT("_instanced");
		}

		inline FString GetModuleName(const FString& MaterialDbName)
		{
			FString ModuleName = MaterialDbName;
			ModuleName.RemoveAt(0, 2);
			ModuleName = ModuleName.Mid(0, ModuleName.Find(TEXT("::")));
			return ModuleName;
		}
	}

	namespace Lookup
	{
		inline FString GetValidSubExpression(const FString& SubExpression, const mi::neuraylib::ICompiled_material* Material)
		{
			return GetValidSubExpression(SubExpression.IsEmpty() ? nullptr : TCHAR_TO_ANSI(*SubExpression), Material);
		}

		inline FString GetCallDefinition(const FString& SubExpression, const mi::neuraylib::ICompiled_material* Material)
		{
			return GetCallDefinition(SubExpression.IsEmpty() ? nullptr : TCHAR_TO_ANSI(*SubExpression), Material);
		}
	}
}
