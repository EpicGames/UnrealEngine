// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISettingsModule.h"
#include "Modules/ModuleManager.h"

class FMDLImporter;
class IMdlFileImporter;

class MDLIMPORTER_API IMDLImporterModule : public IModuleInterface
{
public:
	static const TCHAR* ModuleName;

	/**
	 * Access to the internal MDL importer.
	 *
	 * @return Returns internal importer logic.
	 */
	virtual FMDLImporter& GetMDLImporter() = 0;

	virtual TUniquePtr<IMdlFileImporter> CreateFileImporter() = 0;

	virtual bool IsLoaded() = 0;

	/**
	 * Singleton-like access to MDLImporter
	 *
	 * @return Returns MDLImporter singleton instance, loading the module on demand if needed
	 */
	static IMDLImporterModule& Get();

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static bool IsAvailable();

};

inline bool IMDLImporterModule::IsAvailable()
{
	return FModuleManager::Get().IsModuleLoaded(ModuleName);
}

inline IMDLImporterModule& IMDLImporterModule::Get()
{
	return FModuleManager::LoadModuleChecked<IMDLImporterModule>(ModuleName);
}
