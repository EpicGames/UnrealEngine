// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Engine/EngineTypes.h"
#include "UObject/ObjectMacros.h"

#include "MDLImporterOptions.generated.h"

UCLASS(config = Engine, defaultconfig)
class MDLIMPORTER_API UMDLImporterOptions : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(config, EditAnywhere, Category = "Bake options",
	          meta = (DisplayName = "Baking resolution", ToolTip = "Resolution for baking procedural textures.", ClampMin = 128, ClampMax = 16384,
	                  FixedIncrement = 2))
	uint32 BakingResolution;

	UPROPERTY(config, EditAnywhere, Category = "Bake options",
	          meta = (DisplayName = "Baking samples", ToolTip = "Samples used for baking procedural textures.", ClampMin = 1, ClampMax = 16,
	                  FixedIncrement = 2))
	uint32 BakingSamples;

	UPROPERTY(config, EditAnywhere, Category = "Advanced options",
	          meta = (DisplayName = "Resource folder", ToolTip = "Path to look for resources: textures, light profiles and other."))
	FDirectoryPath ResourcesDir;

	UPROPERTY(config, EditAnywhere, Category = "Advanced options",
	          meta = (DisplayName = "Modules folder", ToolTip = "Path to look for extra MDL modules."))
	FDirectoryPath ModulesDir;

	// hidden properties
	UPROPERTY(config, meta = (DisplayName = "Meters per scene unit", ToolTip = "The conversion ratio between meters and scene units for materials.",
	                          ClampMin = 0.01, ClampMax = 1000))
	float MetersPerSceneUnit;

	UPROPERTY(config,
	          meta = (DisplayName = "Force baking of all maps", ToolTip = "Always bakes the maps to textures instead of using material nodes."))
	bool bForceBaking;

	static FString GetMdlSystemPath();
	static FString GetMdlUserPath();
};
