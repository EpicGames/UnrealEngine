// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

class UMaterialInterface;
class UMDLImporterOptions;
class UPackage;

class MDLIMPORTER_API FMdlMaterialImporter
{
public:
	static UMaterialInterface* ImportMaterialFromModule( UPackage* ParentPackage, EObjectFlags ObjectFlags, const FString& MdlModuleName,
		const FString& MdlDefinitionName, const UMDLImporterOptions& ImporterOptions );

	static void AddSearchPath( const FString& SearchPath );
	static void RemoveSearchPath( const FString& SearchPath );

public:
	struct FScopedSearchPath
	{
		explicit FScopedSearchPath( const FString& InSearchPath )
			: SearchPath( InSearchPath )
		{
			AddSearchPath( SearchPath );
		}

		~FScopedSearchPath()
		{
			RemoveSearchPath( SearchPath );
		}

	private:
		FString SearchPath;
	};
};

namespace UE
{
	namespace Mdl
	{
		namespace Util
		{
			MDLIMPORTER_API FString ConvertFilePathToModuleName( const TCHAR* FilePath );
		}
	}
}

